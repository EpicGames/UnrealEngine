// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AlignedBlockBuffer.h"
#include "CoreMinimal.h"
#include "SignalProcessingModule.h"

namespace Audio
{
	FAlignedBlockBuffer::FAlignedBlockBuffer(int32 InSampleCapacity, int32 InMaxNumInspectSamples, uint32 InByteAlignment, uint32 InAllocByteAlignment)
		:	AllocByteAlignment(InAllocByteAlignment)
		,	ByteAlignment(InByteAlignment)
		,	FloatAlignment(0)
		,	InternalBuffer(nullptr)
		,	RolloverBuffer(nullptr)
		,	ReadPointer(nullptr)
		,	WritePointer(nullptr)
		,	SampleCapacity(InSampleCapacity)
		,	MaxNumInspectSamples(InMaxNumInspectSamples)
		,	WriteCount(0)
		,	WritePos(0)
		,	ReadPos(0)
	{
		FloatAlignment = ByteAlignment / sizeof(float);

		int32 MaxStorage = TNumericLimits<int32>::Max() / 4;

		// The SampleCapacity needs to also be divisible by the byte alignment as it is
		// used in a bunch of the pointer math.  This ensures that the ReadPointer
		// is always pointing to aligned memory
		while (0 != (SampleCapacity % FloatAlignment))
		{
			SampleCapacity++;
		}

		// Check sanity of variables.
		if (MaxNumInspectSamples > MaxStorage)
		{
			UE_LOGF(LogSignalProcessing, Warning, "MaxNumInspectSamples [%d] reduced to maximum storage [%d]", MaxNumInspectSamples, MaxStorage);
			MaxNumInspectSamples = MaxStorage;
		}

		if (SampleCapacity > MaxStorage)
		{
			UE_LOGF(LogSignalProcessing, Warning, "SampleCapacity [%d] reduced to maximum storage [%d]", SampleCapacity, MaxStorage);
			SampleCapacity = MaxStorage;
		}

		if (MaxNumInspectSamples > SampleCapacity)
		{
			UE_LOGF(LogSignalProcessing, Warning, "SampleCapacity [%d] increased to Maxmimum Inspection [%d]", SampleCapacity, MaxNumInspectSamples);
			SampleCapacity = MaxNumInspectSamples;
		}

		// Allocate space
		RolloverBuffer = AllocateAlignedFloatArray(MaxNumInspectSamples);
		InternalBuffer = AllocateAlignedFloatArray(SampleCapacity);
		
		// Set everything to zero
		ZeroFloatArray(RolloverBuffer, MaxNumInspectSamples);
		ZeroFloatArray(InternalBuffer, SampleCapacity);
		
		// Initialize pointers to internal buffers
		WritePointer = InternalBuffer;
		ReadPointer = InternalBuffer;
	}

			
	FAlignedBlockBuffer::~FAlignedBlockBuffer() throw()
	{
		FreeFloatArray(InternalBuffer);
		FreeFloatArray(RolloverBuffer);
	}

	int32 FAlignedBlockBuffer::GetSampleCapacity( ) const
	{
		return SampleCapacity;
	}
			
	int32 FAlignedBlockBuffer::GetNumAvailable() const
	{
		return WriteCount;
	}

	int32 FAlignedBlockBuffer::CheckNumAndIncrementWriteCount(int32 InNum)
	{
		// Clamp InNum to capacity
		checkf(InNum <= SampleCapacity, TEXT("Input buffer [%d] larger than capacity [%d]"), InNum, SampleCapacity);
		InNum = InNum < SampleCapacity ? InNum : SampleCapacity;

		// Increment write count keeping within capacity
		WriteCount += InNum;

		// Handle buffer overflow
		checkf(WriteCount <= SampleCapacity, TEXT("Buffer overflow"));
		if(WriteCount > SampleCapacity)
		{
			// Reduce number of input samples on buffer overflow.
			InNum -= WriteCount - SampleCapacity;
			WriteCount = SampleCapacity;
		}

		return InNum;
	}

	void FAlignedBlockBuffer::AddZeros(int32 InNum)
	{
		InNum = CheckNumAndIncrementWriteCount(InNum);
		
		// Check if we expect to be wrapping around end of input buffer
		int32 RolloverState = SampleCapacity - (WritePos + InNum);
		
		if(RolloverState > 0)
		{
			// No rollover if rollover state is above zero.
			ZeroFloatArray(WritePointer, InNum);
			WritePos += InNum;
			WritePointer += InNum;
		}
		else
		{
			// Zeroing buffer goes over internal buffer border, so perform zeroing
			// in two parts.
			
			// Zero first part of buffer
			int32 RemainingBufferSamples = InNum + RolloverState;
			ZeroFloatArray(WritePointer, RemainingBufferSamples);
			WritePointer = InternalBuffer;
			WritePos = -RolloverState;
			
			// Zero the second part
			ZeroFloatArray(WritePointer, -RolloverState);
			WritePointer -= RolloverState;
		}
	}
			
	void FAlignedBlockBuffer::AddSamples(const float* InSamples, int32 InNum)
	{
		checkf(nullptr != InSamples, TEXT("Null pointer"));
		if (nullptr == InSamples)
		{
			// Quit early with null pointers.
			return;
		}

		InNum = CheckNumAndIncrementWriteCount(InNum);
		
		// If the write operation is expected to go past the internal buffer boundary
		// the RolloverState will be negative. If RolloverState is negative, the copy
		// needs to happen in two parts. 
		int32 RolloverState = SampleCapacity - (WritePos + InNum);
		
		if(RolloverState > 0)
		{
			// No rollover if rollover state above zero.
			CopyFloatArray(WritePointer, InSamples, InNum);
			
			WritePos += InNum;
			WritePointer += InNum;
		}
		else
		{
			//Copy first part of buffer
			int32 RemainingBufferSamples = InNum + RolloverState;
		
			CopyFloatArray(WritePointer, InSamples, RemainingBufferSamples);
			
			WritePointer = InternalBuffer;
			WritePos = -RolloverState;
			InSamples += RemainingBufferSamples;
			
			//Copy the second part
			CopyFloatArray(WritePointer, InSamples, -RolloverState);

			WritePointer -= RolloverState;
		}
	}

	void FAlignedBlockBuffer::RemoveSamples(int32 InNum)
	{
		// In order to keep the read pointer aligned, any operations that move the read pointer
		// must also be aligned. 
		checkf(0 == (InNum % FloatAlignment), TEXT("InNum must be divisible by alignement."));

		//decrement the available samples
		WriteCount -= InNum;

		checkf(WriteCount >= 0, TEXT("Buffer underflow"));
		if (WriteCount < 0)
		{
			// Gracefully handle underflow.
			WriteCount = 0;
			ReadPos = 0;
			ReadPointer = InternalBuffer;
			return;
		}

		int32 RolloverState = SampleCapacity - (ReadPos + InNum);
		
		if( RolloverState > 0 )
		{
			ReadPos += InNum;
			ReadPointer += InNum;
		}
		else
		{
			// Wrap the get pointer around in case where buffer boundary is crossed.
			ReadPos = -RolloverState;
			ReadPointer = InternalBuffer - RolloverState;
		}
	}
			
	const float* FAlignedBlockBuffer::InspectSamples(int32 InNum, int32 InOffset)
	{
		checkf(0 == (InOffset % FloatAlignment), TEXT("InOffset must divisible by alignment"));
		checkf(InNum <= MaxNumInspectSamples, TEXT("Requested samples [%d] are more than maximum [%d]"), InNum, MaxNumInspectSamples);
		checkf((InNum + InOffset) <= WriteCount, TEXT("Buffer underflow"));
		checkf(InOffset >= 0, TEXT("InOffset must be greater than 0"));

		if (InNum < 1)
		{
			return nullptr;
		}

		if (InNum > MaxNumInspectSamples)
		{
			// This is an unfortunate situation because it allocates memory.
			SetMaxNumInspectSamples(InNum);
		}

		if ((InNum + InOffset) > WriteCount)
		{
			// This is a pretty bad situation. Anything done here is a bad option.
			// Hopefully the lesser of all the evils is returning a nullptr so that the issue
			// can be identified quickly. 
			return nullptr;
		}

		if (InOffset < 0)
		{
			// Invalid offset. Despite the profound implications for the advancement of humankind, we cannot see into the future. 
			InOffset = 0;
		}
		
		int32 StartPos = ReadPos + InOffset;
		int32 EndPos = StartPos + InNum;

		while (StartPos > SampleCapacity)
		{
			StartPos -= SampleCapacity;
			EndPos -= SampleCapacity;
		}

		if (EndPos < SampleCapacity)
		{
			return &InternalBuffer[StartPos];
		}
		else
		{
			int32 NumFirstHalf = SampleCapacity - StartPos;
			// Copy first half into rolloverbuffer
			CopyFloatArray(RolloverBuffer, &InternalBuffer[StartPos], NumFirstHalf);
			// Copy second half into rolloverbuffer
			CopyFloatArray(&RolloverBuffer[NumFirstHalf], InternalBuffer, InNum - NumFirstHalf);

			return RolloverBuffer;
		}
	}
			
	void FAlignedBlockBuffer::ClearSamples()
	{
		ZeroFloatArray(InternalBuffer, SampleCapacity);
		ZeroFloatArray(RolloverBuffer, MaxNumInspectSamples);

		ReadPointer = InternalBuffer;
		WritePointer = InternalBuffer;
		ReadPos = 0;
		WritePos = 0;
		
		WriteCount = 0;
	}
					
	int32 FAlignedBlockBuffer::GetMaxNumInspectSamples() const
	{
		return MaxNumInspectSamples;
	}

	void FAlignedBlockBuffer::SetMaxNumInspectSamples(int32 InMax)
	{
		if(InMax > MaxNumInspectSamples)
		{
			FreeFloatArray(RolloverBuffer);
			RolloverBuffer = AllocateAlignedFloatArray(InMax);
		}
		
		MaxNumInspectSamples = InMax;
	}

	float* FAlignedBlockBuffer::AllocateAlignedFloatArray(int32 InNum) const 
	{
		int32 Size = InNum * sizeof(float);
		return (float*)FMemory::Malloc(Size, AllocByteAlignment);
	}

	void FAlignedBlockBuffer::FreeFloatArray(float*& Array) const
	{
		FMemory::Free(Array);
		Array = nullptr;
	}

	void FAlignedBlockBuffer::ZeroFloatArray(float* InArray, int32 InNum) const
	{
		if (InNum > 0) 
		{
			int32 Size = InNum * sizeof(float);
			FMemory::Memset(InArray, 0, Size);
		}
	}

	void FAlignedBlockBuffer::CopyFloatArray(float* ToArray, const float* FromArray, int32 InNum) const 
	{
		if (InNum > 0)
		{
			int32 Size = InNum * sizeof(float);
			FMemory::Memcpy(ToArray, FromArray, Size);
		}
	}
}
