// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AllPassFilter.h"

namespace Audio
{
	FDelayAPF::FDelayAPF()
		: G(0.0f)
	{
	}

	FDelayAPF::~FDelayAPF()
	{
	}

	float FDelayAPF::ProcessAudioSample(const float InputSample)
	{
		// Read the delay line to get w(n-D);
		const float WnD = this->Read();

		// For the APF if the delay is 0.0 we just need to pass input -> output
		if (ReadIndex == WriteIndex)
		{
			this->WriteDelayAndInc(InputSample);
			return InputSample;
		}

		// Form w(n) = x(n) + gw(n-D)
		const float Wn = InputSample + G*WnD;

		// form y(n) = -gw(n) + w(n-D)
		const float Yn = -G*Wn + WnD;

		this->WriteDelayAndInc(Wn);
		return Yn;
	}

}
