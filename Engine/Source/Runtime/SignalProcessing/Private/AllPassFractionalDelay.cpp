// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AllPassFractionalDelay.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FAllPassFractionalDelay::FAllPassFractionalDelay(int32 InMaxDelay, int32 InNumInternalBufferSamples)
	: MaxDelay(InMaxDelay)
	, NumInternalBufferSamples(InNumInternalBufferSamples)
	, Z1(0.0f)
	{
		checkf(MaxDelay > 0, TEXT("Maximum delay must be greater than zero"));
		checkf(InNumInternalBufferSamples > 0, TEXT("Internal buffer length must be greater than zero"));

		// Clamp settings to avoid hard crashes during runtime.
		if (MaxDelay < 1)
		{
			MaxDelay = 1;
		}
		if (InNumInternalBufferSamples < 1)
		{
			InNumInternalBufferSamples = 32;
		}

		// Allocate and prepare delay line for maximum delay.
		DelayLine = MakeUnique<FAlignedBlockBuffer>((2 * MaxDelay) + NumInternalBufferSamples, MaxDelay + NumInternalBufferSamples);
		DelayLine->AddZeros(MaxDelay);

		Coefficients.Reset(NumInternalBufferSamples);
		FractionalDelays.Reset(NumInternalBufferSamples);
		IntegerDelays.Reset(NumInternalBufferSamples);
		IntegerDelayOffsets.Reset(NumInternalBufferSamples);

		Coefficients.AddUninitialized(NumInternalBufferSamples);
		FractionalDelays.AddUninitialized(NumInternalBufferSamples);
		IntegerDelays.AddUninitialized(NumInternalBufferSamples);
		IntegerDelayOffsets.AddUninitialized(NumInternalBufferSamples);

		// Integer delay offsets account for buffer position when doing block processing of data. 
		for (int32 i = 0; i < InNumInternalBufferSamples; i++)
		{
			IntegerDelayOffsets[i] = i + MaxDelay;
		}
	}


	// Destructor
	FAllPassFractionalDelay::~FAllPassFractionalDelay()
	{}

	// Resets the delay line state, flushes buffer and resets read/write pointers.
	void FAllPassFractionalDelay::Reset() 
	{
		DelayLine->ClearSamples();
		DelayLine->AddZeros(MaxDelay);
		Z1 = 0.0f;
	}


	void FAllPassFractionalDelay::ProcessAudio(const FAlignedFloatBuffer& InSamples, const FAlignedFloatBuffer& InDelays, FAlignedFloatBuffer& OutSamples)
	{
		const int32 InNum = InSamples.Num();
		checkf(InNum == InDelays.Num(), TEXT("Input buffers must be equal length."));

		// Prepare output buffer
		OutSamples.Reset(InNum);
		OutSamples.AddUninitialized(InNum);

		float* OutSampleData = OutSamples.GetData();
		const float* InSampleData = InSamples.GetData();
		const float* InDelayData = InDelays.GetData();

		// Process audio one block at a time.
		int32 LeftOver = InNum;
		int32 BufferPos = 0;
		while (LeftOver > 0)
		{
			int32 NumToProcess = FMath::Min(LeftOver, NumInternalBufferSamples);
			ProcessAudioBlock(&InSampleData[BufferPos], &InDelayData[BufferPos], NumToProcess, &OutSampleData[BufferPos]);
			BufferPos += NumToProcess;
			LeftOver -= NumToProcess;
		}
	}

	void FAllPassFractionalDelay::ProcessAudioBlock(const float* InSamples, const float* InDelays, const int32 InNum, float* OutSamples)
	{
		checkf(0 == (InNum % 4), TEXT("Array length must be multiple of 4"));

		// All these assume that InNum <= NumInternalBufferSamples
		const int32* IntegerDelayOffsetData = IntegerDelayOffsets.GetData();
		int32* IntegerDelayData = IntegerDelays.GetData();
		float* FractionalDelayData = FractionalDelays.GetData();
		float* CoefficientsData = Coefficients.GetData();

		FMemory::Memcpy(FractionalDelayData, InDelays, InNum * sizeof(float));
		
		// Determine integer delays and filter coefficients per a sample
		const VectorRegister4Float VTwoAndAHalf = MakeVectorRegister(2.5f, 2.5f, 2.5f, 2.5f);
		const VectorRegister4Float VMaxDelay = MakeVectorRegister((float)MaxDelay, (float)MaxDelay, (float)MaxDelay, (float)MaxDelay);
		for (int32 i = 0; i < InNum; i += 4)
		{
			checkf(FractionalDelayData[i] <= MaxDelay, TEXT("Delay exceeds maximum"));
			checkf(FractionalDelayData[i + 1] <= MaxDelay, TEXT("Delay exceeds maximum"));
			checkf(FractionalDelayData[i + 2] <= MaxDelay, TEXT("Delay exceeds maximum"));
			checkf(FractionalDelayData[i + 3] <= MaxDelay, TEXT("Delay exceeds maximum"));

			// Ensure that delays are in [0.5, MaxDelay]
			// Delay= Max(Delay, 0.5)
			VectorRegister4Float VFractionalDelays = VectorLoad(&FractionalDelayData[i]);
			VFractionalDelays = VectorMax(VFractionalDelays, GlobalVectorConstants::FloatOneHalf);
			// Delay = Min(Delay, MaxDelay)
			VFractionalDelays = VectorMin(VFractionalDelays, VMaxDelay);

			// To ensure fractional delays between -0.5 - 0.5, subtract limit then add back in.
			// Delay = Delay + 0.5
			VFractionalDelays = VectorAdd(VFractionalDelays, GlobalVectorConstants::FloatOneHalf);
			// Delay.floor = Floor(Delay)
			VectorRegister4Float VFloorDelays = VectorFloor(VFractionalDelays);
			// Delay.frac = Delay - Delay.floor
			VFractionalDelays = VectorSubtract(VFractionalDelays, VFloorDelays);
			
			// Reintroduing 0.5 previously removed.
			// alpha = Delay.frac - 0.5
			VectorRegister4Float VCoefficients = VectorSubtract(VFractionalDelays, GlobalVectorConstants::FloatOneHalf);
			// denom = 2.5 - alpha
			VectorRegister4Float VDenominator = VectorSubtract(VTwoAndAHalf, VFractionalDelays);
			// coef = alpha / (2.5 - alpha)
			VCoefficients = VectorDivide(VCoefficients, VDenominator);
			VectorStore(VCoefficients, &Coefficients[i]);

			// Delay.int = int(Delay.floor)
			VectorRegister4Int VIntegerDelays = VectorFloatToInt(VFloorDelays);
			VectorRegister4Int VIntegerDelayOffset = VectorIntLoadAligned(&IntegerDelayOffsetData[i]);
			// Delay.int += BufferIdx
			VectorIntStoreAligned(VectorIntSubtract(VIntegerDelayOffset, VIntegerDelays), &IntegerDelayData[i]);
		}

		// Update delay line.
		DelayLine->AddSamples(InSamples, InNum);

		// All pass filter delay line to get fractional delay
		const float* DelayData = DelayLine->InspectSamples(InNum + MaxDelay);
		int32 DelayPos;
		if (InNum > 0)
		{ 
			DelayPos = IntegerDelayData[0];
			OutSamples[0] = DelayData[DelayPos - 1] + CoefficientsData[0] * DelayData[DelayPos] - CoefficientsData[0] * Z1;
			for (int32 i = 1; i < InNum; i++)
			{
				DelayPos = IntegerDelayData[i];
				OutSamples[i] = DelayData[DelayPos - 1] + CoefficientsData[i] * DelayData[DelayPos] - CoefficientsData[i] * OutSamples[i - 1];
			}

			Z1 = OutSamples[InNum - 1];
		}
		
		// Remove unneeded delay line.
		DelayLine->RemoveSamples(InNum);
	}
}
