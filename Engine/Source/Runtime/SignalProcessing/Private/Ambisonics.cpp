// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Ambisonics.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
void EncodeMonoAmbisonicMixIn(
	TArrayView<const float> Src, TArrayView<float*> Dst, TArrayView<const float> AmbisonicGains)
{
	checkSlow(Dst.Num() == AmbisonicGains.Num());
			
	const int32 NumFrames = Src.Num();
	const int32 NumAmbisonicChannels = AmbisonicGains.Num();
	const float* Gains = AmbisonicGains.GetData();
	float** DstChannels = Dst.GetData();
			
	for (int32 ChannelIndex = 0; ChannelIndex < NumAmbisonicChannels; ++ChannelIndex)
	{
		ArrayMixIn(Src, MakeArrayView(DstChannels[ChannelIndex], NumFrames), Gains[ChannelIndex]);
	}
}
void DecodeMonoAmbisonicMixIn(
	TArrayView<const float*> Src, TArrayView<float> Dst, TArrayView<const float> AmbisonicGains)
{
	checkSlow(Src.Num() == AmbisonicGains.Num());
			
	const int32 NumFrames = Dst.Num();
	const int32 NumAmbisonicChannels = Src.Num();
	const float* Gains = AmbisonicGains.GetData();
	const float** SrcChannels = Src.GetData();

	for (int32 ChannelIndex = 0; ChannelIndex < NumAmbisonicChannels; ++ChannelIndex)
	{
		ArrayMixIn(MakeArrayView(SrcChannels[ChannelIndex], NumFrames), Dst, Gains[ChannelIndex]);
	}
}
}