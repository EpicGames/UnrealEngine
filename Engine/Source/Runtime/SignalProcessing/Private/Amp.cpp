// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Amp.h"
#include "DSP/Dsp.h"

namespace Audio
{
	FAmp::FAmp()
		: VoiceId(0)
		, LeftGain(0.0f)
		, RightGain(0.0f)
		, TargetLeftGain(0.0f)
		, TargetRightGain(0.0f)
		, TargetDeltaSamples(0)
		, CurrentLerpSample(0)
		, TargetLeftSlope(0.0f)
		, TargetRightSlope(0.0f)
		, GainMin(0.0f)
		, GainMax(1.0f)
		, GainControl(1.0f)
		, GainVelocity(1.0f)
		, GainMod(1.0f)
		, GainEnv(1.0f)
		, Pan(0.0f)
		, PanMod(0.0f)
		, ModMatrix(nullptr)
		, bChanged(false)
	{
	}

	FAmp::~FAmp()
	{
	}

	void FAmp::Init(const int32 InVoiceId, FModulationMatrix* InModMatrix)
	{
		VoiceId = InVoiceId;
		ModMatrix = InModMatrix;

		// Number of samples to interpolate
		TargetDeltaSamples = 256;

		if (ModMatrix)
		{
			GainScaleDest = ModMatrix->CreatePatchDestination(VoiceId, 1, 1.0f);
			GainEnvDest = ModMatrix->CreatePatchDestination(VoiceId, 1, 1.0f);
			GainPanDest = ModMatrix->CreatePatchDestination(VoiceId, 1, 1.0f);

			GainScaleDest.SetName("GainScaleDest");

#if MOD_MATRIX_DEBUG_NAMES
			GainScaleDest.Name = TEXT("GainScaleDest");
			GainEnvDest.Name = TEXT("GainEnvDest");
			GainPanDest.Name = TEXT("GainPanDest");
#endif
		}
		bChanged = true;
	}

	void FAmp::SetGainDb(const float InGainDb)
	{
		GainControl = ConvertToLinear(InGainDb);
		bChanged = true;
	}

	void FAmp::SetGainModDb(const float InGainModDb)
	{
		GainMod = ConvertToLinear(InGainModDb);
		GainMod = GetUnipolar(GainMod);
		bChanged = true;
	}

	void FAmp::SetGain(const float InGainLinear)
	{
		GainControl = InGainLinear;
		bChanged = true;
	}

	void FAmp::SetGainMod(const float InBipolarGainModLinear)
	{
		GainMod = GetUnipolar(InBipolarGainModLinear);
		bChanged = true;
	}

	void FAmp::SetGainEnv(const float InGainEnv)
	{
		GainEnv = InGainEnv;
		bChanged = true;
	}

	void FAmp::SetGainEnvDb(const float InGainEnvDb)
	{
		GainEnv = ConvertToLinear(InGainEnvDb);
	}

	void FAmp::SetGainRange(const float InMin, const float InMax)
	{
		GainMin = InMin;
		GainMax = InMax;
		bChanged = true;
	}

	void FAmp::SetVelocity(const float InVelocity)
	{
		GainVelocity = GetGainFromVelocity(InVelocity);
		bChanged = true;
	}

	void FAmp::SetPan(const float InPan)
	{
		Pan = InPan;
		bChanged = true;
	}

	void FAmp::SetPanModulator(const float InPanMod)
	{
		PanMod = InPanMod;
		bChanged = true;
	}

	void FAmp::Update()
	{
		if (ModMatrix)
		{
			bChanged |= ModMatrix->GetDestinationValue(VoiceId, GainScaleDest, GainMod);
			bChanged |= ModMatrix->GetDestinationValue(VoiceId, GainEnvDest, GainEnv);
			bChanged |= ModMatrix->GetDestinationValue(VoiceId, GainPanDest, PanMod);
		}

		if (bChanged)
		{
			bChanged = false;

			// Update the pan values
			const float PanSum = FMath::Clamp(Pan + PanMod, -1.0f, 1.0f);
			float PanLeft = 0.0f;
			float PanRight = 0.0f;
			GetStereoPan(PanSum, PanLeft, PanRight);

			// Compute the gain product accounting for the gain stages
			// TODO: look at this more closely
			float GainProduct = GainControl * GainMod * GainVelocity * GainEnv;
			
			GainProduct = FMath::Clamp(GainProduct, GainMin, GainMax);

			// Mix in the pan value
			TargetLeftGain = GainProduct * PanLeft;
			TargetRightGain = GainProduct * PanRight;

			CurrentLerpSample = 0;
			TargetLeftSlope = (TargetLeftGain - LeftGain) / TargetDeltaSamples;
			TargetRightSlope = (TargetRightGain - RightGain) / TargetDeltaSamples;
		}
	}

	void FAmp::Generate(float& OutGainLeft, float& OutGainRight)
	{
		if (CurrentLerpSample < TargetDeltaSamples)
		{
			LeftGain += TargetLeftSlope;
			RightGain += TargetRightSlope;
			++CurrentLerpSample;
		}

		OutGainLeft *= LeftGain;
		OutGainRight *= RightGain;
	}

	void FAmp::ProcessAudio(const float LeftIn, float* LeftOutput, float* RightOutput)
	{
		if (CurrentLerpSample < TargetDeltaSamples)
		{
			LeftGain += TargetLeftSlope;
			RightGain += TargetRightSlope;
			++CurrentLerpSample;
		}

		*LeftOutput = (LeftIn * LeftGain);
		*RightOutput = (LeftIn * RightGain);
	}

	void FAmp::ProcessAudio(const float LeftIn, const float RightIn, float* LeftOutput, float* RightOutput)
	{
		if (CurrentLerpSample < TargetDeltaSamples)
		{
			LeftGain += TargetLeftSlope;
			RightGain += TargetRightSlope;
			++CurrentLerpSample;
		}

		*LeftOutput = (LeftIn * LeftGain);
		*RightOutput = (RightIn * RightGain);
	}

	void FAmp::Reset()
	{
		GainEnv = 1.0f;
		GainMod = 1.0f;
	}

}
