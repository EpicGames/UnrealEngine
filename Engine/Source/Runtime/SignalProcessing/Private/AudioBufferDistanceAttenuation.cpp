// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AudioBufferDistanceAttenuation.h"
#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	static float ComputeNextLinearAttenuation(const FAudioBufferDistanceAttenuationSettings& InSettings, float InCurrentDistance)
	{
		float NextAttenuationDb = 0.0f;
		const float Denom = FMath::Max(InSettings.DistanceRange.Y - InSettings.DistanceRange.X, SMALL_NUMBER);
		const float Alpha = FMath::Clamp((InCurrentDistance - InSettings.DistanceRange.X) / Denom, 0.0f, 1.0f);

		float CurveValue = 0.0f;

		if (InSettings.AttenuationCurve.Num() > 0)
		{
			bool bSuccess = InSettings.AttenuationCurve.Eval(Alpha, CurveValue);
			// This should succeed since we ensure we always have at least two points in the curve when it's set
			check(bSuccess);
		}
		else
		{
			// We do a linear attenuation if there is no curve
			CurveValue = 1.0f - Alpha;
		}


		// Note the curve is expected to map to the attenuation amount (i.e. at right-most value, it'll be 0.0, which corresponds to max dB attenuation)
		// This then needs to be used to interpolate the dB range (0.0 is no attenuation, -60 for example, is a lot of attenuation)
		NextAttenuationDb = FMath::Lerp(InSettings.AttenuationDbAtMaxRange, 0.0f, CurveValue);

		float TargetAttenuationLinear = 0.0f;
		if (NextAttenuationDb > InSettings.AttenuationDbAtMaxRange)
		{
			TargetAttenuationLinear = Audio::ConvertToLinear(NextAttenuationDb);
		}
		return TargetAttenuationLinear;
	}


	void DistanceAttenuationProcessAudio(TArrayView<int16>& InOutBuffer, uint32 InNumChannels, float InDistance, const FAudioBufferDistanceAttenuationSettings& InSettings, float& InOutAttenuation)
	{
		check(InOutBuffer.Num() > 0);
		check(InNumChannels > 0);
		check(InDistance >= 0.0f);

		int32 FrameCount = InOutBuffer.Num() / InNumChannels;
		float TargetAttenuationLinear = ComputeNextLinearAttenuation(InSettings, InDistance);

		// TODO: investigate adding int16 flavors of utilities in BufferVectorOperations.h to avoid format conversions
		uint32 CurrentSampleIndex = 0;

		// If we're passed in a negative value for InOutAttenuation, that means we don't want to interpolate from that value to target value (i.e. it's the first one).
		// This prevents a pop when first applying attenuation if a sound is far away.
		float Gain = InOutAttenuation < 0.0f ? TargetAttenuationLinear : InOutAttenuation;
		const float DeltaValue = (TargetAttenuationLinear - Gain) / FrameCount;

		for (int32 FrameIndex = 0; FrameIndex < FrameCount; ++FrameIndex)
		{
			for (uint32 ChannelIndex = 0; ChannelIndex < InNumChannels; ++ChannelIndex)
			{
				InOutBuffer[CurrentSampleIndex + ChannelIndex] = InOutBuffer[CurrentSampleIndex + ChannelIndex] * Gain;
			}

			CurrentSampleIndex += InNumChannels;
			Gain += DeltaValue;
		}

		// Update the current attenuation linear for the next render block
		InOutAttenuation = TargetAttenuationLinear;
	}

	void DistanceAttenuationProcessAudio(TArrayView<float>& InOutBuffer, uint32 InNumChannels, float InDistance, const FAudioBufferDistanceAttenuationSettings& InSettings, float& InOutAttenuation)
	{
		check(InOutBuffer.Num() > 0);
		check(InNumChannels > 0);
		check(InDistance >= 0.0f);

		int32 FrameCount = InOutBuffer.Num() / InNumChannels;
		float TargetAttenuationLinear = ComputeNextLinearAttenuation(InSettings, InDistance);

		int32 NumSamples = (int32)(FrameCount * InNumChannels);
		// If we're passed in a negative value for InOutAttenuation, that means we don't want to interpolate from that value to target value (i.e. it's the first one).
		// This prevents a pop when first applying attenuation if a sound is far away.
		float Gain = InOutAttenuation < 0.0f ? TargetAttenuationLinear : InOutAttenuation;
		TArrayView<float> InOutBufferView(InOutBuffer.GetData(), NumSamples);
		Audio::ArrayFade(InOutBufferView, Gain, TargetAttenuationLinear);

		InOutAttenuation = TargetAttenuationLinear;
	}
}
