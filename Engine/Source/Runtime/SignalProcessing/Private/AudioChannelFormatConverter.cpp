// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AudioChannelFormatConverter.h"

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "SignalProcessingModule.h"

namespace Audio
{
	FBaseChannelFormatConverter::~FBaseChannelFormatConverter() = default;

	const IChannelFormatConverter::FInputFormat& FBaseChannelFormatConverter::GetInputFormat() const
	{
		return InputFormat;
	}

	const IChannelFormatConverter::FOutputFormat& FBaseChannelFormatConverter::GetOutputFormat() const
	{
		return OutputFormat;
	}

	void FBaseChannelFormatConverter::SetOutputGain(float InOutputGain, bool bFadeToGain)
	{
		OutputGainState.bFadeToNextGain = bFadeToGain;

		if (bFadeToGain)
		{
			// If fading, set as the next gain to fade to.
			OutputGainState.NextGain = InOutputGain;
		}
		else
		{
			OutputGainState.Gain = InOutputGain;
		}
	}

	void FBaseChannelFormatConverter::SetMixGain(const FChannelMixEntry& InEntry, bool bFadeToGain)
	{
		SetMixGain(InEntry.InputChannelIndex, InEntry.OutputChannelIndex, InEntry.Gain, bFadeToGain);
	}

	void FBaseChannelFormatConverter::SetMixGain(int32 InInputChannelIndex, int32 InOutputChannelIndex, float InGain, bool bFadeToGain)
	{
		FChannelMixKey Key(InInputChannelIndex, InOutputChannelIndex);

		if ((InInputChannelIndex >= InputFormat.NumChannels) || (InInputChannelIndex < 0))
		{
			UE_LOGF(LogSignalProcessing, Warning, "Skipping mix entry. Input channel (%d) does not exist for input format with %d channels.", InInputChannelIndex, InputFormat.NumChannels);
			return;
		}

		if ((InOutputChannelIndex >= OutputFormat.NumChannels) || (InOutputChannelIndex < 0))
		{
			UE_LOGF(LogSignalProcessing, Warning, "Skipping mix entry. Output channel (%d) does not exist for output format with %d channels.", InOutputChannelIndex, OutputFormat.NumChannels);
			return;
		}

		if ((InGain == 0.f) && !bFadeToGain)
		{
			// Remove a mix state if it has zero gain in order to avoid extra
			// processing of gain entries. 
			ChannelMixStates.Remove(Key);
		}
		else
		{
			FChannelMixState* State = ChannelMixStates.Find(Key);

			if (nullptr == State)
			{
				// No existing gain state for the input/output chnannel pair.
				// One is created here. 
				FChannelMixState& NewState = ChannelMixStates.Add(Key);
				
				NewState.InputChannelIndex = InInputChannelIndex;
				NewState.OutputChannelIndex = InOutputChannelIndex;
				NewState.Gain = 0.f;

				State = &NewState;
			}
			
			check(nullptr != State);

			State->bFadeToNextGain = bFadeToGain;

			if (bFadeToGain)
			{
				// Setup next gain if fading.
				State->NextGain = InGain;
			}
			else
			{
				// Set current gain if not fading.
				State->Gain = InGain;
			}
		}
	}

	float FBaseChannelFormatConverter::GetTargetMixGain(int32 InInputChannelIndex, int32 InOutputChannelIndex) const
	{
		FChannelMixKey Key(InInputChannelIndex, InOutputChannelIndex);

		if (ChannelMixStates.Contains(Key))
		{
			const FChannelMixState& State = ChannelMixStates.FindChecked(Key);

			// Check whether to return next gain or current gain.
			// "NextGain" is meaningless if bFadeToNextGain == false.
			return State.bFadeToNextGain ? State.NextGain : State.Gain;
		}

		return 0.f;
	}

	float FBaseChannelFormatConverter::GetTargetOutputGain() const
	{
		// Check whether to return next gain or current gain.
		// "NextGain" is meaningless if bFadeToNextGain == false.
		return OutputGainState.bFadeToNextGain ? OutputGainState.NextGain : OutputGainState.Gain;
	}

	void FBaseChannelFormatConverter::ProcessAudio(const TArray<FAlignedFloatBuffer>& InInputBuffers, TArray<FAlignedFloatBuffer>& OutOutputBuffers)
	{
		using FMixElement = TSortedMap<FChannelMixKey, FChannelMixState>::ElementType;

		check(InInputBuffers.Num() == InputFormat.NumChannels);

		// Ensure output buffers exist in output array.
		while (OutOutputBuffers.Num() < OutputFormat.NumChannels)
		{
			OutOutputBuffers.Emplace();
		}

		for (int32 i = 0; i < OutputFormat.NumChannels; i++)
		{
			// Allocate output buffers to the correct size. 
			OutOutputBuffers[i].Reset();
			OutOutputBuffers[i].AddUninitialized(NumFramesPerCall);

			// Zero out data in output buffers. 
			FMemory::Memset(OutOutputBuffers[i].GetData(), 0, sizeof(float) * NumFramesPerCall);
		}

		// Cache current input buffer information to minimize duplicate processing
		// and improve cache performance.  MixEntries are sorted by input buffer.
		// All mix entries for a single input index will be processed sequentially
		// before the next input index is processed. The `CurrentInputChannelIndex`
		// is used to determine whether the input channel in a mix entry differs
		// from the previous. If so, the "Current" input channel is updated.
		int32 CurrentInputChannelIndex = INDEX_NONE;
		const FAlignedFloatBuffer* CurrentInputChannel = nullptr;

		TArray<FChannelMixKey> EntriesToRemove;

		// Process all mix entries.
		for (FMixElement& Element : ChannelMixStates)
		{
			FChannelMixState& MixState = Element.Value;

			// Check if the input index differs from the previous. If so, update
			// "Current" input index.
			if (MixState.InputChannelIndex != CurrentInputChannelIndex)
			{
				// Check validity of input channel.
				const FAlignedFloatBuffer& InputBuffer = InInputBuffers[MixState.InputChannelIndex];

				if (ensure(InputBuffer.Num() == NumFramesPerCall))
				{
					CurrentInputChannelIndex = MixState.InputChannelIndex;
					CurrentInputChannel = &InputBuffer;
				}
				else
				{
					UE_LOGF(LogSignalProcessing, Warning, "Input buffer frame count (%d) does not match expected frame count (%d)", InputBuffer.Num(), NumFramesPerCall);

					CurrentInputChannelIndex = INDEX_NONE;
					CurrentInputChannel = nullptr;

					continue;
				}
			}

			check(nullptr != CurrentInputChannel);

			// Get gain values for mix entry. Include the output gain.
			const float InitialGain = MixState.Gain * OutputGainState.Gain;
			const float FinalMixGain = MixState.bFadeToNextGain ? MixState.NextGain : MixState.Gain;
			const float FinalOutputGain = OutputGainState.bFadeToNextGain ? OutputGainState.NextGain : OutputGainState.Gain;
			const float FinalGain = FinalMixGain * FinalOutputGain;

			if (FMath::IsNearlyEqual(InitialGain, FinalGain, 0.000001f))
			{
				// No fade is needed because gain is constant. 
				ArrayMultiplyAddInPlace(*CurrentInputChannel, FinalGain, OutOutputBuffers[MixState.OutputChannelIndex]);
			}
			else 
			{
				// Fade required because gain changes. 
				ArrayLerpAddInPlace(*CurrentInputChannel, InitialGain, FinalGain, OutOutputBuffers[MixState.OutputChannelIndex]);
			}

			if (MixState.bFadeToNextGain)
			{
				// Update gain state if fade has been processed. 
				MixState.bFadeToNextGain = false;
				MixState.Gain = MixState.NextGain;

				if (0.f == MixState.Gain)
				{
					// Remove mix states with zero gain.
					EntriesToRemove.Emplace(MixState);
				}
			}
		}

		if (OutputGainState.bFadeToNextGain)
		{
			// Update gain state if fade has been processed. 
			OutputGainState.Gain = OutputGainState.NextGain;
			OutputGainState.bFadeToNextGain = false;
		}

		for (const FChannelMixKey& Key : EntriesToRemove)
		{
			// Remove mix states with zero gain.
			ChannelMixStates.Remove(Key);
		}
	}

	TUniquePtr<FBaseChannelFormatConverter> FBaseChannelFormatConverter::CreateBaseFormatConverter(const FInputFormat& InInputFormat, const FOutputFormat& InOutputFormat, TArrayView<const FChannelMixEntry> InMixEntries, int32 InNumFramesPerCall)
	{
		if (InInputFormat.NumChannels < 1)
		{
			UE_LOGF(LogSignalProcessing, Error, "Invalid input format channel count (%d). Must be greater than zero", InInputFormat.NumChannels);
			return TUniquePtr<FBaseChannelFormatConverter>(nullptr);
		}

		if (InOutputFormat.NumChannels < 1) 
		{
			UE_LOGF(LogSignalProcessing, Error, "Invalid output format channel count (%d). Must be greater than zero", InOutputFormat.NumChannels);
			return TUniquePtr<FBaseChannelFormatConverter>(nullptr);
		}

		if (InNumFramesPerCall < 1)
		{
			UE_LOGF(LogSignalProcessing, Error, "Invalid num frames per call (%d). Must be greater than zero", InNumFramesPerCall);
			return TUniquePtr<FBaseChannelFormatConverter>(nullptr);
		}

		return TUniquePtr<FBaseChannelFormatConverter>(new FBaseChannelFormatConverter(InInputFormat, InOutputFormat, InMixEntries, InNumFramesPerCall));
	}

	FBaseChannelFormatConverter::FBaseChannelFormatConverter(const FInputFormat& InInputFormat, const FOutputFormat& InOutputFormat, TArrayView<const FChannelMixEntry> InMixEntries, int32 InNumFramesPerCall)
	:	InputFormat(InInputFormat)
	,	OutputFormat(InOutputFormat)
	,	NumFramesPerCall(InNumFramesPerCall)
	{
		check(InputFormat.NumChannels > 0);
		check(OutputFormat.NumChannels > 0);
		check(InNumFramesPerCall > 0);

		const bool bFadeToGain = false;

		for (const FChannelMixEntry& Entry : InMixEntries)
		{
			SetMixGain(Entry, bFadeToGain);
		}
	}
}
