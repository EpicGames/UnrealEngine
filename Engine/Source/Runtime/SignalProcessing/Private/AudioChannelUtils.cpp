// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AudioChannelUtils.h"

#include "AudioMixer.h"
#include "Misc/ConfigCacheIni.h"

namespace Audio::ChannelUtils
{	
	bool GetChannelTypeAtIndex(const int32 Index, EAudioMixerChannel::Type& OutType)
	{
		/** The default channel orderings to use when using pro audio interfaces while still supporting surround sound. */
		auto CreateDefaultChannelOrder = []()
		{
			// Create a hard-coded default channel order
			TArray<EAudioMixerChannel::Type> DefaultChannelOrder
			{
				EAudioMixerChannel::FrontLeft,
				EAudioMixerChannel::FrontRight,
				EAudioMixerChannel::FrontCenter,
				EAudioMixerChannel::LowFrequency,
				EAudioMixerChannel::SideLeft,
				EAudioMixerChannel::SideRight,
				EAudioMixerChannel::BackLeft,
				EAudioMixerChannel::BackRight
			};

			TArray<EAudioMixerChannel::Type> ChannelMapOverride = DefaultChannelOrder;

			check(DefaultChannelOrder.Num() == AUDIO_MIXER_MAX_OUTPUT_CHANNELS);
			check(ChannelMapOverride.Num()  == AUDIO_MIXER_MAX_OUTPUT_CHANNELS);

			// Now check the ini file to see if this is overridden
			bool bOverridden = false;

			for (int32 i = 0; i < AUDIO_MIXER_MAX_OUTPUT_CHANNELS; ++i)
			{
				int32 ChannelPositionOverride = 0;

				const TCHAR* ChannelName = EAudioMixerChannel::ToString(DefaultChannelOrder[i]);

				if (GConfig->GetInt(TEXT("AudioDefaultChannelOrder"), ChannelName, ChannelPositionOverride, GEngineIni))
				{
					if (ChannelPositionOverride >= 0 && ChannelPositionOverride < AUDIO_MIXER_MAX_OUTPUT_CHANNELS)
					{
						bOverridden = true;
						ChannelMapOverride[ChannelPositionOverride] = DefaultChannelOrder[i];
					}
					else
					{
						UE_LOGF(LogSignalProcessing, Error, "Invalid channel index '%d' in AudioDefaultChannelOrder in ini file.", i);
						bOverridden = false;
						break;
					}
				}
			}

			// Now validate that there's no duplicates.
			if (bOverridden)
			{
				bool bIsValid = true;

				for (int32 i = 0; i < AUDIO_MIXER_MAX_OUTPUT_CHANNELS; ++i)
				{
					for (int32 j = 0; j < AUDIO_MIXER_MAX_OUTPUT_CHANNELS; ++j)
					{
						if (j != i && ChannelMapOverride[j] == ChannelMapOverride[i])
						{
							bIsValid = false;
							break;
						}
					}
				}

				if (!bIsValid)
				{
					UE_LOGF(LogSignalProcessing, Error, "Invalid channel index or duplicate entries in AudioDefaultChannelOrder in ini file.");
				}
				else
				{
					for (int32 i = 0; i < AUDIO_MIXER_MAX_OUTPUT_CHANNELS; ++i)
					{
						DefaultChannelOrder[i] = ChannelMapOverride[i];
					}
				}
			}

			return DefaultChannelOrder;
		};

		static const TArray<EAudioMixerChannel::Type> DefaultChannelOrder = CreateDefaultChannelOrder();

		if (Index >= 0 && Index < AUDIO_MIXER_MAX_OUTPUT_CHANNELS)
		{
			OutType = DefaultChannelOrder[Index];
			return true;
		}

		return false;
	}
	
	float GetChannelWeight(EAudioMixerChannel::Type InType) 
	{
		static_assert(static_cast<int32>(EAudioMixerChannel::ChannelTypeCount) == 19, "Possibly missing channel type");

		switch (InType)
		{
			case EAudioMixerChannel::LowFrequency:
				return 0.f; // LFE channels ignored
				
			case EAudioMixerChannel::BackLeft:
			case EAudioMixerChannel::BackRight:
			case EAudioMixerChannel::SideLeft:
			case EAudioMixerChannel::SideRight:
			// If elevation angle is less than 30 degrees and azimuth is between 60 and 120, set channel weight to sqrt(2)
				return FMath::Sqrt(2.f);

			case EAudioMixerChannel::FrontLeft:
			case EAudioMixerChannel::FrontRight:
			case EAudioMixerChannel::FrontCenter:
			case EAudioMixerChannel::FrontLeftOfCenter:
			case EAudioMixerChannel::FrontRightOfCenter:
			case EAudioMixerChannel::BackCenter:
			case EAudioMixerChannel::TopCenter:
			case EAudioMixerChannel::TopFrontLeft:
			case EAudioMixerChannel::TopFrontCenter:
			case EAudioMixerChannel::TopFrontRight:
			case EAudioMixerChannel::TopBackLeft:
			case EAudioMixerChannel::TopBackCenter:
			case EAudioMixerChannel::TopBackRight:
			default:
				return 1.f;
		}
	}
} // namespace Audio::ChannelUtils
