// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AudioDebuggingUtilities.h"

#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"
#include "HAL/Platform.h"

void BreakWhenAudible(float* InBuffer, int32 NumSamples)
{
	static const float AudibilityThreshold = Audio::ConvertToLinear(-40.0f);

	TArrayView<const float> InBufferView(InBuffer, NumSamples);
	float BufferAmplitude = Audio::ArrayGetAverageAbsValue(InBufferView);

	if (BufferAmplitude > AudibilityThreshold)
	{
		PLATFORM_BREAK();
	}
}

void BreakWhenTooLoud(float* InBuffer, int32 NumSamples)
{
	static const float PainThreshold = Audio::ConvertToLinear(3.0f);

	TArrayView<const float> InBufferView(InBuffer, NumSamples);
	float BufferAmplitude = Audio::ArrayGetAverageAbsValue(InBufferView);

	if (BufferAmplitude > PainThreshold)
	{
		PLATFORM_BREAK();
	}
}
