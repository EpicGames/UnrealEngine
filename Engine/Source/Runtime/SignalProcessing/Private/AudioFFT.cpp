// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AudioFFT.h"

#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "HAL/IConsoleManager.h"

#define IFFT_PRESERVE_COMPLEX_COMPONENT 0

static int32 FFTMethodCVar = 0;
TAutoConsoleVariable<int32> CVarFFTMethod(
	TEXT("au.dsp.FFTMethod"),
	FFTMethodCVar,
	TEXT("Determines whether we use an iterative FFT method or the DFT.\n")
	TEXT("0: Use Iterative FFT, 1:: Use DFT"),
	ECVF_Default);

namespace Audio
{
	void GenerateRectangularWindow(float* WindowBuffer, int32 NumFrames, int32 NumChannels)
	{
		int32 NumSamples = NumFrames * NumChannels;
		check(NumSamples >= 0);
		for (int32 i = 0; i < NumSamples; i++)
		{
			WindowBuffer[i] = 1.f;
		}
	}

	void GenerateHammingWindow(float* WindowBuffer, int32 NumFrames, int32 NumChannels, bool bIsPeriodic)
	{
		const int32 N = bIsPeriodic ? NumFrames : NumFrames - 1;
		const float PhaseDelta = 2.0f * PI / N;
		float Phase = 0.0f;

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			const float Value = 0.54 -  (0.46f * FMath::Cos(Phase));
			Phase += PhaseDelta;

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
			{
				WindowBuffer[FrameIndex * NumChannels + ChannelIndex] = Value;
			}
		}
	}

	void GenerateHannWindow(float* WindowBuffer, int32 NumFrames, int32 NumChannels, bool bIsPeriodic)
	{
		const int32 N = bIsPeriodic ? NumFrames : NumFrames - 1;
		const float PhaseDelta = 2.0f * PI / N;
		float Phase = 0.0f;

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			const float Value = 0.5f * (1 - FMath::Cos(Phase));
			Phase += PhaseDelta;

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
			{
				WindowBuffer[FrameIndex * NumChannels + ChannelIndex] = Value;
			}
		}
	}

	void GenerateBlackmanWindow(float* WindowBuffer, int32 NumFrames, int32 NumChannels, bool bIsPeriodic)
	{
		const int32 N = bIsPeriodic ? NumFrames : NumFrames - 1;
		const int32 Midpoint = (N % 2) ? (N + 1) / 2 : N / 2;

		const float PhaseDelta = 2.0f * PI / (N - 1);
		float Phase = 0.0f;

		// Generate the first half of the window:
		for (int32 FrameIndex = 0; FrameIndex <= Midpoint && FrameIndex < NumFrames; FrameIndex++)
		{
			const float Value = 0.42f - 0.5 * FMath::Cos(Phase) + 0.08 * FMath::Cos(2 * Phase);
			Phase += PhaseDelta;

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
			{
				WindowBuffer[FrameIndex * NumChannels + ChannelIndex] = Value;
			}
		}

		// Flip first half for the second half of the window:
		for (int32 FrameIndex = Midpoint + 1; FrameIndex < NumFrames; FrameIndex++)
		{
			const float Value = WindowBuffer[Midpoint - (FrameIndex - Midpoint)];
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
			{
				WindowBuffer[FrameIndex * NumChannels + ChannelIndex] = Value;
			}
		}
	}

	void GenerateWindow(float* InOutWindowBuffer, EWindowType InType, int32 InNumFrames, int32 InNumChannels, bool bIsPeriodic)
	{
		switch (InType)
		{
		case EWindowType::None:
			GenerateRectangularWindow(InOutWindowBuffer, InNumFrames, InNumChannels);	
			break;

		case EWindowType::Hamming:
			GenerateHammingWindow(InOutWindowBuffer, InNumFrames, InNumChannels, bIsPeriodic);	
			break;

		case EWindowType::Hann:
			GenerateHannWindow(InOutWindowBuffer, InNumFrames, InNumChannels, bIsPeriodic);	
			break;

		case EWindowType::Blackman:
			GenerateBlackmanWindow(InOutWindowBuffer, InNumFrames, InNumChannels, bIsPeriodic);	
			break;

		default:
			{
				checkNoEntry();
				GenerateRectangularWindow(InOutWindowBuffer, InNumFrames, InNumChannels);	
			}
		}
	}

	uint32 GetCOLAHopSizeForWindow(EWindowType InType, uint32 WindowLength)
	{
		switch (InType)
		{
		case EWindowType::Hann:
		case EWindowType::Hamming:
			return FMath::FloorToInt((0.5f) * WindowLength);
		case EWindowType::Blackman:
			// Optimal overlap for any Blackman window is derived in this paper:
			// http://edoc.mpg.de/395068
			return FMath::FloorToInt((0.339f) * WindowLength);
		case EWindowType::None:
		default:
			return WindowLength;
		}
	}

	FWindow::FWindow(EWindowType InType, int32 InNumFrames, int32 InNumChannels, bool bIsPeriodic)
		: WindowType(InType)
		, NumSamples(InNumFrames * InNumChannels)
	{
		checkf(NumSamples % 4 == 0, TEXT("For performance reasons, this window's length should be a multiple of 4."));
		Generate(InNumFrames, InNumChannels, bIsPeriodic);
	}

	// Apply this window to InBuffer, which is expected to be an interleaved buffer with the same amount of frames
	// and channels this window was constructed with.
	void FWindow::ApplyToBuffer(float* InBuffer)
	{
		if (WindowType == EWindowType::None)
		{
			return;
		}

		TArrayView<const float> WindowBufferView(WindowBuffer.GetData(), NumSamples);
		TArrayView<float> InBufferView(InBuffer, NumSamples);
		ArrayMultiplyInPlace(WindowBufferView, InBufferView);
	}

	EWindowType FWindow::GetWindowType() const
	{
		return WindowType;
	}


	// Generate the window. Called on constructor.
	void FWindow::Generate(int32 NumFrames, int32 NumChannels, bool bIsPeriodic)
	{
		if (WindowType == EWindowType::None)
		{
			return;
		}

		WindowBuffer.Reset();
		WindowBuffer.AddZeroed(NumSamples);

		switch (WindowType)
		{
			case EWindowType::Hann:
			{
				GenerateHannWindow(WindowBuffer.GetData(), NumFrames, NumChannels, bIsPeriodic);
				break;
			}
			case EWindowType::Hamming:
			{
				GenerateHammingWindow(WindowBuffer.GetData(), NumFrames, NumChannels, bIsPeriodic);
				break;
			}
			case EWindowType::Blackman:
			{
				GenerateBlackmanWindow(WindowBuffer.GetData(), NumFrames, NumChannels, bIsPeriodic);
				break;
			}
			default:
			{
				checkf(false, TEXT("Unknown window type!"));
				break;
			}
		}
	}


	namespace FFTIntrinsics
	{
		float GetScalingExponent(EFFTScaling InScaling)
		{
			switch (InScaling)
			{
				case EFFTScaling::None:
					return 0.f;

				case EFFTScaling::MultipliedByFFTSize:
					return 1.f;

				case EFFTScaling::MultipliedBySqrtFFTSize:
					return 0.5f;

				case EFFTScaling::DividedByFFTSize:
					return -1.f;

				case EFFTScaling::DividedBySqrtFFTSize:
					return -0.5f;

				default:
				{
					checkNoEntry();
					return 0;
				}
			}
		}
	} // namespace FFTIntrinsic

	int32 CeilLog2(int32 InNum)
	{
		static constexpr int32 MaxValue = 0x40000000;
		static constexpr int32 One = 1;

		int32 Result = 0;
		int32 Value = 1;

		while ((Value < InNum) && (Value < MaxValue))
		{
			Result++;
			Value = One << Result;
		}

		return Result;
	}


	float GetPowerSpectrumScaling(int32 FFTSize, EFFTScaling InCurrentScaling, EFFTScaling InTargetScaling)
	{
		if (!ensureMsgf(FFTSize > 0, TEXT("Invalid FFTSize %d"), FFTSize))
		{
			return 1.f;
		}

		const float ScalingExponentDiff = FFTIntrinsics::GetScalingExponent(InTargetScaling) - FFTIntrinsics::GetScalingExponent(InCurrentScaling);
		return FMath::Pow(static_cast<float>(FFTSize), ScalingExponentDiff * 2.f);
	}

	void ScalePowerSpectrumInPlace(int32 FFTSize, EFFTScaling InCurrentScaling, EFFTScaling InTargetScaling, TArrayView<float> InPowerSpectrum)
	{
		if (InCurrentScaling != InTargetScaling)
		{
			const float Scaling = GetPowerSpectrumScaling(FFTSize, InCurrentScaling, InTargetScaling);
			ArrayMultiplyByConstantInPlace(InPowerSpectrum, Scaling);
		}
	}


}
