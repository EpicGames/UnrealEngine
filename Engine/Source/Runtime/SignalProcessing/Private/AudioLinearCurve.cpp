// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/AudioLinearCurve.h"
#include "Algo/BinarySearch.h"

namespace Audio
{
	FLinearCurve::FLinearCurve(const TArray<FVector2D>& InPoints)
	{
		Points = InPoints;
		Sort();
	}

	FLinearCurve::FLinearCurve(TArray<FVector2D>&& InPoints)
	{
		Points = MoveTemp(InPoints);
		Sort();
	}

	void FLinearCurve::SetCurvePoints(const TArray<FVector2D>& InPoints)
	{
		Points = InPoints;
		Sort();
	}

	void FLinearCurve::SetCurvePoints(TArray<FVector2D>&& InPoints)
	{
		Points = MoveTemp(InPoints);
		Sort();
	}

	void FLinearCurve::ClearPoints()
	{
		Points.Reset();
	}

	void FLinearCurve::AddPoint(const FVector2D& InPoint)
	{
		Points.Add(InPoint);
		Sort();
	}

	void FLinearCurve::AddPoints(const TArray<FVector2D>& InPoints)
	{
		Points.Append(InPoints);
		Sort();
	}

	const TArray<FVector2D>& FLinearCurve::GetCurvePoints() const
	{
		return Points;
	}

	void FLinearCurve::Sort()
	{
		Points.Sort([](const FVector2D& A, const FVector2D& B) { return A.X < B.X; });
	}

	bool FLinearCurve::Eval(float InDomain, float& OutValue) const
	{
		// Can't be evaluated
		if (!Points.Num())
		{
			return false;
		}
		
		// Note the curve is already sorted according to increasing domain values

		// Check edge cases first. We clamp the output value to the edges of the curve.
		if (InDomain <= Points[0].X)
		{
			// To the left of the curve domain
			OutValue = Points[0].Y;
			return true;
		}
		else if (InDomain >= Points[Points.Num() - 1].X)
		{
			// To the right of the curve domain
			OutValue = Points[Points.Num() - 1].Y;
			return true;
		}

		// Find the two points to select to do an interpolation
		FVector2D PrevPoint;
		FVector2D CurrPoint;

		PrevPoint.X = InDomain;
		int32 Index = Algo::UpperBound(Points, PrevPoint, [](const FVector2D& A, const FVector2D& B) { return A.X < B.X; });

		if (Index <= Points.Num() && Index >= 1)
		{
			PrevPoint = Points[Index - 1];
			CurrPoint = Points[Index];

			// Linearly interpolate between the two points using the given domain value
			float Alpha = (InDomain - PrevPoint.X) / FMath::Max(CurrPoint.X - PrevPoint.X, SMALL_NUMBER);
			check(Alpha >= 0.0f && Alpha <= 1.0f);

			OutValue = FMath::Lerp(PrevPoint.Y, CurrPoint.Y, Alpha);
			return true;
		}
		return false;

	}

}