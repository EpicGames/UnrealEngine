// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/BitCrusher.h"

namespace Audio
{
	static constexpr float MaxBitDepth = 32.0f;

	FBitCrusher::FBitCrusher()
		: SampleRate(0)
		, BitDepth(16.0f)
		, BitDelta(0.0f)
		, Phase(1.0f)
		, PhaseDelta(1.0f)
	{
		BitDelta = 1.0f / FMath::Pow(2.0f, BitDepth);
		LastOutput[0] = 0.0f;
		LastOutput[1] = 0.0f;
	}

	FBitCrusher::~FBitCrusher()
	{
	}

	void FBitCrusher::Init(const float InSampleRate, const int32 InNumChannels)
	{
		SampleRate = InSampleRate;
		Phase = 1.0f;
		NumChannels = InNumChannels;
	}

	void FBitCrusher::SetSampleRateCrush(const float InFrequency)
	{
		PhaseDelta = FMath::Clamp(InFrequency, 1.0f, SampleRate) / SampleRate;
	}

	void FBitCrusher::SetBitDepthCrush(const float InBitDepth)
	{
		BitDepth = FMath::Clamp(InBitDepth, 1.0f, MaxBitDepth);
		BitDelta = 1.0f / FMath::Pow(2.0f, BitDepth);
		ReciprocalBitDelta = 1.0f / BitDelta;
	}

	void FBitCrusher::ProcessAudioFrame(const float* InFrame, float* OutFrame)
	{
		Phase += PhaseDelta;
		if (Phase >= 1.0f)
		{
			Phase -= 1.0f;

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				LastOutput[ChannelIndex] = BitDelta * FMath::FloorToFloat(InFrame[ChannelIndex] * ReciprocalBitDelta + 0.5f);
			}
		}

		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			OutFrame[ChannelIndex] = LastOutput[ChannelIndex];
		}
	}

	void FBitCrusher::ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer)
	{
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
		{
			ProcessAudioFrame(&InBuffer[SampleIndex], &OutBuffer[SampleIndex]);
		}
	}

	float FBitCrusher::GetMaxBitDepth()
	{
		return MaxBitDepth;
	}
}
