// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/BlockCorrelator.h"
#include "DSP/FFTAlgorithm.h"
#include "DSP/FloatArrayMath.h"

#include "SignalProcessingModule.h"

namespace Audio
{
	FBlockCorrelator::FBlockCorrelator(const FBlockCorrelatorSettings& InSettings)
	:	Settings(InSettings)
	,	NumValuesInBlock(1 << Settings.Log2NumValuesInBlock)
	,	NumValuesInFFTRealBuffer(0)
	,	NumValuesInFFTComplexBuffer(0)
	,	Window(InSettings.WindowType, 1 << Settings.Log2NumValuesInBlock, 1, false)
	{
		// Determine FFT Settings
		FFFTSettings FFTSettings;

		// Make FFT twice as big as block to avoid temporal aliasing.
		FFTSettings.Log2Size = Settings.Log2NumValuesInBlock + 1;
		FFTSettings.bArrays128BitAligned = true;
		FFTSettings.bEnableHardwareAcceleration = true;

		// Create FFT
		FFTAlgorithm = FFFTFactory::NewFFTAlgorithm(FFTSettings);

		if (!FFTAlgorithm.IsValid())
		{
			UE_LOGF(LogSignalProcessing, Warning, "Failed to create FFT algorithm for FBlockCorrelator");

			// Set these to default value to avoid having to do many if statements during calls to *Correlate(...)
			NumValuesInFFTRealBuffer = NumValuesInBlock * 2;
			NumValuesInFFTComplexBuffer = NumValuesInBlock + 2;
		}
		else
		{
			NumValuesInFFTRealBuffer = FFTAlgorithm->NumInputFloats();
			NumValuesInFFTComplexBuffer = FFTAlgorithm->NumOutputFloats();
		}

		// Allocate buffers
		if (NumValuesInFFTRealBuffer > 0)
		{
			WindowedBufferA.AddZeroed(NumValuesInFFTRealBuffer);
			WindowedBufferB.AddZeroed(NumValuesInFFTRealBuffer);
			FullOutputBuffer.AddZeroed(NumValuesInFFTRealBuffer);
		}

		if (NumValuesInFFTComplexBuffer > 0)
		{
			ComplexBufferA.AddZeroed(NumValuesInFFTComplexBuffer);
			ComplexBufferB.AddZeroed(NumValuesInFFTComplexBuffer);
			ComplexCorrelationBuffer.AddZeroed(NumValuesInFFTComplexBuffer);
		}

		// Initialize normalization buffer to valid values. 
		InitializeNormalizationBuffer();
	}


	const FBlockCorrelatorSettings& FBlockCorrelator::GetSettings() const
	{
		return Settings;
	}

	int32 FBlockCorrelator::GetNumInputValues() const
	{
		return NumValuesInBlock;
	}

	int32 FBlockCorrelator::GetNumOutputValues() const
	{
		return NumValuesInFFTRealBuffer;
	}

	void FBlockCorrelator::CrossCorrelate(const FAlignedFloatBuffer& InputA, const FAlignedFloatBuffer& InputB, FAlignedFloatBuffer& Output)
	{
		check(InputA.Num() == NumValuesInBlock);
		check(InputB.Num() == NumValuesInBlock);
		check(Output.Num() == NumValuesInFFTRealBuffer);

		// Be a little defensive about folks passing in the wrong size of input buffer.
		int32 NumA = FMath::Min(NumValuesInBlock, InputA.Num());
		int32 NumB = FMath::Min(NumValuesInBlock, InputB.Num());

		if ((NumA > 0) && (NumB > 0))
		{
			// WindowBufferA and WindowBufferB are twice as long as the input and padded with zeros.
			// This memcpy effectively zeropads the input buffers.
			FMemory::Memcpy(WindowedBufferA.GetData(), InputA.GetData(), NumA * sizeof(float));
			FMemory::Memcpy(WindowedBufferB.GetData(), InputB.GetData(), NumB * sizeof(float));

			Window.ApplyToBuffer(WindowedBufferA.GetData());
			Window.ApplyToBuffer(WindowedBufferB.GetData());

			CyclicCrossCorrelate(WindowedBufferA, WindowedBufferB, Output);

			if (Settings.bDoNormalize)
			{
				ArrayMultiplyInPlace(NormalizationBuffer, Output);
			}
		}
	}

	void FBlockCorrelator::AutoCorrelate(const FAlignedFloatBuffer& Input, FAlignedFloatBuffer& Output)
	{
		check(Input.Num() == NumValuesInBlock);
		check(Output.Num() == NumValuesInFFTRealBuffer);

		// Be a little defensive about folks passing in the wrong size of input buffer.
		int32 Num = FMath::Min(NumValuesInBlock, Input.Num());

		if (Num > 0)
		{
			// WindowBufferA is twice as long as the input and padded with zeros.
			// This memcpy effectively zeropads the input buffer.
			FMemory::Memcpy(WindowedBufferA.GetData(), Input.GetData(), Num * sizeof(float));

			Window.ApplyToBuffer(WindowedBufferA.GetData());

			CyclicAutoCorrelate(WindowedBufferA, Output);

			if (Settings.bDoNormalize)
			{
				ArrayMultiplyInPlace(NormalizationBuffer, Output);
			}
		}
	}

	void FBlockCorrelator::CyclicCrossCorrelate(const FAlignedFloatBuffer& InputA, const FAlignedFloatBuffer& InputB, FAlignedFloatBuffer& Output)
	{
		check(InputA.Num() == NumValuesInFFTRealBuffer);
		check(InputB.Num() == NumValuesInFFTRealBuffer);
		check(Output.Num() == NumValuesInFFTRealBuffer);

		// Cyclical correlation can be done more quickly in the frequency domain. 
		if (FFTAlgorithm.IsValid())
		{
			// Convert to freq domain.
			FFTAlgorithm->ForwardRealToComplex(InputA.GetData(), ComplexBufferA.GetData());
			FFTAlgorithm->ForwardRealToComplex(InputB.GetData(), ComplexBufferB.GetData());

			// Do complex conjugate multiplication.
			ArrayComplexConjugateInPlace(ComplexBufferB);
			ArrayComplexMultiplyInPlace(ComplexBufferB, ComplexBufferA);

			// Back to the time domain.
			FFTAlgorithm->InverseComplexToReal(ComplexBufferA.GetData(), Output.GetData());
		}
	}

	void FBlockCorrelator::CyclicAutoCorrelate(const FAlignedFloatBuffer& Input, FAlignedFloatBuffer& Output)
	{
		check(Input.Num() == NumValuesInFFTRealBuffer);
		check(Output.Num() == NumValuesInFFTRealBuffer);

		// Cyclical correlation can be done more quickly in the frequency domain. 
		if (FFTAlgorithm.IsValid())
		{
			// Convert to freq domain.
			FFTAlgorithm->ForwardRealToComplex(Input.GetData(), ComplexBufferA.GetData());

			// Do complex conjugate multiplication.
			ArrayComplexConjugate(ComplexBufferA, ComplexBufferB);
			ArrayComplexMultiplyInPlace(ComplexBufferB, ComplexBufferA);

			// Back to the time domain.
			FFTAlgorithm->InverseComplexToReal(ComplexBufferA.GetData(), Output.GetData());
		}
	}

	void FBlockCorrelator::InitializeNormalizationBuffer()
	{
		// Normalization by the autocorrelation of the window. This increases variance
		// of information at end of autocorrelation output, but normalizes the output
		// magnitude to remove bais.
		if (!FFTAlgorithm.IsValid() || (NumValuesInBlock < 1))
		{
			return;
		}

		/** Determine the normalization buffer by taking the inverse of the autocorrelation of the 
		 * input window.*/

		NormalizationBuffer.Reset();

		if ((NumValuesInFFTRealBuffer > 0) && (NumValuesInFFTRealBuffer > NumValuesInBlock))
		{
			// Initialize normalization buffer to be the time domain window applied to incoming data.
			NormalizationBuffer.Init(1.f, NumValuesInBlock);
			NormalizationBuffer.AddZeroed(NumValuesInFFTRealBuffer - NumValuesInBlock);

			Window.ApplyToBuffer(NormalizationBuffer.GetData());
		}

		FAlignedFloatBuffer CorrelationBuffer;
		CorrelationBuffer.AddUninitialized(NumValuesInFFTRealBuffer);

		// Take auto correlation of the window.
		CyclicAutoCorrelate(NormalizationBuffer, CorrelationBuffer);

		float* NormData = NormalizationBuffer.GetData();
		const float* CorrData = CorrelationBuffer.GetData();

		for (int32 i = 0; i < NumValuesInFFTRealBuffer; i++)
		{
			if (CorrData[i] > SMALL_NUMBER)
			{
				// Set the normalization data to be the inverse of the window's autocorrelation.
				NormData[i] = 1.f / CorrData[i];
			}
			else
			{
				// Avoid divide by zero. 
				NormData[i] = 0.f;
			}
		}
	}
}
