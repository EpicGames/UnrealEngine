// Copyright Epic Games, Inc. All Rights Reserved.
#include "DSP/BufferDiagnostics.h"

#include "Algo/Accumulate.h"
#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"
#include "Misc/VarArgs.h"

namespace Audio
{
	// denormalized
	int32 FindDenormalized(TArrayView<const float> InBuffer)
	{
		return Algo::IndexOfByPredicate(InBuffer, [](const float InSample) { return IsDenormalized(InSample); });
	}

	bool ContainsDenormalized(TArrayView<const float> InBuffer)
	{
		return FindDenormalized(InBuffer) != INDEX_NONE;
	}

	// +inf/-inf
	int32 FindInfinity(TArrayView<const float> InBuffer)
	{
		return Algo::IndexOfByPredicate(InBuffer, [](const float InSample) { return IsInfinity(InSample); });
	}

	bool ContainsInfinity(TArrayView<const float> InBuffer)
	{
		return FindInfinity(InBuffer) != INDEX_NONE;
	}

	// NaN
	int32 FindNan(TArrayView<const float> InBuffer)
	{
		return Algo::IndexOfByPredicate(InBuffer, [](const float InSample) { return FMath::IsNaN(InSample); });
	}

	bool ContainsNan(TArrayView<const float> InBuffer)
	{
		return FindNan(InBuffer) != INDEX_NONE;
	}

	// Because our find checks are composite, once failed, ask for the flags that failed.
	static ECheckBufferFlags ClassifyFailure(TArrayView<const float> InBuffer, const int32 InIndexOfFailure)
	{
		if (InIndexOfFailure == INDEX_NONE || InBuffer.Num() == 0)
		{
			return ECheckBufferFlags::None;
		}

		ECheckBufferFlags FailFlags = ECheckBufferFlags::None;
		const float BadValue = InBuffer[InIndexOfFailure];
		if (IsDenormalized(BadValue))
		{
			FailFlags |= ECheckBufferFlags::Denormalized;
		}
		if (FMath::IsNaN(BadValue))
		{
			FailFlags |= ECheckBufferFlags::Nan;
		}
		if (IsInfinity(BadValue))
		{
			FailFlags |= ECheckBufferFlags::Infinity;
		}
		return FailFlags;
	}

	bool CheckBuffer(TArrayView<const float> InBuffer, const ECheckBufferFlags InFlags, ECheckBufferFlags& OutFailedFlags)
	{
		// Reset flags to start.
		OutFailedFlags = ECheckBufferFlags::None;

		// If all flags are on do combined check for speed.
		if (InFlags == ECheckBufferFlags::All)
		{
			const int32 IndexOf = Algo::IndexOfByPredicate(InBuffer, [](const float InSample) -> bool
			{
				return IsDenormalized(InSample) || !FMath::IsFinite(InSample);
			});
			OutFailedFlags = ClassifyFailure(InBuffer, IndexOf);
			return IndexOf == INDEX_NONE;
		}

		// Another common one is Inf/Nan together. !InFinite is equiv.
		if (InFlags == (ECheckBufferFlags::Infinity | ECheckBufferFlags::Nan))
		{
			const int32 IndexOf = Algo::IndexOfByPredicate(InBuffer, [](const float InSample) -> bool
			{
				return !FMath::IsFinite(InSample);
			});
			OutFailedFlags = ClassifyFailure(InBuffer, IndexOf);
			return IndexOf == INDEX_NONE;
		}

		// Otherwise do piece-meal testing.
		if ((InFlags & ECheckBufferFlags::Nan) != ECheckBufferFlags::None)
		{
			if (ContainsNan(InBuffer))
			{
				OutFailedFlags |= ECheckBufferFlags::Nan;
				return false;
			}
		}
		if ((InFlags & ECheckBufferFlags::Denormalized) != ECheckBufferFlags::None)
		{
			if (ContainsDenormalized(InBuffer))
			{
				OutFailedFlags |= ECheckBufferFlags::Denormalized;
				return false;
			}
		}
		if ((InFlags & ECheckBufferFlags::Infinity) != ECheckBufferFlags::None)
		{
			if (ContainsInfinity(InBuffer))
			{
				OutFailedFlags |= ECheckBufferFlags::Infinity;
				return false;
			}
		}

		// Pass.
		return true;
	}

	// Static linkage.
	FCriticalSection FCheckedBufferState::ListCs;
	FCheckedBufferState* FCheckedBufferState::Head = nullptr;
	
	FCheckedBufferState::FCheckedBufferState(const int32 InLine, const TCHAR* InFile, const TCHAR* InName, const ECheckBufferFlags InCheckFlags, const EBufferCheckBehavior InCheckBehavior)
		: Name(InName)
		, Line(InLine)
		, File(InFile)
		, CheckFlags(InCheckFlags)
		, Behavior(InCheckBehavior)
	{
		FScopeLock Lock(&ListCs);
		Next = Head;
		Head = this;	
	}

	void FCheckedBufferState::ForEach(TFunctionRef<void(FCheckedBufferState&)> InCmd)
	{
		FScopeLock Lock(&ListCs);
		for (FCheckedBufferState* i=Head; i; i=i->Next)
		{
			InCmd(*i);
		}
	}


	bool FCheckedBufferState::DoCheck(TArrayView<const float> InBuffer)
	{
		// Only do the check if we have some flags to test against
		// and only do the check if we've not had a prior failure.
		if (CheckFlags != ECheckBufferFlags::None && FailFlags == ECheckBufferFlags::None)
		{
			if (!CheckBuffer(InBuffer, CheckFlags, FailFlags))
			{
				return false;
			}
		}

		// Pass.
		return true;
	}
	
	void FCheckedBufferState::FailedBufferCheckImpl(const TCHAR* InFormat, ...) const
	{
		TCHAR Msg[512];
		GET_TYPED_VARARGS(TCHAR, Msg, UE_ARRAY_COUNT(Msg), UE_ARRAY_COUNT(Msg)-1, InFormat, InFormat);
		
		switch (Behavior)
		{
			case EBufferCheckBehavior::Ensure:
			{
				ensureMsgf(false, TEXT("Failed Buffer Check: '%s', Failure='%s', %s:%d, CheckFlags='%s', %s"),Name,
					*ToDelimitedString(FailFlags), File, Line, *ToDelimitedString(CheckFlags), Msg);
				break;
			}
			case EBufferCheckBehavior::Log:
			{
				UE_LOGF(LogTemp, Warning, "Failed Buffer Check: '%ls', Failure='%ls', %ls:%d, CheckFlags='%ls', %ls", Name,
					*ToDelimitedString(FailFlags), File, Line, *ToDelimitedString(CheckFlags), Msg);
				break;
			}
			case EBufferCheckBehavior::Break:	
			{
				UE_LOGF(LogTemp, Warning, "Failed Buffer Check: '%ls', Failure='%ls', %ls:%d, CheckFlags='%ls', %ls", Name,
					*ToDelimitedString(FailFlags), File, Line, *ToDelimitedString(CheckFlags), Msg);
				UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
				break;
			}
			case EBufferCheckBehavior::Nothing:
			default:
			{
				break;
			}
		}
	}
}

namespace BufferDiagnosticsPrivate
{
	static void HandleDiagnosticsCmd(const TArray<FString>& InArgs, FOutputDevice& InDevice);

	// Handle the enabling diagnostics.
	FAutoConsoleCommandWithArgsAndOutputDevice CmdBufferDiagnostics(
		TEXT("au.debug.bufferdiagnostics"),
		TEXT("Enables per buffer sample diagnostics (Nans/denorms/Infs)"),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateStatic(HandleDiagnosticsCmd));
	
	using namespace Audio;
	
	// Simple compile time function to remove the fully qualified name.
	// Example "ECheckBufferFlags::Infinity" -> "Infinity".
	static constexpr const TCHAR* ToShortEnumName(const TCHAR* InFullyQualifiedName) 
	{
		const TCHAR* Iter = InFullyQualifiedName;
		for(; *Iter; ++Iter)
		{
			if (*Iter == L':') 
			{
				return Iter + 2;
			}
		}
		return InFullyQualifiedName;
	}
#define FOREACH_ECHECKBUFFERFLAGS(OP)\
	OP(ECheckBufferFlags::None)\
	OP(ECheckBufferFlags::Infinity)\
	OP(ECheckBufferFlags::Nan)\
	OP(ECheckBufferFlags::Denormalized)\
	OP(ECheckBufferFlags::All)

#define FOREACH_ECHECKBUFFERBEHAVIOR(OP)\
	OP(EBufferCheckBehavior::Nothing)\
	OP(EBufferCheckBehavior::Ensure)\
	OP(EBufferCheckBehavior::Log)\
	OP(EBufferCheckBehavior::Break)
	
// Switch on the enum and return a short string version.
#ifndef TEST_CASE_ENUM_RETURN_SHORTNAME 
	#define TEST_CASE_ENUM_RETURN_SHORTNAME(X) case X: { return ToShortEnumName(TEXT(#X)); }
#endif //TEST_CASE_ENUM_RETURN_SHORTNAME
	
	static const TCHAR* LexToString(const ECheckBufferFlags InEnum)
	{
		switch (InEnum)
		{
			FOREACH_ECHECKBUFFERFLAGS(TEST_CASE_ENUM_RETURN_SHORTNAME)
			default: break;
		}
		return TEXT("Unknown ECheckBufferFlags");
	}
	static const TCHAR* LexToString(const EBufferCheckBehavior InEnum)
	{
		switch (InEnum)
		{
			FOREACH_ECHECKBUFFERBEHAVIOR(TEST_CASE_ENUM_RETURN_SHORTNAME)
			default: break;
		}
		return TEXT("Unknown EBufferCheckBehavior");
	}
	template<typename EnumType> static bool IsEnumEntry(const TCHAR* InString, const EnumType InEnum)
	{
		return FCString::Stricmp(InString, LexToString(InEnum)) == 0;
	}

// Test string against an enum entry, return true on a match and set output property to the match.
#ifndef TEST_STRING_AGAINST_ENUM_AND_RETURN
	#define TEST_STRING_AGAINST_ENUM_AND_RETURN(X) if (IsEnumEntry(InStr, X)) { OutEnum=X; return true; }
#endif //TEST_STRING_AGAINST_ENUM_AND_RETURN
	
	static bool TryLexFromString(const TCHAR* InStr, ECheckBufferFlags& OutEnum)
	{
		FOREACH_ECHECKBUFFERFLAGS(TEST_STRING_AGAINST_ENUM_AND_RETURN);
		return false; // Fail.
	}
	static bool TryLexFromString(const TCHAR* InStr, EBufferCheckBehavior& OutEnum)
	{
		FOREACH_ECHECKBUFFERBEHAVIOR(TEST_STRING_AGAINST_ENUM_AND_RETURN)
		return false; // Fail.
	}
	static void BuildBitfieldString(const ECheckBufferFlags InBitfield, const ECheckBufferFlags InFlag, FString& OutString)
	{
		if ((InBitfield & InFlag) == InFlag && static_cast<uint32>(InFlag) != 0)
		{
			OutString += LexToString(InFlag);
			OutString += TEXT("|");
		}	
	}

// Test enum against each flag of bitfield and build a string
#ifndef TEST_AND_BUILD_BITFIELD_STRING 
	#define TEST_AND_BUILD_BITFIELD_STRING(X) BuildBitfieldString(InEnum, X, String);
#endif //TEST_AND_BUILD_BITFIELD_STRING
	
	FString ToDelimString(const ECheckBufferFlags InEnum)
	{		
		if (const TCHAR* ExactMatch = LexToString(InEnum))
		{
			return ExactMatch;
		}
		FString String;
		FOREACH_ECHECKBUFFERFLAGS(TEST_AND_BUILD_BITFIELD_STRING)
		return String;
	}
	
	// Clean up macros.
	#undef TEST_AND_BUILD_BITFIELD_STRING
	#undef TEST_STRING_AGAINST_ENUM_AND_RETURN
	#undef TEST_CASE_ENUM_RETURN_SHORTNAME
	#undef FOREACH_ECHECKBUFFERFLAGS
	#undef FOREACH_ECHECKBUFFERBEHAVIOR
		
	static bool TryParseDelimitedBitfield(const FString& InBitfieldString, Audio::ECheckBufferFlags& OutEnum)
	{
		using namespace Audio;
		TArray<FString> Fields;
		static const TCHAR* Delims[] = { TEXT("|"),TEXT(","),TEXT(" ") };
		InBitfieldString.ParseIntoArray(Fields, Delims, UE_ARRAY_COUNT(Delims), true);
		OutEnum = Algo::TransformAccumulate(
			Fields,
			[](const FString& i) -> ECheckBufferFlags
			{
				Audio::ECheckBufferFlags Flags = ECheckBufferFlags::None;
				TryLexFromString(*i, Flags);
				return Flags;
			},
			ECheckBufferFlags::None,
			[](const ECheckBufferFlags& A, const ECheckBufferFlags& B) -> ECheckBufferFlags
			{
				return A | B;
			});
		return true;
	}

	TOptional<FString> FindArg(const TCHAR* InName, const TArray<FString>& InArgs)
	{
		if (const FString* Found = Algo::FindByPredicate(InArgs, [=](const FString& InStr){ return InStr.StartsWith(InName); }) )
		{
			FString Result;
			Found->Split(InName, nullptr, &Result);
			return Result;
		}
		return {};
	}

	void HandleDiagnosticsCmd(const TArray<FString>& InArgs, FOutputDevice& InDevice)
	{
		using namespace Audio;

		if (FindArg(TEXT("-Help"), InArgs).IsSet())
		{
			InDevice.Logf(TEXT("-Filter=<match> -Behavior=Break|Log|Ensure|Nothing -Check=Nan|Infinity|Denormalized|All -Reset -Verbose -Help"));
			return;
		}
		
		FString Filter; // Default (no filter).
		if (TOptional<FString> Arg = FindArg(TEXT("-Filter="),InArgs); Arg.IsSet())
		{
			Filter = *Arg;
		}
		EBufferCheckBehavior Behavior = EBufferCheckBehavior::Ensure;
		if (TOptional<FString> Arg = FindArg(TEXT("-Behavior="),InArgs); Arg.IsSet())
		{
			TryLexFromString(**Arg, Behavior);
		}
		ECheckBufferFlags CheckFlags = ECheckBufferFlags::All;
		if (TOptional<FString> Arg = FindArg(TEXT("-Checks="),InArgs); Arg.IsSet())
		{
			TryParseDelimitedBitfield(*Arg, CheckFlags);
		}
		bool bVerbose = false;
		if (FindArg(TEXT("-Verbose"),InArgs).IsSet())
		{
			bVerbose = true;
		}
		bool bReset = false;
		if (FindArg(TEXT("-Reset"),InArgs).IsSet())
		{
			bReset = true;
		}
		
		InDevice.Logf(TEXT("Submix Buffer Diagnostics. UsingFlags='%s', Behavior='%s'"),
			ToCStr(ToDelimitedString(CheckFlags)), ToCStr(LexToString(Behavior)) );

		int32 Total = 0;
		int32 FilterCount = 0;
		FCheckedBufferState::ForEach([&](Audio::FCheckedBufferState& i)
		{
			Total++;
			if (FString(i.Name).Contains(Filter) || Filter.IsEmpty())
			{
				i.Behavior = Behavior;
				i.CheckFlags = CheckFlags;
				
				if (bReset)
				{
					i.FailFlags = ECheckBufferFlags::None;
				}
				
				FilterCount++;
				if (bVerbose)
				{
					InDevice.Logf(TEXT("Check='%s', File='%s:%d', Flags='%s', Behavior='%s'"),
						i.Name,i.File,i.Line, *ToDelimitedString(i.CheckFlags), ToCStr(LexToString(Behavior)) );
				}
			}
		});
		InDevice.Logf(TEXT("TotalChecks=%d, Filtered=%d"), Total, FilterCount);
	}
}//namespace BufferDiagnosticsPrivate

namespace Audio
{
	FString ToDelimitedString(const ECheckBufferFlags InEnum)
	{
		return BufferDiagnosticsPrivate::ToDelimString(InEnum);
	}
}
	