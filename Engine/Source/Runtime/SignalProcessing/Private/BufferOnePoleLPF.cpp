// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/BufferOnePoleLPF.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	// Constructor 
	FBufferOnePoleLPF::FBufferOnePoleLPF(float InG)
		: CutoffFrequency(0.0f)
		, B1(0.0f)
		, A0(1.0f)
		, Z1(0.0f)
	{
		SetG(InG);
	}

	// Set the LPF gain coefficient
	void FBufferOnePoleLPF::SetG(float InG)
	{ 
		B1 = InG;
		A0 = 1.0f - B1;
	}

	// Resets the sample delay to 0
	void FBufferOnePoleLPF::Reset()
	{
		SetG(0.0f);
		FlushAudio();
	}

	void FBufferOnePoleLPF::FlushAudio()
	{
		Z1 = 0.0f;
	}

	/** Sets the filter frequency using normalized frequency (between 0.0 and 1.0f or 0.0 hz and Nyquist Frequency in Hz) */
	void FBufferOnePoleLPF::SetFrequency(const float InFrequency)
	{
		if (!FMath::IsNearlyEqual(InFrequency, CutoffFrequency))
		{
			CutoffFrequency = InFrequency;
			SetG(FMath::Exp(-PI * CutoffFrequency));
		}
	}

	void FBufferOnePoleLPF::ProcessAudio(const Audio::FAlignedFloatBuffer& InSamples, Audio::FAlignedFloatBuffer& OutSamples)
	{
		int32 Index;
		const int32 InNum = InSamples.Num();
		const float* InSampleData = InSamples.GetData();
		
		// Prepare output samples
		OutSamples.Reset(InNum);
		OutSamples.AddUninitialized(InNum);
		float* OutSampleData = OutSamples.GetData();

		// Filter audio
		int32 DelayIndex;
		OutSampleData[0] = InSampleData[0] * A0 + B1 * Z1;
		for(Index = 1, DelayIndex = 0; Index < InNum; Index++, DelayIndex++)
		{
			OutSampleData[Index] = InSampleData[Index] * A0 + B1 * OutSampleData[DelayIndex];
		}

		// Store delay value
		Z1 = OutSampleData[InNum - 1];
	}
}
