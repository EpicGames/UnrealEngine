// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	/**
	 * CHANNEL MIXING OPERATIONS:
	 * To understand these functions, it's best that you have prior experience reading SIMD code.
	 * These functions are all variations on component-wise matrix multiplies. 
	 * There are two types of functions below:
	 * 
	 * Apply[N]ChannelGain:
	 * These are all in-place multiplies of an N-length gain vector and an N-length frame.
	 * There are two flavors of every variant of this function: The non-interpolating form (which takes a single gain matrix)
	 * And the interpolating form (which takes a start gain matrix and interpolates to the end gain matrix over the given number of frames).
	 * All non-interpolating forms of these functions use the following steps:
	 *    1. Create a const GainVector, or series of GainVectors, that maps to the multiplies required for each iteration.
	 *    2. In a loop:
	 *           i.   load a frame or number of frames into a vector register or series of vector registers (these are named Result).
	 *           ii.  perform a vector multiply on result with the corresponding gain vector.
	 *           iii. store the result vector in the same position in the buffer we loaded from.
	 *
	 * The interpolating forms of these functions use the following steps:
	 *    1. Initialize a non-const GainVector, or series of GainVectors, from StartGains, that maps to the multiplies required for each iteration.
	 *    2. Compute the amount we add to GainVector for each iteration to reach Destination Gains and store it in the const GainDeltasVector.
	 *    3. In a loop:
	 *           i.   load a frame or number of frames into a vector register or series of vector registers (these are named Result).
	 *           ii.  perform a vector multiply on result with the corresponding gain vector.
	 *           iii. store the result vector in the same position in the buffer we loaded from.
	 *           iv.  increment each GainVector by it's corresponding GainDeltasVector.
	 *
	 *
	 * MixMonoTo[N]ChannelsFast and Mix2ChannelsTo[N]ChannelsFast:
	 * These, like Apply[N]ChannelGain, all have non-interpolating and interpolating forms.
	 * All non-interpolating forms of these functions use the following steps:
	 *    1. Create a const GainVector, or series of GainVectors, that maps to the multiplies required for each input channel for each iteration.
	 *    2. In a loop:
	 *           i.   load a frame or number of frames into a const vector register or series of const vector registers (these are named Input).
	 *           ii.  perform a vector multiply on input with the corresponding gain vector and store the result in a new vector or series of vectors named Result.
	 *           iii. if there is a second input channel, store the results of the following MultiplyAdd operation to Results: (Gain Vectors for second channel) * (Input vectors for second channel) + (Result vectors from step ii).
	 *
	 * Interpolating forms of these functions use the following steps:
	 *    1. Initialize a non-const GainVector, or series of GainVectors, from StartGains, that maps to the multiplies required for each input channel for each iteration.
	 *    2. Compute the amount we add to each GainVector for each iteration to reach the vector's corresponding DestinationGains and store it in a corresponding GainDeltaVector.
	 *    3. In a loop:
	 *           i.   load a frame or number of frames into a const vector register or series of const vector registers (these are named Input).
	 *           ii.  perform a vector multiply on input with the corresponding gain vector and store the result in a new vector or series of vectors named Result.
	 *           iii. if there is a second input channel, store the results of the following MultiplyAdd operation to Results: (Gain Vectors for second channel) * (Input vectors for second channel) + (Result vectors from step ii).
	 *           iv.  increment each GainVector by it's corresponding GainDeltasVector.
	 * 
	 * DETERMINING THE VECTOR LAYOUT FOR EACH FUNCTION:
	 * For every variant of Mix[N]ChannelsTo[N]ChannelsFast, we use the least common multiple of the number of output channels and the SIMD vector length (4) to calulate the length of our matrix.
	 * For example, MixMonoTo4ChannelsFast can use a single VectorRegister4Float for each variable. GainVector's values are [g0, g1, g2, g3], input channels are mapped to [i0, i0, i0, i0], and output channels are mapped to [o0, o1, o2, o3].
	 * MixMonoTo8ChannelsFast has an LCM of 8, so we use two VectorRegister4Float for each variable. This results in the following layout:
	 * GainVector1:   [g0, g1, g2, g3] GainVector2:   [g4, g5, g6, g7]
	 * InputVector1:  [i0, i0, i0, i0] InputVector2:  [i0, i0, i0, i0]
	 * ResultVector1: [o0, o1, o2, o3] ResultVector2: [o4, o5, o6, o7]
	 *
	 * The general naming convention for vector variables is [Name]Vector[VectorIndex] for MixMonoTo[N]ChannelsFast functions.
	 * For Mix2ChannelsTo[N]ChannelsFast functions, the naming convention for vector variables is [Name]Vector[VectorIndex][InputChannelIndex].
	 *
	 * For clarity, the layout of vectors for each function variant is given in a block comment above that function.
	 */

	void Apply2ChannelGain(FAlignedFloatBuffer& StereoBuffer, const float* RESTRICT Gains)
	{
		Apply2ChannelGain(StereoBuffer.GetData(), StereoBuffer.Num(), Gains);
	}

	void Apply2ChannelGain(float* RESTRICT StereoBuffer, int32 NumSamples, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector = VectorLoadFloat2(Gains);
		
		for (int32 i = 0; i < NumSamples; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			VectorRegister4Float Result = VectorLoad(&StereoBuffer[i]);
			Result = VectorMultiply(Result, GainVector);
			VectorStore(Result, &StereoBuffer[i]);
		}
	}

	void Apply2ChannelGain(FAlignedFloatBuffer& StereoBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		Apply2ChannelGain(StereoBuffer.GetData(), StereoBuffer.Num(), StartGains, EndGains);
	}

	void Apply2ChannelGain(float* RESTRICT StereoBuffer, int32 NumSamples, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		// Initialize GainVector at StartGains and compute GainDeltasVector:
		VectorRegister4Float GainVector = VectorLoadFloat2(StartGains);
		const VectorRegister4Float DestinationVector = VectorLoadFloat2(EndGains);
		const VectorRegister4Float NumFramesVector = VectorSetFloat1(NumSamples / 4.0f);
		const VectorRegister4Float GainDeltasVector = VectorDivide(VectorSubtract(DestinationVector, GainVector), NumFramesVector);

		for (int32 i = 0; i < NumSamples; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			VectorRegister4Float Result = VectorLoad(&StereoBuffer[i]);
			Result = VectorMultiply(Result, GainVector);
			VectorStore(Result, &StereoBuffer[i]);

			GainVector = VectorAdd(GainVector, GainDeltasVector);
		}
	}

	void MixMonoTo2ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		MixMonoTo2ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 2, Gains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Gain       | g0      | g1      | g0      | g1      |
	* |            | *       | *       | *       | *       |
	* | Input      | i0      | i0      | i0      | i0      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/
	void MixMonoTo2ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector = VectorLoadFloat2(Gains);
		for (int32 i = 0; i < NumFrames; i += 2)
		{
			VectorRegister4Float Result = VectorSet(MonoBuffer[i], MonoBuffer[i], MonoBuffer[i + 1], MonoBuffer[i + 1]);
			Result = VectorMultiply(Result, GainVector);
			VectorStore(Result, &DestinationBuffer[i*2]);
		}
	}

	void MixMonoTo2ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		MixMonoTo2ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 2, StartGains, EndGains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Gain       | g0      | g1      | g0      | g1      |
	* |            | *       | *       | *       | *       |
	* | Input      | i0      | i0      | i1      | i1      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/
	void MixMonoTo2ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		// Initialize GainVector at StartGains and compute GainDeltasVector:
		VectorRegister4Float GainVector = VectorLoadFloat2(StartGains);
		const VectorRegister4Float DestinationVector = VectorLoadFloat2(EndGains);
		const VectorRegister4Float NumFramesVector = VectorSetFloat1(NumFrames / 2.0f);
		const VectorRegister4Float GainDeltasVector = VectorDivide(VectorSubtract(DestinationVector, GainVector), NumFramesVector);

		// To help with stair stepping, we initialize the second frame in GainVector to be half a GainDeltas vector higher than the first frame.
		const VectorRegister4Float VectorOfHalf = VectorSet(0.5f, 0.5f, 1.0f, 1.0f);
		GainVector = VectorMultiplyAdd(GainDeltasVector, VectorOfHalf, GainVector);

		for (int32 i = 0; i < NumFrames; i += 2)
		{
			VectorRegister4Float Result = VectorSet(MonoBuffer[i], MonoBuffer[i], MonoBuffer[i + 1], MonoBuffer[i + 1]);
			Result = VectorMultiply(Result, GainVector);
			VectorStore(Result, &DestinationBuffer[i*2]);

			GainVector = VectorAdd(GainVector, GainDeltasVector);
		}
	}

	void MixMonoTo2ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer)
	{
		MixMonoTo2ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), MonoBuffer.Num());
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Input      | i0      | i0      | i1      | i1      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/
	void MixMonoTo2ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 InNumFrames)
	{
		checkf(InNumFrames >= AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER, TEXT("Buffer must have at least %i elements."), AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		checkf(0 == (InNumFrames % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER), TEXT("Buffer length be a multiple of %i."), AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);

		int32 OutPos = 0;
		for (int32 i = 0; i < InNumFrames; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			VectorRegister4Float Input = VectorLoad(&MonoBuffer[i]);
			VectorRegister4Float Output = VectorSwizzle(Input, 0, 0, 1, 1);
			VectorStore(Output, &DestinationBuffer[OutPos]);
			OutPos += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER;
			Output = VectorSwizzle(Input, 2, 2, 3, 3);
			VectorStore(Output, &DestinationBuffer[OutPos]);
			OutPos += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER;
		}
	}

	void Mix2ChannelsTo2ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		Mix2ChannelsTo2ChannelsFast(SourceBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 2, Gains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Gain1      | g0      | g1      | g0      | g1      |
	* |            | *       | *       | *       | *       |
	* | Input1     | i0      | i0      | i2      | i2      |
	* |            | +       | +       | +       | +       |
	* | Gain2      | g2      | g3      | g2      | g3      |
	* |            | *       | *       | *       | *       |
	* | Input2     | i1      | i1      | i3      | i3      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/

	void Mix2ChannelsTo2ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector1 = VectorLoadFloat2(Gains);
		const VectorRegister4Float GainVector2 = VectorLoadFloat2(Gains + 2);

		for (int32 i = 0; i < NumFrames; i += 2)
		{
			const VectorRegister4Float Input1 = VectorSet(SourceBuffer[i * 2], SourceBuffer[i * 2], SourceBuffer[i * 2 + 2], SourceBuffer[i * 2 + 2]);
			const VectorRegister4Float Input2 = VectorSet(SourceBuffer[i * 2 + 1], SourceBuffer[i * 2 + 1], SourceBuffer[i * 2 + 3], SourceBuffer[i * 2 + 3]);

			VectorRegister4Float Result = VectorMultiply(Input1, GainVector1);
			Result = VectorMultiplyAdd(Input2, GainVector2, Result);

			VectorStore(Result, &DestinationBuffer[i * 2]);
		}
	}

	void Mix2ChannelsTo2ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		Mix2ChannelsTo2ChannelsFast(SourceBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 2, StartGains, EndGains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Gain1      | g0      | g1      | g0      | g1      |
	* |            | *       | *       | *       | *       |
	* | Input1     | i0      | i0      | i2      | i2      |
	* |            | +       | +       | +       | +       |
	* | Gain2      | g2      | g3      | g2      | g3      |
	* |            | *       | *       | *       | *       |
	* | Input2     | i1      | i1      | i3      | i3      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/

	void Mix2ChannelsTo2ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		const VectorRegister4Float NumFramesVector = VectorSetFloat1(NumFrames / 2.0f);

		VectorRegister4Float GainVector1 = VectorLoadFloat2(StartGains);
		const VectorRegister4Float DestinationVector1 = VectorLoadFloat2(EndGains);
		const VectorRegister4Float GainDeltasVector1 = VectorDivide(VectorSubtract(DestinationVector1, GainVector1), NumFramesVector);

		// To help with stair stepping, we initialize the second frame in GainVector to be half a GainDeltas vector higher than the first frame.
		const VectorRegister4Float VectorOfHalf = VectorSet(0.5f, 0.5f, 1.0f, 1.0f);

		GainVector1 = VectorMultiplyAdd(GainDeltasVector1, VectorOfHalf, GainVector1);

		VectorRegister4Float GainVector2 = VectorLoadFloat2(StartGains + 2);
		const VectorRegister4Float DestinationVector2 = VectorLoadFloat2(EndGains + 2);
		const VectorRegister4Float GainDeltasVector2 = VectorDivide(VectorSubtract(DestinationVector2, GainVector2), NumFramesVector);

		GainVector2 = VectorMultiplyAdd(GainDeltasVector2, VectorOfHalf, GainVector2);

		for (int32 i = 0; i < NumFrames; i += 2)
		{
			const VectorRegister4Float Input1 = VectorSet(SourceBuffer[i * 2], SourceBuffer[i * 2], SourceBuffer[i * 2 + 2], SourceBuffer[i * 2 + 2]);
			const VectorRegister4Float Input2 = VectorSet(SourceBuffer[i * 2 + 1], SourceBuffer[i * 2 + 1], SourceBuffer[i * 2 + 3], SourceBuffer[i * 2 + 3]);

			VectorRegister4Float Result = VectorMultiply(Input1, GainVector1);
			Result = VectorMultiplyAdd(Input2, GainVector2, Result);

			VectorStore(Result, &DestinationBuffer[i * 2]);

			GainVector1 = VectorAdd(GainVector1, GainDeltasVector1);
			GainVector2 = VectorAdd(GainVector2, GainDeltasVector2);
		}
	}

	void Apply4ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT Gains)
	{
		Apply4ChannelGain(InterleavedBuffer.GetData(), InterleavedBuffer.Num(), Gains);
	}

	void Apply4ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector = VectorLoad(Gains);

		for (int32 i = 0; i < NumSamples; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			VectorRegister4Float Result = VectorLoad(&InterleavedBuffer[i]);
			Result = VectorMultiply(Result, GainVector);
			VectorStore(Result, &InterleavedBuffer[i]);
		}
	}

	void Apply4ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		Apply4ChannelGain(InterleavedBuffer.GetData(), InterleavedBuffer.Num(), StartGains, EndGains);
	}

	void Apply4ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		// Initialize GainVector at StartGains and compute GainDeltasVector:
		VectorRegister4Float GainVector = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector = VectorLoad(EndGains);
		const VectorRegister4Float NumFramesVector = VectorSetFloat1(NumSamples / (float)AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		const VectorRegister4Float GainDeltasVector = VectorDivide(VectorSubtract(DestinationVector, GainVector), NumFramesVector);

		for (int32 i = 0; i < NumSamples; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			VectorRegister4Float Result = VectorLoad(&InterleavedBuffer[i]);
			Result = VectorMultiply(Result, GainVector);
			VectorStore(Result, &InterleavedBuffer[i]);

			GainVector = VectorAdd(GainVector, GainDeltasVector);
		}
	}

	void MixMonoTo4ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		MixMonoTo4ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER, Gains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 1 frame per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Gain       | g0      | g1      | g2      | g3      |
	* |            | *       | *       | *       | *       |
	* | Input      | i0      | i0      | i0      | i0      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/

	void MixMonoTo4ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector = VectorLoad(Gains);

		for (int32 i = 0; i < NumFrames; i++)
		{
			VectorRegister4Float Result = VectorLoadFloat1(&MonoBuffer[i]);
			Result = VectorMultiply(Result, GainVector);
			VectorStore(Result, &DestinationBuffer[i*4]);
		}
	}

	void MixMonoTo4ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		MixMonoTo4ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER, StartGains, EndGains);
	}


	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 1 frame per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Gain       | g0      | g1      | g2      | g3      |
	* |            | *       | *       | *       | *       |
	* | Input      | i0      | i0      | i0      | i0      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/
	void MixMonoTo4ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		VectorRegister4Float GainVector = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector = VectorLoad(EndGains);
		const VectorRegister4Float NumFramesVector = VectorSetFloat1((float) NumFrames);
		const VectorRegister4Float GainDeltasVector = VectorDivide(VectorSubtract(DestinationVector, GainVector), NumFramesVector);

		for (int32 i = 0; i < NumFrames; i++)
		{
			VectorRegister4Float Result = VectorLoadFloat1(&MonoBuffer[i]);
			Result = VectorMultiply(Result, GainVector);
			VectorStore(Result, &DestinationBuffer[i * 4]);

			GainVector = VectorAdd(GainVector, GainDeltasVector);
		}
	}

	void Mix2ChannelsTo4ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		Mix2ChannelsTo4ChannelsFast(SourceBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER, Gains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 1 frame per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Gain1      | g0      | g1      | g2      | g3      |
	* |            | *       | *       | *       | *       |
	* | Input1     | i0      | i0      | i0      | i0      |
	* |            | +       | +       | +       | +       |
	* | Gain2      | g4      | g5      | g6      | g7      |
	* |            | *       | *       | *       | *       |
	* | Input2     | i1      | i1      | i1      | i1      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/

	void Mix2ChannelsTo4ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector1 = VectorLoad(Gains);
		const VectorRegister4Float GainVector2 = VectorLoad(Gains + 4);

		for (int32 i = 0; i < NumFrames; i++)
		{
			const VectorRegister4Float Input1 = VectorLoadFloat1(&SourceBuffer[i * 2]);
			const VectorRegister4Float Input2 = VectorLoadFloat1(&SourceBuffer[i * 2 + 1]);

			VectorRegister4Float Result = VectorMultiply(Input1, GainVector1);
			Result = VectorMultiplyAdd(Input2, GainVector2, Result);
			VectorStore(Result, &DestinationBuffer[i * 4]);
		}
	}

	void Mix2ChannelsTo4ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		Mix2ChannelsTo4ChannelsFast(SourceBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER, StartGains, EndGains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 1 frame per iteration:
	* +------------+---------+---------+---------+---------+
	* | VectorName | Index 0 | Index 1 | Index 2 | Index 3 |
	* +------------+---------+---------+---------+---------+
	* | Gain1      | g0      | g1      | g2      | g3      |
	* |            | *       | *       | *       | *       |
	* | Input1     | i0      | i0      | i0      | i0      |
	* |            | +       | +       | +       | +       |
	* | Gain2      | g4      | g5      | g6      | g7      |
	* |            | *       | *       | *       | *       |
	* | Input2     | i1      | i1      | i1      | i1      |
	* |            | =       | =       | =       | =       |
	* | Output     | o0      | o1      | o2      | o3      |
	* +------------+---------+---------+---------+---------+
	*/
	void Mix2ChannelsTo4ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		const VectorRegister4Float NumFramesVector = VectorSetFloat1((float) NumFrames);

		VectorRegister4Float GainVector1 = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector1 = VectorLoad(EndGains);
		const VectorRegister4Float GainDeltasVector1 = VectorDivide(VectorSubtract(DestinationVector1, GainVector1), NumFramesVector);

		VectorRegister4Float GainVector2 = VectorLoad(StartGains + 4);
		const VectorRegister4Float DestinationVector2 = VectorLoad(EndGains + 4);
		const VectorRegister4Float GainDeltasVector2 = VectorDivide(VectorSubtract(DestinationVector2, GainVector2), NumFramesVector);

		for (int32 i = 0; i < NumFrames; i++)
		{
			const VectorRegister4Float Input1 = VectorLoadFloat1(&SourceBuffer[i * 2]);
			const VectorRegister4Float Input2 = VectorLoadFloat1(&SourceBuffer[i * 2 + 1]);

			VectorRegister4Float Result = VectorMultiply(Input1, GainVector1);
			Result = VectorMultiplyAdd(Input2, GainVector2, Result);
			VectorStore(Result, &DestinationBuffer[i * 4]);

			GainVector1 = VectorAdd(GainVector1, GainDeltasVector1);
			GainVector2 = VectorAdd(GainVector2, GainDeltasVector2);
		}
	}

	void Apply6ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT Gains)
	{
		Apply6ChannelGain(InterleavedBuffer.GetData(), InterleavedBuffer.Num(), Gains);
	}

	void Apply6ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector1 = VectorLoad(Gains);
		const VectorRegister4Float GainVector2 = VectorSet(Gains[4], Gains[5], Gains[0], Gains[1]);
		const VectorRegister4Float GainVector3 = VectorLoad(&Gains[2]);

		for (int32 i = 0; i < NumSamples; i += 12)
		{
			VectorRegister4Float Result = VectorLoad(&InterleavedBuffer[i]);
			Result = VectorMultiply(Result, GainVector1);
			VectorStore(Result, &InterleavedBuffer[i]);

			Result = VectorLoad(&InterleavedBuffer[i + 4]);
			Result = VectorMultiply(Result, GainVector2);
			VectorStore(Result, &InterleavedBuffer[i + 4]);

			Result = VectorLoad(&InterleavedBuffer[i + 8]);
			Result = VectorMultiply(Result, GainVector3);
			VectorStore(Result, &InterleavedBuffer[i + 8]);
		}
	}

	void Apply6ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		Apply6ChannelGain(InterleavedBuffer.GetData(), InterleavedBuffer.Num(), StartGains, EndGains);
	}

	void Apply6ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		const VectorRegister4Float NumFramesVector = VectorSetFloat1(NumSamples / 12.0f);

		VectorRegister4Float GainVector1 = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector1 = VectorLoad(EndGains);
		const VectorRegister4Float GainDeltasVector1 = VectorDivide(VectorSubtract(DestinationVector1, GainVector1), NumFramesVector);

		VectorRegister4Float GainVector2 = VectorSet(StartGains[4], StartGains[5], StartGains[0], StartGains[1]);
		const VectorRegister4Float DestinationVector2 = VectorSet(EndGains[4], EndGains[5], EndGains[0], EndGains[1]);
		const VectorRegister4Float GainDeltasVector2 = VectorDivide(VectorSubtract(DestinationVector2, GainVector2), NumFramesVector);

		VectorRegister4Float GainVector3 = VectorLoad(&StartGains[2]);
		const VectorRegister4Float DestinationVector3 = VectorLoad(&EndGains[2]);
		const VectorRegister4Float GainDeltasVector3 = VectorDivide(VectorSubtract(DestinationVector3, GainVector3), NumFramesVector);

		for (int32 i = 0; i < NumSamples; i += 12)
		{
			VectorRegister4Float Result = VectorLoad(&InterleavedBuffer[i]);
			Result = VectorMultiply(Result, GainVector1);
			VectorStore(Result, &InterleavedBuffer[i]);

			GainVector1 = VectorAdd(GainVector1, GainDeltasVector1);

			Result = VectorLoad(&InterleavedBuffer[i + 4]);
			Result = VectorMultiply(Result, GainVector2);
			VectorStore(Result, &InterleavedBuffer[i + 4]);

			GainVector2 = VectorAdd(GainVector2, GainDeltasVector2);

			Result = VectorLoad(&InterleavedBuffer[i + 8]);
			Result = VectorMultiply(Result, GainVector3);
			VectorStore(Result, &InterleavedBuffer[i + 8]);

			GainVector3 = VectorAdd(GainVector3, GainDeltasVector3);
		}
	}

	void MixMonoTo6ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		MixMonoTo6ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 6, Gains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	* |            | Vector 1 |         |         |         | Vector 2 |         |         |         | Vector 3 |         |          |          |
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	* |            | Index 0  | Index 1 | Index 2 | Index 3 | Index 4  | Index 5 | Index 6 | Index 7 | Index 8  | Index 9 | Index 10 | Index 11 |
	* | Gain       | g0       | g1      | g2      | g3      | g4       | g5      | g0      | g1      | g2       | g3      | g4       | g5       |
	* |            | *        | *       | *       | *       | *        | *       | *       | *       | *        | *       | *        | *        |
	* | Input      | i0       | i0      | i0      | i0      | i0       | i0      | i1      | i1      | i1       | i1      | i1       | i1       |
	* |            | =        | =       | =       | =       | =        | =       | =       | =       | =        | =       | =        | =        |
	* | Output     | o0       | o1      | o2      | o3      | o4       | o5      | o6      | o7      | o8       | o9      | o10      | o11      |
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	*/

	void MixMonoTo6ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector1 = VectorLoad(Gains);
		const VectorRegister4Float GainVector2 = VectorSet(Gains[4], Gains[5], Gains[0], Gains[1]);
		const VectorRegister4Float GainVector3 = VectorLoad(&Gains[2]);

		for (int32 i = 0; i < NumFrames; i += 2)
		{
			const VectorRegister4Float Input1 = VectorLoadFloat1(&MonoBuffer[i]);
			const VectorRegister4Float Input2 = VectorSet(MonoBuffer[i], MonoBuffer[i], MonoBuffer[i + 1], MonoBuffer[i + 1]);
			const VectorRegister4Float Input3 = VectorLoadFloat1(&MonoBuffer[i + 1]);

			VectorRegister4Float Result = VectorMultiply(Input1, GainVector1);
			VectorStore(Result, &DestinationBuffer[i * 6]);

			Result = VectorMultiply(Input2, GainVector2);
			VectorStore(Result, &DestinationBuffer[i * 6 + 4]);

			Result = VectorMultiply(Input3, GainVector3);
			VectorStore(Result, &DestinationBuffer[i * 6 + 8]);
		}
	}

	void MixMonoTo6ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		MixMonoTo6ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 6, StartGains, EndGains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	* |            | Vector 1 |         |         |         | Vector 2 |         |         |         | Vector 3 |         |          |          |
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	* |            | Index 0  | Index 1 | Index 2 | Index 3 | Index 4  | Index 5 | Index 6 | Index 7 | Index 8  | Index 9 | Index 10 | Index 11 |
	* | Gain       | g0       | g1      | g2      | g3      | g4       | g5      | g0      | g1      | g2       | g3      | g4       | g5       |
	* |            | *        | *       | *       | *       | *        | *       | *       | *       | *        | *       | *        | *        |
	* | Input      | i0       | i0      | i0      | i0      | i0       | i0      | i1      | i1      | i1       | i1      | i1       | i1       |
	* |            | =        | =       | =       | =       | =        | =       | =       | =       | =        | =       | =        | =        |
	* | Output     | o0       | o1      | o2      | o3      | o4       | o5      | o6      | o7      | o8       | o9      | o10      | o11      |
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	*/

	void MixMonoTo6ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		const VectorRegister4Float NumFramesVector = VectorSetFloat1(NumFrames / 2.0f);

		VectorRegister4Float GainVector1 = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector1 = VectorLoad(EndGains);
		const VectorRegister4Float GainDeltasVector1 = VectorDivide(VectorSubtract(DestinationVector1, GainVector1), NumFramesVector);

		VectorRegister4Float GainVector2 = VectorSet(StartGains[4], StartGains[5], StartGains[0], StartGains[1]);
		const VectorRegister4Float DestinationVector2 = VectorSet(EndGains[4], EndGains[5], EndGains[0], EndGains[1]);
		const VectorRegister4Float GainDeltasVector2 = VectorDivide(VectorSubtract(DestinationVector2, GainVector2), NumFramesVector);

		VectorRegister4Float GainVector3 = VectorLoad(&StartGains[2]);
		const VectorRegister4Float DestinationVector3 = VectorLoad(&EndGains[2]);
		const VectorRegister4Float GainDeltasVector3 = VectorDivide(VectorSubtract(DestinationVector3, GainVector3), NumFramesVector);

		for (int32 i = 0; i < NumFrames; i += 2)
		{
			const VectorRegister4Float Input1 = VectorLoadFloat1(&MonoBuffer[i]);
			const VectorRegister4Float Input2 = VectorSet(MonoBuffer[i], MonoBuffer[i], MonoBuffer[i + 1], MonoBuffer[i + 1]);
			const VectorRegister4Float Input3 = VectorLoadFloat1(&MonoBuffer[i + 1]);

			VectorRegister4Float Result = VectorMultiply(Input1, GainVector1);
			VectorStore(Result, &DestinationBuffer[i * 6]);

			GainVector1 = VectorAdd(GainVector1, GainDeltasVector1);

			Result = VectorMultiply(Input2, GainVector2);
			VectorStore(Result, &DestinationBuffer[i * 6 + 4]);

			GainVector2 = VectorAdd(GainVector2, GainDeltasVector2);

			Result = VectorMultiply(Input3, GainVector3);
			VectorStore(Result, &DestinationBuffer[i * 6 + 8]);

			GainVector3 = VectorAdd(GainVector3, GainDeltasVector3);
		}
	}

	void Mix2ChannelsTo6ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		Mix2ChannelsTo6ChannelsFast(SourceBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 6, Gains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	* |        | Vector 1 |         |         |         | Vector 2 |         |         |         | Vector 3 |         |          |          |
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	* |        | Index 0  | Index 1 | Index 2 | Index 3 | Index 4  | Index 5 | Index 6 | Index 7 | Index 8  | Index 9 | Index 10 | Index 11 |
	* | Gain1  | g0       | g1      | g2      | g3      | g4       | g5      | g0      | g1      | g2       | g3      | g4       | g5       |
	* |        | *        | *       | *       | *       | *        | *       | *       | *       | *        | *       | *        | *        |
	* | Input1 | i0       | i0      | i0      | i0      | i0       | i0      | i2      | i2      | i2       | i2      | i2       | i2       |
	* |        | +        | +       | +       | +       | +        | +       | +       | +       | +        | +       | +        | +        |
	* | Gain2  | g6       | g7      | g8      | g9      | g10      | g11     | g6      | g7      | g8       | g9      | g10      | g11      |
	* |        | *        | *       | *       | *       | *        | *       | *       | *       | *        | *       | *        | *        |
	* | Input2 | i1       | i1      | i1      | i1      | i1       | i1      | i1      | i3      | i3       | i3      | i3       | i3       |
	* |        | =        | =       | =       | =       | =        | =       | =       | =       | =        | =       | =        | =        |
	* | Output | o0       | o1      | o2      | o3      | o4       | o5      | o6      | o7      | o8       | o9      | o10      | o11      |
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	*/

	void Mix2ChannelsTo6ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector11 = VectorLoad(Gains);
		const VectorRegister4Float GainVector21 = VectorSet(Gains[4], Gains[5], Gains[0], Gains[1]);
		const VectorRegister4Float GainVector31 = VectorLoad(&Gains[2]);

		const VectorRegister4Float GainVector12 = VectorLoad(Gains + 6);
		const VectorRegister4Float GainVector22 = VectorSet(Gains[10], Gains[11], Gains[6], Gains[7]);
		const VectorRegister4Float GainVector32 = VectorLoad(&Gains[8]);

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex += 2)
		{
			const int32 InputIndex = FrameIndex * 2;
			const int32 OutputIndex = FrameIndex * 6;

			const VectorRegister4Float Input11 = VectorLoadFloat1(&SourceBuffer[InputIndex]);
			const VectorRegister4Float Input21 = VectorSet(SourceBuffer[InputIndex], SourceBuffer[InputIndex], SourceBuffer[InputIndex + 2], SourceBuffer[InputIndex + 2]);
			const VectorRegister4Float Input31 = VectorLoadFloat1(&SourceBuffer[InputIndex + 2]);

			const VectorRegister4Float Input12 = VectorLoadFloat1(&SourceBuffer[InputIndex + 1]);
			const VectorRegister4Float Input22 = VectorSet(SourceBuffer[InputIndex + 1], SourceBuffer[InputIndex + 1], SourceBuffer[InputIndex + 3], SourceBuffer[InputIndex + 3]);
			const VectorRegister4Float Input32 = VectorLoadFloat1(&SourceBuffer[InputIndex + 3]);

			VectorRegister4Float Result = VectorMultiply(Input11, GainVector11);
			Result = VectorMultiplyAdd(Input12, GainVector12, Result);
			VectorStore(Result, &DestinationBuffer[OutputIndex]);

			Result = VectorMultiply(Input21, GainVector21);
			Result = VectorMultiplyAdd(Input22, GainVector22, Result);
			VectorStore(Result, &DestinationBuffer[OutputIndex + 4]);

			Result = VectorMultiply(Input31, GainVector31);
			Result = VectorMultiplyAdd(Input32, GainVector32, Result);
			VectorStore(Result, &DestinationBuffer[OutputIndex + 8]);
		}
	}

	void Mix2ChannelsTo6ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		Mix2ChannelsTo6ChannelsFast(SourceBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 6, StartGains, EndGains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 2 frames per iteration:
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	* |        | Vector 1 |         |         |         | Vector 2 |         |         |         | Vector 3 |         |          |          |
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	* |        | Index 0  | Index 1 | Index 2 | Index 3 | Index 4  | Index 5 | Index 6 | Index 7 | Index 8  | Index 9 | Index 10 | Index 11 |
	* | Gain1  | g0       | g1      | g2      | g3      | g4       | g5      | g0      | g1      | g2       | g3      | g4       | g5       |
	* |        | *        | *       | *       | *       | *        | *       | *       | *       | *        | *       | *        | *        |
	* | Input1 | i0       | i0      | i0      | i0      | i0       | i0      | i2      | i2      | i2       | i2      | i2       | i2       |
	* |        | +        | +       | +       | +       | +        | +       | +       | +       | +        | +       | +        | +        |
	* | Gain2  | g6       | g7      | g8      | g9      | g10      | g11     | g6      | g7      | g8       | g9      | g10      | g11      |
	* |        | *        | *       | *       | *       | *        | *       | *       | *       | *        | *       | *        | *        |
	* | Input2 | i1       | i1      | i1      | i1      | i1       | i1      | i3      | i3      | i3       | i3      | i3       | i3       |
	* |        | =        | =       | =       | =       | =        | =       | =       | =       | =        | =       | =        | =        |
	* | Output | o0       | o1      | o2      | o3      | o4       | o5      | o6      | o7      | o8       | o9      | o10      | o11      |
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+----------+---------+----------+----------+
	*/

	void Mix2ChannelsTo6ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		const VectorRegister4Float NumFramesVector = VectorSetFloat1(NumFrames / 2.0f);

		VectorRegister4Float GainVector11 = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector11 = VectorLoad(EndGains);
		const VectorRegister4Float GainDeltasVector11 = VectorDivide(VectorSubtract(DestinationVector11, GainVector11), NumFramesVector);

		VectorRegister4Float GainVector21 = VectorSet(StartGains[4], StartGains[5], StartGains[0], StartGains[1]);
		const VectorRegister4Float DestinationVector21 = VectorSet(EndGains[4], EndGains[5], EndGains[0], EndGains[1]);
		const VectorRegister4Float GainDeltasVector21 = VectorDivide(VectorSubtract(DestinationVector21, GainVector21), NumFramesVector);

		// In order to ease stair stepping, we ensure that the second frame is initialized to half the GainDelta more than the first frame.
		// This gives us a consistent increment across every frame.
		const VectorRegister4Float DeltaHalf21 = VectorSet(0.0f, 0.0f, 0.5f, 0.5f);
		GainVector21 = VectorMultiplyAdd(GainDeltasVector21, DeltaHalf21, GainVector21);

		VectorRegister4Float GainVector31 = VectorLoad(&StartGains[2]);
		const VectorRegister4Float DestinationVector31 = VectorLoad(&EndGains[2]);
		const VectorRegister4Float GainDeltasVector31 = VectorDivide(VectorSubtract(DestinationVector31, GainVector31), NumFramesVector);

		const VectorRegister4Float DeltaHalf31 = VectorSetFloat1(0.5f);
		GainVector31 = VectorMultiplyAdd(GainDeltasVector31, DeltaHalf31, GainVector31);

		VectorRegister4Float GainVector12 = VectorLoad(StartGains + 6);
		const VectorRegister4Float DestinationVector12 = VectorLoad(EndGains + 6);
		const VectorRegister4Float GainDeltasVector12 = VectorDivide(VectorSubtract(DestinationVector12, GainVector12), NumFramesVector);

		VectorRegister4Float GainVector22 = VectorSet(StartGains[10], StartGains[11], StartGains[6], StartGains[7]);
		const VectorRegister4Float DestinationVector22 = VectorSet(EndGains[10], EndGains[11], EndGains[6], EndGains[7]);
		const VectorRegister4Float GainDeltasVector22 = VectorDivide(VectorSubtract(DestinationVector22, GainVector22), NumFramesVector);
		GainVector22 = VectorMultiplyAdd(GainDeltasVector22, DeltaHalf21, GainVector22);

		VectorRegister4Float GainVector32 = VectorLoad(StartGains + 8);
		const VectorRegister4Float DestinationVector32 = VectorLoad(EndGains + 8);
		const VectorRegister4Float GainDeltasVector32 = VectorDivide(VectorSubtract(DestinationVector32, GainVector32), NumFramesVector);
		GainVector32 = VectorMultiplyAdd(GainDeltasVector32, DeltaHalf31, GainVector32);

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex += 2)
		{
			const int32 InputIndex = FrameIndex * 2;
			const int32 OutputIndex = FrameIndex * 6;

			const VectorRegister4Float Input11 = VectorLoadFloat1(&SourceBuffer[InputIndex]);
			const VectorRegister4Float Input21 = VectorSet(SourceBuffer[InputIndex], SourceBuffer[InputIndex], SourceBuffer[InputIndex + 2], SourceBuffer[InputIndex + 2]);
			const VectorRegister4Float Input31 = VectorLoadFloat1(&SourceBuffer[InputIndex + 2]);

			const VectorRegister4Float Input12 = VectorLoadFloat1(&SourceBuffer[InputIndex + 1]);
			const VectorRegister4Float Input22 = VectorSet(SourceBuffer[InputIndex + 1], SourceBuffer[InputIndex + 1], SourceBuffer[InputIndex + 3], SourceBuffer[InputIndex + 3]);
			const VectorRegister4Float Input32 = VectorLoadFloat1(&SourceBuffer[InputIndex + 3]);

			VectorRegister4Float Result = VectorMultiply(Input11, GainVector11);
			Result = VectorMultiplyAdd(Input12, GainVector12, Result);
			VectorStore(Result, &DestinationBuffer[OutputIndex]);

			GainVector11 = VectorAdd(GainVector11, GainDeltasVector11);
			GainVector12 = VectorAdd(GainVector12, GainDeltasVector12);

			Result = VectorMultiply(Input21, GainVector21);
			Result = VectorMultiplyAdd(Input22, GainVector22, Result);
			VectorStore(Result, &DestinationBuffer[OutputIndex + 4]);

			GainVector21 = VectorAdd(GainVector21, GainDeltasVector21);
			GainVector22 = VectorAdd(GainVector22, GainDeltasVector22);

			Result = VectorMultiply(Input31, GainVector31);
			Result = VectorMultiplyAdd(Input32, GainVector31, Result);
			VectorStore(Result, &DestinationBuffer[OutputIndex + 8]);

			GainVector31 = VectorAdd(GainVector31, GainDeltasVector31);
			GainVector32 = VectorAdd(GainVector32, GainDeltasVector32);
		}
	}

	void Apply8ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT Gains)
	{
		Apply8ChannelGain(InterleavedBuffer.GetData(), InterleavedBuffer.Num(), Gains);
	}

	void Apply8ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector1 = VectorLoad(Gains);
		const VectorRegister4Float GainVector2 = VectorLoad(Gains + 4);

		for (int32 i = 0; i < NumSamples; i += 8)
		{
			VectorRegister4Float Result = VectorLoad(&InterleavedBuffer[i]);
			Result = VectorMultiply(Result, GainVector1);
			VectorStore(Result, &InterleavedBuffer[i]);

			Result = VectorLoad(&InterleavedBuffer[i + 4]);
			Result = VectorMultiply(Result, GainVector2);
			VectorStore(Result, &InterleavedBuffer[i + 4]);
		}
	}

	void Apply8ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		Apply8ChannelGain(InterleavedBuffer.GetData(), InterleavedBuffer.Num(), StartGains, EndGains);
	}

	void Apply8ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		const VectorRegister4Float NumFramesVector = VectorSetFloat1(NumSamples / 8.0f);

		VectorRegister4Float GainVector1 = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector1 = VectorLoad(EndGains);
		const VectorRegister4Float GainDeltasVector1 = VectorDivide(VectorSubtract(DestinationVector1, GainVector1), NumFramesVector);

		VectorRegister4Float GainVector2 = VectorLoad(StartGains + 4);
		const VectorRegister4Float DestinationVector2 = VectorLoad(EndGains + 4);
		const VectorRegister4Float GainDeltasVector2 = VectorDivide(VectorSubtract(DestinationVector2, GainVector2), NumFramesVector);

		for (int32 i = 0; i < NumSamples; i += 8)
		{
			VectorRegister4Float Result = VectorLoad(&InterleavedBuffer[i]);
			Result = VectorMultiply(Result, GainVector1);
			VectorStore(Result, &InterleavedBuffer[i]);

			GainVector1 = VectorAdd(GainVector1, GainDeltasVector1);

			Result = VectorLoad(&InterleavedBuffer[i + 4]);
			Result = VectorMultiply(Result, GainVector2);
			VectorStore(Result, &InterleavedBuffer[i + 4]);

			GainVector2 = VectorAdd(GainVector2, GainDeltasVector2);
		}
	}

	void MixMonoTo8ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		MixMonoTo8ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 8, Gains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 1 frames per iteration:
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+
	* |            | Vector 1 |         |         |         | Vector 2 |         |         |         |
	* | VectorName | Index 0  | Index 1 | Index 2 | Index 3 | Index 4  | Index 5 | Index 6 | Index 7 |
	* | Gain       | g0       | g1      | g2      | g3      | g4       | g5      | g6      | g7      |
	* |            | *        | *       | *       | *       | *        | *       | *       | *       |
	* | Input      | i0       | i0      | i0      | i0      | i0       | i0      | i0      | i0      |
	* |            | =        | =       | =       | =       | =        | =       | =       | =       |
	* | Output     | o0       | o1      | o2      | o3      | o4       | o5      | o6      | o7      |
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+
	*/

	void MixMonoTo8ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector1 = VectorLoad(Gains);
		const VectorRegister4Float GainVector2 = VectorLoad(Gains + 4);

		for (int32 i = 0; i < NumFrames; i++)
		{
			VectorRegister4Float Result = VectorLoadFloat1(&MonoBuffer[i]);
			Result = VectorMultiply(Result, GainVector1);
			VectorStore(Result, &DestinationBuffer[i * 8]);

			Result = VectorLoadFloat1(&MonoBuffer[i]);
			Result = VectorMultiply(Result, GainVector2);
			VectorStore(Result, &DestinationBuffer[i * 8 + 4]);
		}
	}

	void MixMonoTo8ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		MixMonoTo8ChannelsFast(MonoBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 8, StartGains, EndGains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 1 frames per iteration:
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+
	* |            | Vector 1 |         |         |         | Vector 2 |         |         |         |
	* | VectorName | Index 0  | Index 1 | Index 2 | Index 3 | Index 4  | Index 5 | Index 6 | Index 7 |
	* | Gain       | g0       | g1      | g2      | g3      | g4       | g5      | g6      | g7      |
	* |            | *        | *       | *       | *       | *        | *       | *       | *       |
	* | Input      | i0       | i0      | i0      | i0      | i0       | i0      | i0      | i0      |
	* |            | =        | =       | =       | =       | =        | =       | =       | =       |
	* | Output     | o0       | o1      | o2      | o3      | o4       | o5      | o6      | o7      |
	* +------------+----------+---------+---------+---------+----------+---------+---------+---------+
	*/
	void MixMonoTo8ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		const VectorRegister4Float NumFramesVector = VectorSetFloat1((float) NumFrames);

		VectorRegister4Float GainVector1 = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector1 = VectorLoad(EndGains);
		const VectorRegister4Float GainDeltasVector1 = VectorDivide(VectorSubtract(DestinationVector1, GainVector1), NumFramesVector);

		VectorRegister4Float GainVector2 = VectorLoad(StartGains + 4);
		const VectorRegister4Float DestinationVector2 = VectorLoad(EndGains + 4);
		const VectorRegister4Float GainDeltasVector2 = VectorDivide(VectorSubtract(DestinationVector2, GainVector2), NumFramesVector);

		for (int32 i = 0; i < NumFrames; i++)
		{
			const VectorRegister4Float Input = VectorLoadFloat1(&MonoBuffer[i]);
			VectorRegister4Float Result = VectorMultiply(Input, GainVector1);
			VectorStore(Result, &DestinationBuffer[i * 8]);

			GainVector1 = VectorAdd(GainVector1, GainDeltasVector1);

			Result = VectorMultiply(Input, GainVector2);
			VectorStore(Result, &DestinationBuffer[i * 8 + 4]);

			GainVector2 = VectorAdd(GainVector2, GainDeltasVector2);
		}
	}

	void Mix2ChannelsTo8ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		Mix2ChannelsTo8ChannelsFast(SourceBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 8, Gains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 1 frames per iteration:
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+
	* |        | Vector 1 |         |         |         | Vector 2 |         |         |         |
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+
	* |        | Index 0  | Index 1 | Index 2 | Index 3 | Index 4  | Index 5 | Index 6 | Index 7 |
	* | Gain1  | g0       | g1      | g2      | g3      | g4       | g5      | g6      | g7      |
	* |        | *        | *       | *       | *       | *        | *       | *       | *       |
	* | Input1 | i0       | i0      | i0      | i0      | i0       | i0      | i0      | i0      |
	* |        | +        | +       | +       | +       | +        | +       | +       | +       |
	* | Gain2  | g8       | g9      | g10     | g11     | g12      | g13     | g14     | g5      |
	* |        | *        | *       | *       | *       | *        | *       | *       | *       |
	* | Input2 | i1       | i1      | i1      | i1      | i1       | i1      | i1      | i1      |
	* |        | =        | =       | =       | =       | =        | =       | =       | =       |
	* | Output | o0       | o1      | o2      | o3      | o4       | o5      | o6      | o7      |
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+
	*/

	void Mix2ChannelsTo8ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		const VectorRegister4Float GainVector11 = VectorLoad(Gains);
		const VectorRegister4Float GainVector21 = VectorLoad(Gains + 4);
		const VectorRegister4Float GainVector12 = VectorLoad(Gains + 8);
		const VectorRegister4Float GainVector22 = VectorLoad(Gains + 12);

		for (int32 i = 0; i < NumFrames; i++)
		{
			const VectorRegister4Float Input1 = VectorLoadFloat1(&SourceBuffer[i*2]);
			const VectorRegister4Float Input2 = VectorLoadFloat1(&SourceBuffer[i * 2 + 1]);

			VectorRegister4Float Result = VectorMultiply(Input1, GainVector11);
			Result = VectorMultiplyAdd(Input2, GainVector12, Result);

			VectorStore(Result, &DestinationBuffer[i * 8]);

			Result = VectorMultiply(Input1, GainVector21);
			Result = VectorMultiplyAdd(Input2, GainVector22, Result);
			VectorStore(Result, &DestinationBuffer[i * 8 + 4]);
		}
	}

	void Mix2ChannelsTo8ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		Mix2ChannelsTo8ChannelsFast(SourceBuffer.GetData(), DestinationBuffer.GetData(), DestinationBuffer.Num() / 8, StartGains, EndGains);
	}

	/**
	* See CHANNEL MIXING OPERATIONS above for more info.
	* 1 frames per iteration:
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+
	* |        | Vector 1 |         |         |         | Vector 2 |         |         |         |
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+
	* |        | Index 0  | Index 1 | Index 2 | Index 3 | Index 4  | Index 5 | Index 6 | Index 7 |
	* | Gain1  | g0       | g1      | g2      | g3      | g4       | g5      | g6      | g7      |
	* |        | *        | *       | *       | *       | *        | *       | *       | *       |
	* | Input1 | i0       | i0      | i0      | i0      | i0       | i0      | i0      | i0      |
	* |        | +        | +       | +       | +       | +        | +       | +       | +       |
	* | Gain2  | g8       | g9      | g10     | g11     | g12      | g13     | g14     | g15     |
	* |        | *        | *       | *       | *       | *        | *       | *       | *       |
	* | Input2 | i1       | i1      | i1      | i1      | i1       | i1      | i1      | i1      |
	* |        | =        | =       | =       | =       | =        | =       | =       | =       |
	* | Output | o0       | o1      | o2      | o3      | o4       | o5      | o6      | o7      |
	* +--------+----------+---------+---------+---------+----------+---------+---------+---------+
	*/

	void Mix2ChannelsTo8ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		const VectorRegister4Float NumFramesVector = VectorSetFloat1((float) NumFrames);

		VectorRegister4Float GainVector11 = VectorLoad(StartGains);
		const VectorRegister4Float DestinationVector11 = VectorLoad(EndGains);
		const VectorRegister4Float GainDeltasVector11 = VectorDivide(VectorSubtract(DestinationVector11, GainVector11), NumFramesVector);

		VectorRegister4Float GainVector21 = VectorLoad(StartGains + 4);
		const VectorRegister4Float DestinationVector21 = VectorLoad(EndGains + 4);
		const VectorRegister4Float GainDeltasVector21 = VectorDivide(VectorSubtract(DestinationVector21, GainVector21), NumFramesVector);

		VectorRegister4Float GainVector12 = VectorLoad(StartGains + 8);
		const VectorRegister4Float DestinationVector12 = VectorLoad(EndGains + 8);
		const VectorRegister4Float GainDeltasVector12 = VectorDivide(VectorSubtract(DestinationVector12, GainVector12), NumFramesVector);

		VectorRegister4Float GainVector22 = VectorLoad(StartGains + 12);
		const VectorRegister4Float DestinationVector22 = VectorLoad(EndGains + 12);
		const VectorRegister4Float GainDeltasVector22 = VectorDivide(VectorSubtract(DestinationVector22, GainVector22), NumFramesVector);

		for (int32 i = 0; i < NumFrames; i++)
		{
			const VectorRegister4Float Input1 = VectorLoadFloat1(&SourceBuffer[i*2]);
			const VectorRegister4Float Input2 = VectorLoadFloat1(&SourceBuffer[i * 2 + 1]);

			VectorRegister4Float Result = VectorMultiply(Input1, GainVector11);
			Result = VectorMultiplyAdd(Input2, GainVector12, Result);
			VectorStore(Result, &DestinationBuffer[i * 8]);

			GainVector11 = VectorAdd(GainVector11, GainDeltasVector11);
			GainVector12 = VectorAdd(GainVector12, GainDeltasVector12);

			Result = VectorMultiply(Input1, GainVector21);
			Result = VectorMultiplyAdd(Input2, GainVector22, Result);
			VectorStore(Result, &DestinationBuffer[i * 8 + 4]);

			GainVector21 = VectorAdd(GainVector21, GainDeltasVector21);
			GainVector22 = VectorAdd(GainVector22, GainDeltasVector22);
		}
	}

	/**
	 * These functions are non-vectorized versions of the Mix[N]ChannelsTo[N]Channels functions above:
	 */
	void DownmixBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains)
	{
		DownmixBuffer(NumSourceChannels, NumDestinationChannels, SourceBuffer.GetData(), DestinationBuffer.GetData(), SourceBuffer.Num() / NumSourceChannels, Gains);
	}

	void DownmixBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains)
	{
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			float* RESTRICT OutputFrame = &DestinationBuffer[FrameIndex * NumDestinationChannels];
			const float* RESTRICT InputFrame = &SourceBuffer[FrameIndex * NumSourceChannels];

			for (int32 OutputChannelIndex = 0; OutputChannelIndex < NumDestinationChannels; OutputChannelIndex++)
			{
				float Value = 0.f;
				for (int32 InputChannelIndex = 0; InputChannelIndex < NumSourceChannels; InputChannelIndex++)
				{
					Value += InputFrame[InputChannelIndex] * Gains[InputChannelIndex * NumDestinationChannels + OutputChannelIndex];
				}
				OutputFrame[OutputChannelIndex] = Value;
			}
		}
	}

	void DownmixBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		DownmixBuffer(NumSourceChannels, NumDestinationChannels, SourceBuffer.GetData(), DestinationBuffer.GetData(), SourceBuffer.Num() / NumSourceChannels, StartGains, EndGains);
	}

	void DownmixBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, float* RESTRICT StartGains, const float* RESTRICT EndGains)
	{
		// First, build a map of the per-frame delta that we will use to increment StartGains every frame:
		check(NumSourceChannels <= 8 && NumDestinationChannels <= 8);
		alignas(16) float GainDeltas[8 * 8];

		for (int32 OutputChannelIndex = 0; OutputChannelIndex < NumDestinationChannels; OutputChannelIndex++)
		{
			for (int32 InputChannelIndex = 0; InputChannelIndex < NumSourceChannels; InputChannelIndex++)
			{
				const int32 GainMatrixIndex = InputChannelIndex * NumDestinationChannels + OutputChannelIndex;
				GainDeltas[GainMatrixIndex] = (EndGains[GainMatrixIndex] - StartGains[GainMatrixIndex]) / NumFrames;
			}
		}

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			float* RESTRICT OutputFrame = &DestinationBuffer[FrameIndex * NumDestinationChannels];
			const float* RESTRICT InputFrame = &SourceBuffer[FrameIndex * NumSourceChannels];

			for (int32 OutputChannelIndex = 0; OutputChannelIndex < NumDestinationChannels; OutputChannelIndex++)
			{
				float Value = 0.f;
				for (int32 InputChannelIndex = 0; InputChannelIndex < NumSourceChannels; InputChannelIndex++)
				{
					const int32 GainMatrixIndex = InputChannelIndex * NumDestinationChannels + OutputChannelIndex;
					Value += InputFrame[InputChannelIndex] * StartGains[GainMatrixIndex];
					StartGains[GainMatrixIndex] += GainDeltas[GainMatrixIndex];
				}
				OutputFrame[OutputChannelIndex] = Value;
			}
		}
	}


	void DownmixAndSumIntoBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& BufferToSumTo, const float* RESTRICT Gains)
	{
		DownmixAndSumIntoBuffer(NumSourceChannels, NumDestinationChannels, SourceBuffer.GetData(), BufferToSumTo.GetData(), SourceBuffer.Num() / NumSourceChannels, Gains);
	}

	void DownmixAndSumIntoBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const float* RESTRICT SourceBuffer, float* RESTRICT BufferToSumTo, int32 NumFrames, const float* RESTRICT Gains)
	{
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			float* RESTRICT OutputFrame = &BufferToSumTo[FrameIndex * NumDestinationChannels];
			const float* RESTRICT InputFrame = &SourceBuffer[FrameIndex * NumSourceChannels];

			for (int32 OutputChannelIndex = 0; OutputChannelIndex < NumDestinationChannels; OutputChannelIndex++)
			{
				float Value = 0.f;
				for (int32 InputChannelIndex = 0; InputChannelIndex < NumSourceChannels; InputChannelIndex++)
				{
					 Value += InputFrame[InputChannelIndex] * Gains[InputChannelIndex * NumDestinationChannels + OutputChannelIndex];
				}
				OutputFrame[OutputChannelIndex] += Value;
			}
		}
	}

	/** Interleaves samples from two input buffers */
	void BufferInterleave2ChannelFast(const FAlignedFloatBuffer& InBuffer1, const FAlignedFloatBuffer& InBuffer2, FAlignedFloatBuffer& OutBuffer)
	{
		checkf(InBuffer1.Num() == InBuffer2.Num(), TEXT("InBuffer1 Num not equal to InBuffer2 Num"));

		const int32 InNum = InBuffer1.Num();

		OutBuffer.Reset(2 * InNum);
		OutBuffer.AddUninitialized(2 * InNum);
		
		BufferInterleave2ChannelFast(InBuffer1.GetData(), InBuffer2.GetData(), OutBuffer.GetData(), InNum);
	}

	/** Interleaves samples from two input buffers */
	void BufferInterleave2ChannelFast(const float* RESTRICT InBuffer1, const float* RESTRICT InBuffer2, float* RESTRICT OutBuffer, const int32 InNum)
	{
		checkf(InNum >= 4, TEXT("Buffer must have at least %i elements."), AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		checkf(0 == (InNum % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER), TEXT("Buffer length be a multiple of %i."), AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);

		const int32 OutNum = 2 * InNum;

		int32 OutPos = 0;
		for (int32 i = 0; i < InNum; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			// Vector1[L0, L1, L2, L3]
			VectorRegister4Float Vector1 = VectorLoad(&InBuffer1[i]);
			// Vector2[R0, R1, R2, R3]
			VectorRegister4Float Vector2 = VectorLoad(&InBuffer2[i]);

			// HalfInterleaved[L0, L1, R0, R1]
			VectorRegister4Float HalfInterleaved = VectorShuffle(Vector1, Vector2, 0, 1, 0, 1);
			// Interleaved[L0, R0, L1, R1]
			VectorRegister4Float Interleaved = VectorSwizzle(HalfInterleaved, 0, 2, 1, 3);
			VectorStore(Interleaved, &OutBuffer[OutPos]);
			OutPos += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER;

			// HalfInterleaved[L2, L3, R2, R3]
			HalfInterleaved = VectorShuffle(Vector1, Vector2, 2, 3, 2, 3);
			// Interleaved[L2, R2, L3, R3]
			Interleaved = VectorSwizzle(HalfInterleaved, 0, 2, 1, 3);
			VectorStore(Interleaved, &OutBuffer[OutPos]);
			OutPos += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER;
		}
	}

	/** Deinterleaves samples from a 2 channel input buffer */
	void BufferDeinterleave2ChannelFast(const FAlignedFloatBuffer& InBuffer, FAlignedFloatBuffer& OutBuffer1, FAlignedFloatBuffer& OutBuffer2)
	{
		const int32 InNum = InBuffer.Num();
		const int32 InNumFrames = InNum / 2;
		const int32 OutNum = InNumFrames;

		OutBuffer1.Reset(OutNum);
		OutBuffer2.Reset(OutNum);
		OutBuffer1.AddUninitialized(OutNum);
		OutBuffer2.AddUninitialized(OutNum);
		
		BufferDeinterleave2ChannelFast(InBuffer.GetData(), OutBuffer1.GetData(), OutBuffer2.GetData(), InNumFrames);
	}

	/** Deinterleaves samples from a 2 channel input buffer */
	void BufferDeinterleave2ChannelFast(const float* RESTRICT InBuffer, float* RESTRICT OutBuffer1, float* RESTRICT OutBuffer2, const int32 InNumFrames)
	{
		checkf(InNumFrames >= AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER, TEXT("Buffer must have at least %i elements."), AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		checkf(0 == (InNumFrames % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER), TEXT("Buffer length be a multiple of %i."), AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);

		int32 InNum = InNumFrames * 2;
		int32 OutPos = 0;
		for (int32 InPos = 0; InPos < InNum; InPos += 8)
		{
			// load 4 frames (2 frames per vector)
			VectorRegister4Float InVector1 = VectorLoad(&InBuffer[InPos]);
			VectorRegister4Float InVector2 = VectorLoad(&InBuffer[InPos + 4]);

			// Write channel 0
			VectorRegister4Float OutVector = VectorShuffle(InVector1, InVector2, 0, 2, 0, 2);
			VectorStore(OutVector, &OutBuffer1[OutPos]);

			// Write channel 1
			OutVector = VectorShuffle(InVector1, InVector2, 1, 3, 1, 3);
			VectorStore(OutVector, &OutBuffer2[OutPos]);

			OutPos += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER;
		}
	}

	/** Sums 2 channel interleaved input samples. OutSamples[n] = InSamples[2n] + InSamples[2n + 1] */
	void BufferSum2ChannelToMonoFast(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples)
	{
		const int32 InNum = InSamples.Num();
		const int32 Frames = InNum / 2;

		OutSamples.Reset(Frames);
		OutSamples.AddUninitialized(Frames);
		
		BufferSum2ChannelToMonoFast(InSamples.GetData(), OutSamples.GetData(), Frames);
	}

	/** Sums 2 channel interleaved input samples. OutSamples[n] = InSamples[2n] + InSamples[2n + 1] */
	void BufferSum2ChannelToMonoFast(const float* RESTRICT InSamples, float* RESTRICT OutSamples, const int32 InNumFrames)
	{
		checkf(InNumFrames >= AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER, TEXT("Buffer must have at least %i elements."), AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		checkf(0 == (InNumFrames % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER), TEXT("Buffer length be a multiple of %i."), AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);

		const int32 InNum = InNumFrames * 2;
		int32 OutPos = 0;
		for (int32 i = 0; i < InNum; i += 8)
		{
			// Load 4 frames (2 frames per vector)
			// Buffer1[L0, R0, L1, R1]
			VectorRegister4Float Buffer1 = VectorLoad(&InSamples[i]);
			// Buffer2[L2, R2, L3, R3]
			VectorRegister4Float Buffer2 = VectorLoad(&InSamples[i + 4]);

			// Shuffle samples into order
			// Channel0[L0, L1, L2, L3]
			VectorRegister4Float Channel0 = VectorShuffle(Buffer1, Buffer2, 0, 2, 0, 2);
			// Channel1[R0, R1, R2, R3]
			VectorRegister4Float Channel1 = VectorShuffle(Buffer1, Buffer2, 1, 3, 1, 3);

			// Sum left and right.
			// Out[L0 + R0, L1 + R1, L2 + R2, L3 + R3]
			VectorRegister4Float Out = VectorAdd(Channel0, Channel1);

			VectorStore(Out, &OutSamples[OutPos]);
			OutPos += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER;
		}
	}

	// class FBufferLinearEase implementation
	FBufferLinearEase::FBufferLinearEase() {}
	FBufferLinearEase::FBufferLinearEase(const FAlignedFloatBuffer & InSourceValues, const FAlignedFloatBuffer & InTargetValues, int32 InLerpLength)
	{
		Init(InSourceValues, InTargetValues, InLerpLength);
	}

	FBufferLinearEase::~FBufferLinearEase() {}

	void FBufferLinearEase::Init(const FAlignedFloatBuffer & InSourceValues, const FAlignedFloatBuffer & InTargetValues, int32 InLerpLength)
	{
		check(InSourceValues.Num());
		check(InTargetValues.Num());
		check(InLerpLength > 0);

		BufferLength = InSourceValues.Num();

		check(InTargetValues.Num() == BufferLength);
		LerpLength = InLerpLength;
		CurrentLerpStep = 0;

		// init deltas
		DeltaBuffer.Reset();
		DeltaBuffer.AddZeroed(BufferLength);

		const float OneOverLerpLength = 1.0f / static_cast<float>(LerpLength);

		ArraySubtract(InTargetValues, InSourceValues, DeltaBuffer);
		ArrayMultiplyByConstantInPlace(DeltaBuffer, OneOverLerpLength);
	}

	bool FBufferLinearEase::Update(FAlignedFloatBuffer & InSourceValues)
	{
		check(InSourceValues.Num() == BufferLength);
		check(CurrentLerpStep != LerpLength);

		ArrayMixIn(DeltaBuffer, InSourceValues);

		if (++CurrentLerpStep == LerpLength)
		{
			return true;
		}

		return false;
	}

	bool FBufferLinearEase::Update(uint32 StepsToJumpForward, FAlignedFloatBuffer & InSourceValues)
	{
		check(InSourceValues.Num() == BufferLength);
		check(CurrentLerpStep != LerpLength);
		check(StepsToJumpForward);

		bool bIsComplete = false;
		if ((CurrentLerpStep += StepsToJumpForward) >= LerpLength)
		{
			StepsToJumpForward -= (CurrentLerpStep - LerpLength);
			CurrentLerpStep = LerpLength;
			bIsComplete = true;
		}

		ArrayMixIn(DeltaBuffer, InSourceValues, static_cast<float>(StepsToJumpForward));

		return bIsComplete;

	}
	const FAlignedFloatBuffer & FBufferLinearEase::GetDeltaBuffer()
	{
		return DeltaBuffer;
	}
}
