// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/ChannelMap.h"
#include "AudioDefines.h"

namespace Audio
{
	namespace ChannelMapPrivate
	{
		// Tables based on Ac-3 down-mixing
		// Rows: output speaker configuration
		// Cols: input channels

		static constexpr float ToMonoMatrix[ChannelMapMaxNumChannels * 1] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight  
			0.707f,			0.707f,		1.0f,		0.0f,			0.5f,		0.5f,		0.5f,		0.5f,		// FrontLeft
		};

		static constexpr float VorbisToMonoMatrix[ChannelMapVorbisNumChannels * 1] =
		{
			// FrontLeft	Center		FrontRight	SideLeft		SideRight	LowFrequency		
			0.707f,			1.0f,		0.707f,		0.5f,			0.5f,		0.0f,		// FrontLeft
		};

		static constexpr float ToStereoMatrix[ChannelMapMaxNumChannels * 2] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight  
			1.0f,			0.0f,		0.707f,		0.0f,			0.707f,		0.0f,		0.707f,		0.0f,		// FrontLeft
			0.0f,			1.0f,		0.707f,		0.0f,			0.0f,		0.707f,		0.0f,		0.707f,		// FrontRight
		};

		static constexpr float VorbisToStereoMatrix[ChannelMapVorbisNumChannels * 2] =
		{
			// FrontLeft	Center		FrontRight	SideLeft		SideRight	LowFrequency		
			1.0f,			0.707f,		0.0f,		0.707f,			0.0f,		0.0f,		// FrontLeft
			0.0f,			0.707f,		1.0f,		0.0f,			0.707f,		0.0f,		// FrontRight
		};

		static constexpr float ToTriMatrix[ChannelMapMaxNumChannels * 3] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight  
			1.0f,			0.0f,		0.0f,		0.0f,			0.707f,		0.0f,		0.707f,		0.0f,		// FrontLeft
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.707f,		0.0f,		0.707f,		// FrontRight
			0.0f,			0.0f,		1.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// Center
		};

		static constexpr float VorbisToTriMatrix[ChannelMapVorbisNumChannels * 3] =
		{
			// FrontLeft	Center		FrontRight	SideLeft		SideRight	LowFrequency		
			1.0f,			0.0f,		0.0f,		0.707f,			0.0f,		0.0f,		// FrontLeft
			0.0f,			0.0f,		1.0f,		0.0f,			0.707f,		0.0f,		// FrontRight
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// Center
		};

		static constexpr float ToQuadMatrix[ChannelMapMaxNumChannels * 4] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight	
			1.0f,			0.0f,		0.707f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
			0.0f,			1.0f,		0.707f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		1.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		0.0f,		1.0f,		// SideRight
		};

		static constexpr float VorbisToQuadMatrix[ChannelMapVorbisNumChannels * 4] =
		{
			// FrontLeft	Center		FrontRight	SideLeft		SideRight	LowFrequency		
			1.0f,			0.707f,		0.0f,		0.0f,			0.0f,		0.0f,		// FrontLeft
			0.0f,			0.707f,		1.0f,		0.0f,			0.0f,		0.0f,		// FrontRight
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		// SideRight
		};

		static constexpr float To5Matrix[ChannelMapMaxNumChannels * 5] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight	
			1.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
			0.0f,			0.0f,		1.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// Center
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		1.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		0.0f,		1.0f,		// SideRight
		};

		static constexpr float VorbisTo5Matrix[ChannelMapVorbisNumChannels * 5] =
		{
			// FrontLeft	Center		FrontRight	SideLeft		SideRight	LowFrequency		
			1.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// FrontLeft
			0.0f,			0.0f,		1.0f,		0.0f,			0.0f,		0.0f,		// FrontRight
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// Center
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		// SideRight
		};

		static constexpr float To5Point1Matrix[ChannelMapMaxNumChannels * 6] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight	
			1.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
			0.0f,			0.0f,		1.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// Center
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// LowFrequency
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		1.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		0.0f,		1.0f,		// SideRight
		};

		static constexpr float VorbisTo5Point1Matrix[ChannelMapVorbisNumChannels * 6] =
		{
			// FrontLeft	Center		FrontRight	SideLeft		SideRight	LowFrequency	
			1.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// FrontLeft
			0.0f,			0.0f,		1.0f,		0.0f,			0.0f,		0.0f,		// FrontRight
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// Center
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		// LowFrequency
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		// SideRight
		};

		static constexpr float ToHexMatrix[ChannelMapMaxNumChannels * 7] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight	
			1.0f,			0.0f,		0.707f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
			0.0f,			1.0f,		0.707f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		1.0f,		0.0f,		// BackLeft
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// LFE
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		1.0f,		// BackRight
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		0.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		0.0f,		0.0f,		// SideRight
		};

		static constexpr float VorbisToHexMatrix[ChannelMapVorbisNumChannels * 7] =
		{
			// FrontLeft	Center		FrontRight	SideLeft		SideRight	LowFrequency	
			1.0f,			0.707f,		0.0f,		0.0f,			0.0f,		0.0f,		// FrontLeft
			0.0f,			0.707f,		1.0f,		0.0f,			0.0f,		0.0f,		// FrontRight
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// BackLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		// LFE
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// BackRight
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		// SideRight
		};

#if PLATFORM_MICROSOFT

		static constexpr float To7Point1Matrix[ChannelMapMaxNumChannels * 8] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight
			1.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
			0.0f,			0.0f,		1.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontCenter
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// LowFrequency
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		0.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		0.0f,		0.0f,		// SideRight
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		1.0f,		0.0f,		// BackLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		1.0f,		// BackRight
		};

#else //PLATFORM_MICROSOFT

		// NOTE: the BackLeft/BackRight and SideLeft/SideRight are reversed than they should be since our 7.1 importer code has it backward
		static constexpr float To7Point1Matrix[ChannelMapMaxNumChannels * 8] =
		{
			// FrontLeft	FrontRight	Center		LowFrequency	SideLeft	SideRight	BackLeft	BackRight
			1.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
			0.0f,			0.0f,		1.0f,		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontCenter
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// LowFrequency
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		1.0f,		0.0f,		// BackLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		0.0f,		1.0f,		// BackRight
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		0.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		0.0f,		0.0f,		// SideRight
		};

#endif//PLATFORM_MICROSOFT

		static constexpr float VorbisTo7Point1Matrix[ChannelMapVorbisNumChannels * 8] =
		{
			// FrontLeft	Center		FrontRight	SideLeft		SideRight	LowFrequency		
			1.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// FrontLeft
			0.0f,			0.0f,		1.0f,		0.0f,			0.0f,		0.0f,		// FrontRight
			0.0f,			1.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// FrontCenter
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		1.0f,		// LowFrequency
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// SideLeft
			0.0f,			0.0f,		0.0f,		0.0f,			0.0f,		0.0f,		// SideRight
			0.0f,			0.0f,		0.0f,		1.0f,			0.0f,		0.0f,		// BackLeft
			0.0f,			0.0f,		0.0f,		0.0f,			1.0f,		0.0f,		// BackRight
		};

		static float const * const OutputChannelMaps[ChannelMapMaxNumChannels] =
		{
			ToMonoMatrix,
			ToStereoMatrix,
			ToTriMatrix,	// Experimental
			ToQuadMatrix,
			To5Matrix,		// Experimental
			To5Point1Matrix,
			ToHexMatrix,	// Experimental
			To7Point1Matrix
		};

		// 5.1 Vorbis files have a different channel order than normal
		static float const * const VorbisChannelMaps[ChannelMapMaxNumChannels] =
		{
			VorbisToMonoMatrix,
			VorbisToStereoMatrix,
			VorbisToTriMatrix,	// Experimental
			VorbisToQuadMatrix,
			VorbisTo5Matrix,		// Experimental
			VorbisTo5Point1Matrix,
			VorbisToHexMatrix,	// Experimental
			VorbisTo7Point1Matrix
		};

		// Create a channel map for a generic multichannel input to any supported
		// multichannel output.
		//
		// @param InNumInputChannels - The number of channels in the input content.
		// @param InNumOutputChannels - The number of channels in the output audio.
		// @param OutChannelMap - An array of channel gains in OutputMajorOrder (see EChannelMapOrder for details on format).
		//
		// @return true on success, false on failure. 
		bool Create2DGenericInputChannelMap(int32 InNumInputChannels, int32 InNumOutputChannels, TArray<float>& OutChannelMap)
		{
			const int32 OutputChannelMapIndex = InNumOutputChannels - 1;
			check(OutputChannelMapIndex >= 0);
			check(OutputChannelMapIndex < ChannelMapMaxNumChannels);

			const float* RESTRICT Matrix = OutputChannelMaps[OutputChannelMapIndex];
			check(Matrix != nullptr);

			for (int32 InputChannel = 0; InputChannel < InNumInputChannels; ++InputChannel)
			{
				for (int32 OutputChannel = 0; OutputChannel < InNumOutputChannels; ++OutputChannel)
				{
					const int32 Index = OutputChannel * ChannelMapMaxNumChannels + InputChannel;
					OutChannelMap.Add(Matrix[Index]);
				}
			}

			return true;
		}

		// Create a channel map for a mono input to any supported multichannel output.
		//
		// @param InNumOutputChannels - The number of channels in the output audio.
		// @param InMonoChannelUpmixMethod - Method for upmixing mono to front-left and front-right (not used if bInIsCenterChannelOnly is true and output format has center channel, or if output is Mono or Quad).
		// @param bInIsCenterChannelOnly - If true, and if the output channel count corresponds to format containing a center channel, the mono input will be mapped directly to the center channel.
		// @param OutChannelMap - An array of channel gains in OutputMajorOrder (see EChannelMapOrder for details on format).
		//
		// @return true on success, false on failure. 
		bool Create2DMonoInputChannelMap(int32 InNumOutputChannels, EChannelMapMonoUpmixMethod InMonoChannelUpmixMethod, bool bInIsCenterChannelOnly, TArray<float>& OutChannelMap)
		{
			const int32 OutputChannelMapIndex = InNumOutputChannels - 1;
			check(OutputChannelMapIndex >= 0);
			check(OutputChannelMapIndex < ChannelMapMaxNumChannels);

			// Mono-in mono-out channel map
			if (InNumOutputChannels == 1)
			{
				OutChannelMap.Add(1.0f);
			}
			else
			{
				// If we have more than stereo output (means we have center channel, which is always the 3rd index)
				// Then we need to only apply 1.0 to the center channel, 0.0 for everything else
				if ((InNumOutputChannels == 3 || InNumOutputChannels > 4) && bInIsCenterChannelOnly)
				{
					for (int32 OutputChannel = 0; OutputChannel < InNumOutputChannels; ++OutputChannel)
					{
						// Center channel is always 3rd index
						if (OutputChannel == 2)
						{
							OutChannelMap.Add(1.0f);
						}
						else
						{
							OutChannelMap.Add(0.0f);
						}
					}
				}
				else
				{
					// Mapping out to more than 2 channels, mono inputs should be equally spread to left and right
					switch (InMonoChannelUpmixMethod)
					{
						case EChannelMapMonoUpmixMethod::Linear:
							OutChannelMap.Add(Audio::MonoUpmixLinear);
							OutChannelMap.Add(Audio::MonoUpmixLinear);
							break;

						case EChannelMapMonoUpmixMethod::EqualPower:
							OutChannelMap.Add(Audio::MonoUpmixEqualPower);
							OutChannelMap.Add(Audio::MonoUpmixEqualPower);
							break;

						case EChannelMapMonoUpmixMethod::FullVolume:
							OutChannelMap.Add(Audio::MonoUpmixFullVolume);
							OutChannelMap.Add(Audio::MonoUpmixFullVolume);
							break;
					}

					const float* RESTRICT Matrix = OutputChannelMaps[OutputChannelMapIndex];
					check(Matrix != nullptr);

					for (int32 OutputChannel = 2; OutputChannel < InNumOutputChannels; ++OutputChannel)
					{
						const int32 Index = OutputChannel * ChannelMapMaxNumChannels;
						OutChannelMap.Add(Matrix[Index]);
					}
				}
			}

			return true;
		}

		// Create a channel map for a quad input to any supported multichannel output.
		//
		// @param InNumOutputChannels - The number of channels in the output audio.
		// @param OutChannelMap - An array of channel gains in OutputMajorOrder (see EChannelMapOrder for details on format).
		//
		// @return true on success, false on failure. 
		bool Create2DQuadInputChannelMap(int32 InNumOutputChannels, TArray<float>& OutChannelMap)
		{
			const int32 OutputChannelMapIndex = InNumOutputChannels - 1;

			check(OutputChannelMapIndex >= 0);
			check(OutputChannelMapIndex < ChannelMapMaxNumChannels);

			const float* RESTRICT Matrix = OutputChannelMaps[OutputChannelMapIndex];
			check(Matrix != nullptr);

			// Quad has a special-case to map input channels 0 1 2 3 to 0 1 4 5: 
			const int32 FrontLeftChannel = 0;
			for (int32 OutputChannel = 0; OutputChannel < InNumOutputChannels; ++OutputChannel)
			{
				const int32 Index = OutputChannel * ChannelMapMaxNumChannels + FrontLeftChannel;
				OutChannelMap.Add(Matrix[Index]);
			}

			const int32 FrontRightChannel = 1;
			for (int32 OutputChannel = 0; OutputChannel < InNumOutputChannels; ++OutputChannel)
			{
				const int32 Index = OutputChannel * ChannelMapMaxNumChannels + FrontRightChannel;
				OutChannelMap.Add(Matrix[Index]);
			}

			const int32 SurroundLeftChannel = 4;
			for (int32 OutputChannel = 0; OutputChannel < InNumOutputChannels; ++OutputChannel)
			{
				const int32 Index = OutputChannel * ChannelMapMaxNumChannels + SurroundLeftChannel;
				OutChannelMap.Add(Matrix[Index]);
			}

			const int32 SurroundRightChannel = 5;
			for (int32 OutputChannel = 0; OutputChannel < InNumOutputChannels; ++OutputChannel)
			{
				const int32 Index = OutputChannel * ChannelMapMaxNumChannels + SurroundRightChannel;
				OutChannelMap.Add(Matrix[Index]);
			}

			return true;
		}

		// Returns true if channel counts are supported by the hardcoded channel maps.
		bool EnsureValidChannelCounts(int32 InNumInputChannels, int32 InNumOutputChannels)
		{
			const bool bIsValidNumOutputChannels = (InNumOutputChannels > 0) && (InNumOutputChannels <= ChannelMapMaxNumChannels);
			if (ensureMsgf(bIsValidNumOutputChannels, TEXT("Invalid number of output channels: %d"), InNumOutputChannels))
			{
				const bool bIsValidNumInputChannels = (InNumInputChannels > 0) && (InNumInputChannels <= ChannelMapMaxNumChannels);
				if (ensureMsgf(bIsValidNumInputChannels, TEXT("Invalid number of input channels: %d"), InNumInputChannels))
				{
					return true;
				}
			}

			return false;
		}

		// Transpose a matrix from OutputMajorOrder to InputMajorOrder. See EChannelMapOrder
		// for details on the formats. 
		void TransposeOutputMajorOrderToInputMajorOrder(int32 InNumInputChannels, int32 InNumOutputChannels, TArray<float>& OutChannelMap)
		{
			check(OutChannelMap.Num() == (InNumInputChannels * InNumOutputChannels));

			// Transpose matrix to get input major order. 
			TArray<float> CopyOfMap = OutChannelMap;
			for (int32 InputChannelIndex = 0; InputChannelIndex < InNumInputChannels; InputChannelIndex++)
			{
				int32 OriginalIndex = InputChannelIndex * InNumOutputChannels;
				int32 NewIndex = InputChannelIndex;
				for (int32 OutputChannelIndex = 0; OutputChannelIndex < InNumOutputChannels; OutputChannelIndex++)
				{
					OutChannelMap[NewIndex] = CopyOfMap[OriginalIndex];
					OriginalIndex++;
					NewIndex += InNumInputChannels;
				}
			}
		}
	}


	bool Create2DChannelMap(const FChannelMapParams& InParams, TArray<float>& OutChannelMap)
	{
		using namespace ChannelMapPrivate;

		if (!EnsureValidChannelCounts(InParams.NumInputChannels, InParams.NumOutputChannels))
		{
			return false;
		}

		bool bSuccess = false;

		// Mono and quad have special considerations for upmixing.
		if (InParams.NumInputChannels == 1)
		{
			bSuccess = Create2DMonoInputChannelMap(InParams.NumOutputChannels, InParams.MonoUpmixMethod, InParams.bIsCenterChannelOnly, OutChannelMap);
		}
		else if (InParams.NumInputChannels == 4)
		{
			bSuccess = Create2DQuadInputChannelMap(InParams.NumOutputChannels, OutChannelMap);
		}
		else
		{
			bSuccess = Create2DGenericInputChannelMap(InParams.NumInputChannels, InParams.NumOutputChannels, OutChannelMap);
		}

		// By default, functions create channel map in "OutputMajorOrder". Check
		// if it needs to be transposed. 
		if (bSuccess && (InParams.Order == EChannelMapOrder::InputMajorOrder))
		{
			TransposeOutputMajorOrderToInputMajorOrder(InParams.NumInputChannels, InParams.NumOutputChannels, OutChannelMap);
		}

		return bSuccess;
	}

	bool CreateVorbis2DChannelMap(int32 InNumOutputChannels, EChannelMapOrder InOrder, TArray<float>& OutVorbisChannelMap)
	{
		using namespace ChannelMapPrivate;

		if (!ensureMsgf((InNumOutputChannels > 0) && (InNumOutputChannels <= ChannelMapMaxNumChannels), TEXT("Invalid number of output channels: %d"), InNumOutputChannels))
		{
			return false;
		}

		// Get the matrix for the channel map index
		const int32 OutputChannelMapIndex = InNumOutputChannels - 1;
		const float* VorbisMatrix = VorbisChannelMaps[OutputChannelMapIndex];
		check(nullptr != VorbisMatrix);

		// Build it by looping over the vorbis ordered 5.1 input channels
		for (int32 InputChannel = 0; InputChannel < ChannelMapVorbisNumChannels; ++InputChannel)
		{
			for (int32 OutputChannel = 0; OutputChannel < InNumOutputChannels; ++OutputChannel)
			{
				const int32 Index = OutputChannel * ChannelMapVorbisNumChannels + InputChannel;
				OutVorbisChannelMap.Add(VorbisMatrix[Index]);
			}
		}

		// By default, channel map is in "OutputMajorOrder". Check if it needs to be transposed. 
		if (InOrder == EChannelMapOrder::InputMajorOrder)
		{
			TransposeOutputMajorOrderToInputMajorOrder(ChannelMapVorbisNumChannels, InNumOutputChannels, OutVorbisChannelMap);
		}

		return true;
	}
}
