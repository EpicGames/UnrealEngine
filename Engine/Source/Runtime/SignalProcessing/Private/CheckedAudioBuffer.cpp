// Copyright Epic Games, Inc. All Rights Reserved.
#include "CheckedAudioBuffer.h"
#include "CoreGlobals.h"

// This defaults to 'on' unless it's defined ahead.
#ifndef WITH_AUDIO_BUFFER_CHECKS
	#define WITH_AUDIO_BUFFER_CHECKS (1)
#endif //WITH_AUDIO_BUFFER_CHECKS

namespace Audio
{
	void FCheckedAudioBuffer::operator=(const FAlignedFloatBuffer& InOther)
	{
		DoCheck(Buffer);
		DoCheck(InOther);
		Buffer = InOther;
	}
	
	float* FCheckedAudioBuffer::GetData()
	{
		DoCheck(Buffer);
		return Buffer.GetData();
	}

	int32 FCheckedAudioBuffer::Num() const
	{
		DoCheck(Buffer);
		return Buffer.Num();
	}

	void FCheckedAudioBuffer::Reserve(const int32 InSize)
	{
		Buffer.Reserve(InSize);
	}

	void FCheckedAudioBuffer::Reset(const int32 InSize/*=0*/)
	{
		Buffer.Reset(InSize);
		bFailedChecks = false; // Might as well reset here.
		FailedFlags = ECheckBufferFlags::None; 
	}

	void FCheckedAudioBuffer::AddZeroed(const int32 InSize)
	{
		DoCheck(Buffer);
		Buffer.AddZeroed(InSize);
	}

	void FCheckedAudioBuffer::SetNumZeroed(const int32 InSize)
	{
		DoCheck(Buffer);
		Buffer.SetNumZeroed(InSize);
	}

	void FCheckedAudioBuffer::SetNumUninitialized(const int32 InNum)
	{
		return Buffer.SetNumUninitialized(InNum);
	}

	Audio::FAlignedFloatBuffer& FCheckedAudioBuffer::GetBuffer()
	{
		DoCheck(Buffer);
		return Buffer;
	}

	const Audio::FAlignedFloatBuffer& FCheckedAudioBuffer::GetBuffer() const
	{
		DoCheck(Buffer);
		return Buffer;
	}

	void FCheckedAudioBuffer::Append(const FCheckedAudioBuffer& InBuffer)
	{
		DoCheck(InBuffer );
		DoCheck(Buffer);
		Buffer.Append(InBuffer.GetBuffer());
	}

	void FCheckedAudioBuffer::Append(TArrayView<const float> InView)
	{
		DoCheck(InView);
		DoCheck(Buffer);
		Buffer.Append(InView);
	}

	void FCheckedAudioBuffer::Append(const FAlignedFloatBuffer& InBuffer)
	{
		DoCheck(InBuffer);
		DoCheck(Buffer);
		Buffer.Append(InBuffer);
	}
	
	void FCheckedAudioBuffer::DoCheck(TArrayView<const float> InBuffer) const
	{
#if WITH_AUDIO_BUFFER_CHECKS
		// Only do check until we fail.
		if (!bFailedChecks && CheckFlags != ECheckBufferFlags::None)
		{
			bFailedChecks = CheckBuffer(InBuffer, CheckFlags, FailedFlags);
			if (bFailedChecks)
			{
				switch (Behavior)
				{
				// Do nothing.
				case ECheckBehavior::Ensure:
					{
						ensureMsgf(false, TEXT("Buffer Check Ensure. CheckFlags='%s', Failed='%s', Buffer=%s"), 
							*ToDelimitedString(CheckFlags), *ToDelimitedString(FailedFlags), *DescriptiveName);
						break;
					}
				case ECheckBehavior::Log:
					{
						UE_LOGF(LogInit, Warning, "Buffer Check Failed. CheckFlags='%ls', Failed='%ls', Buffer=%ls", 
							*ToDelimitedString(CheckFlags), *ToDelimitedString(FailedFlags), *DescriptiveName);
						break;
					}
				case ECheckBehavior::Break:
					{
						UE_LOGF(LogInit, Warning, "Buffer Check Failed. CheckFlags='%ls', Failed='%ls', Buffer=%ls",
							*ToDelimitedString(CheckFlags), *ToDelimitedString(FailedFlags), *DescriptiveName);

						UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
						break;
					}
				case ECheckBehavior::Nothing:
				default:
					{
						break;
					}
				}
			}
		}
#endif //WITH_AUDIO_BUFFER_CHECKS
	}
}
