// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Chorus.h"

namespace Audio
{
	FChorus::FChorus()
		: MinDelayMsec(5.0f)
		, MaxDelayMsec(50.0f)
		, DelayRangeMsec(MaxDelayMsec - MinDelayMsec)
		, Spread(0.0f)
		, MaxFrequencySpread(10.0f)
		, WetLevel(0.5f)
		, DryLevel(0.5f)
		, NumChannels(0)
	{
		FMemory::Memzero(Feedback, sizeof(float)*EChorusDelays::NumDelayTypes);
	}

	FChorus::~FChorus()
	{
	}

	void FChorus::Init(const float InSampleRate, const int32 InNumChannels, const float InBufferLengthSec, const int32 InControlSamplePeriod)
	{
		NumChannels = InNumChannels;

		for (int32 i = 0; i < EChorusDelays::NumDelayTypes; ++i)
		{
			Delays[i].Init(InSampleRate, InBufferLengthSec);
			Depth[i].Init(InSampleRate);
			Depth[i].SetValue(0.5f);

			LFOs[i].Init(InSampleRate);
			LFOs[i].SetType(ELFO::Triangle);
			LFOs[i].Update();
			LFOs[i].Start();
		}
	}

	void FChorus::SetDepth(const EChorusDelays::Type InType, const float InDepth)
	{
		Depth[InType].SetValue(FMath::Clamp(InDepth, 0.0f, 1.0f), 20.0f);
	}

	void FChorus::SetFrequency(const EChorusDelays::Type InType, const float InFrequency)
	{
		LFOs[InType].SetFrequency(InFrequency);
		LFOs[InType].Update();
	}

	void FChorus::SetFeedback(const EChorusDelays::Type InType, const float InFeedback)
	{
		Feedback[InType] = FMath::Clamp(InFeedback, 0.0f, 1.0f);
	}

	void FChorus::SetWetLevel(const float InWetLevel)
	{
		WetLevel = InWetLevel;
	}

	void FChorus::SetDryLevel(const float InDryLevel)
	{
		DryLevel = InDryLevel;
	}

	void FChorus::SetSpread(const float InSpread)
	{
		Spread = FMath::Clamp(InSpread, 0.0f, 1.0f);

		LFOs[EChorusDelays::Left].SetFrequencyMod(-Spread * MaxFrequencySpread);
		LFOs[EChorusDelays::Right].SetFrequencyMod(Spread * MaxFrequencySpread);

		LFOs[EChorusDelays::Left].Update();
		LFOs[EChorusDelays::Right].Update();
	}

	void FChorus::ProcessAudioFrame(const float* InFrame, float* OutFrame)
	{
		float LFONormalPhaseOut = 0.0f;
		float LFOQuadPhaseOut = 0.0f;
		float LFOQuadPhaseOut2 = 0.0f;

		for (int32 i = 0; i < EChorusDelays::NumDelayTypes; ++i)
		{
			LFONormalPhaseOut = LFOs[i].Generate(&LFOQuadPhaseOut);
			LFONormalPhaseOut = Audio::GetUnipolar(LFONormalPhaseOut);

			LFOQuadPhaseOut2 = Audio::GetUnipolar(-LFOQuadPhaseOut);
			LFOQuadPhaseOut = Audio::GetUnipolar(LFOQuadPhaseOut);

			float DepthValue = Depth[i].GetNextValue();

			if (i == EChorusDelays::Left)
			{
				float Phase = LFOQuadPhaseOut - Spread;
				const float NewDelay = LFOQuadPhaseOut * DepthValue * DelayRangeMsec + MinDelayMsec;
				Delays[i].SetDelayMsec(NewDelay);
			}
			else if (i == EChorusDelays::Center)
			{
				const float NewDelay = LFONormalPhaseOut * DepthValue * DelayRangeMsec + MinDelayMsec;
				Delays[i].SetDelayMsec(NewDelay);
			}
			else
			{
				const float NewDelay = LFOQuadPhaseOut2 * DepthValue * DelayRangeMsec + MinDelayMsec;
				Delays[i].SetDelayMsec(NewDelay);
			}
		}

		float DelayInputs[EChorusDelays::NumDelayTypes];

		if (NumChannels == 2)
		{
			DelayInputs[EChorusDelays::Left] = InFrame[0];
			DelayInputs[EChorusDelays::Center] = 0.5f * InFrame[0] + 0.5f * InFrame[1];
			DelayInputs[EChorusDelays::Right] = InFrame[1];
		}
		else
		{
			DelayInputs[EChorusDelays::Left] = InFrame[0];
			DelayInputs[EChorusDelays::Center] = InFrame[0];
			DelayInputs[EChorusDelays::Right] = InFrame[0];
		}

		float DelayOutputs[EChorusDelays::NumDelayTypes];

		for (int32 i = 0; i < EChorusDelays::NumDelayTypes; ++i)
		{
			DelayOutputs[i] = Delays[i].Read();

			Delays[i].WriteDelayAndInc(DelayInputs[i] + DelayOutputs[i] * Feedback[i]);
		}

		if (NumChannels == 2)
		{
			OutFrame[0] = InFrame[0] * DryLevel + WetLevel * (DelayOutputs[EChorusDelays::Left] + 0.5f * DelayOutputs[EChorusDelays::Center]);
			OutFrame[1] = InFrame[1] * DryLevel + WetLevel * (DelayOutputs[EChorusDelays::Right] + 0.5f * DelayOutputs[EChorusDelays::Center]);
		}
		else
		{
			OutFrame[0] = InFrame[0] * DryLevel + WetLevel * (DelayOutputs[EChorusDelays::Left] + 0.5f * DelayOutputs[EChorusDelays::Center]);
			OutFrame[0] += InFrame[0] * DryLevel + WetLevel * (DelayOutputs[EChorusDelays::Right] + 0.5f * DelayOutputs[EChorusDelays::Center]);
			OutFrame[0] *= 0.5f;
		}
	}

	void FChorus::ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer)
	{
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
		{
			ProcessAudioFrame(&InBuffer[SampleIndex], &OutBuffer[SampleIndex]);
		}
	}

}
