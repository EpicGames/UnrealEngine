// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/ConstantQ.h"
#include "CoreMinimal.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	namespace PseudoConstantQIntrinsics
	{
		void FillArrayWithTruncatedGaussian(const float InCenterFreq, const float InBandWidth, const int32 InFFTSize, const float InSampleRate, FAlignedFloatBuffer& OutOffsetArray, int32& OutOffsetIndex)
		{
			check(InBandWidth > 0.f);
			check(InFFTSize > 0);
			check(InCenterFreq >= 0.f);
			check(InSampleRate > 0.f);

			
			// Determine points where gaussian will value will go below a small number
			const float SignificantHalfBandWidth = InBandWidth * FMath::Sqrt(-2.f * FMath::Loge(SMALL_NUMBER));
			const float LowestSignificantFreq = FMath::Clamp(InCenterFreq - SignificantHalfBandWidth, 0.f, InSampleRate / 2.f);
			const float HighestSignificantFreq = FMath::Clamp(InCenterFreq + SignificantHalfBandWidth, 0.f, InSampleRate / 2.f);
			int32 LowestSignificantIndex = FMath::CeilToInt(InFFTSize * LowestSignificantFreq / InSampleRate);
			int32 HighestSignificantIndex = FMath::FloorToInt(InFFTSize * HighestSignificantFreq / InSampleRate);
			int32 Num = HighestSignificantIndex - LowestSignificantIndex + 1;

			if (Num < 1)
			{
				Num = 1;
			}

			// Prepare outputs
			OutOffsetIndex = LowestSignificantIndex;
			OutOffsetArray.Reset(Num);
			OutOffsetArray.AddUninitialized(Num);

			// Fill array with truncated Gaussian
			const float BandWidthSquared = InBandWidth * InBandWidth;
			for (int32 i = 0; i < Num; i++)
			{
				float FFTBinHz = static_cast<float>(i + LowestSignificantIndex) * InSampleRate / InFFTSize;
				float DeltaHz = FFTBinHz - InCenterFreq;
				OutOffsetArray[i] = FMath::Exp(-0.5 * (DeltaHz * DeltaHz) / BandWidthSquared);
			}

			// set infs and nans to zero
			for (int32 i = 0; i < OutOffsetArray.Num(); i++)
			{
				if (!FMath::IsFinite(OutOffsetArray[i]))
				{
					OutOffsetArray[i] = 0.f;
				}
			}
		}
	}


	float FPseudoConstantQ::GetConstantQCenterFrequency(const int32 InBandIndex, const float InBaseFrequency, const float InBandsPerOctave)
	{
		check(InBandsPerOctave > 0.f);
		return InBaseFrequency * FMath::Pow(2.f, static_cast<float>(InBandIndex) / InBandsPerOctave);
	}

	float FPseudoConstantQ::GetConstantQBandWidth(const float InBandCenter, const float InBandsPerOctave, const float InBandWidthStretch)
	{
		check(InBandsPerOctave > 0.f);
		return InBandWidthStretch * InBandCenter * (FMath::Pow(2.f, 1.f / InBandsPerOctave) - 1.f);
	}


	void FPseudoConstantQ::FillArrayWithConstantQBand(const FPseudoConstantQBandSettings& InSettings, FAlignedFloatBuffer& OutOffsetArray, int32& OutOffsetIndex)
	{
		check(InSettings.SampleRate > 0.f);
		check(InSettings.FFTSize > 0);

		const int32 NumUsefulFFTBins = (InSettings.FFTSize / 2) + 1;

		// Create gaussian centered around center freq and with appropriate bandwidth
		OutOffsetIndex = 0;
		float BandWidth = FMath::Max(SMALL_NUMBER, InSettings.BandWidth);
		PseudoConstantQIntrinsics::FillArrayWithTruncatedGaussian(InSettings.CenterFreq, BandWidth, InSettings.FFTSize, InSettings.SampleRate, OutOffsetArray, OutOffsetIndex);

		// Need to have a lower bound for 
		float DigitalBandWidth = FMath::Max(1.f / static_cast<float>(InSettings.FFTSize), BandWidth / InSettings.SampleRate);

		// Sanity check the CQT bins to make sure we the bandwidth wasn't so small that the array is essentially empty
		if (OutOffsetArray.Num() > 0)
		{
			float GaussianSum = 0.f;
			Audio::ArraySum(OutOffsetArray, GaussianSum);

			if (GaussianSum < 0.95f)
			{
				// It's a bit of an arbitrary threshold, but it tells us that the bandwidth
				// is low enough and the FFT granularity course enough to where our pseudo 
				// cqt windows will likely miss data. In this case we force the window to 
				// interpolate between two nearest fft bins.
				FMemory::Memset(OutOffsetArray.GetData(), 0, OutOffsetArray.Num() * sizeof(float));
				float Position = (InSettings.FFTSize * InSettings.CenterFreq / InSettings.SampleRate) - static_cast<float>(OutOffsetIndex);
				int32 LowerIndex = FMath::FloorToInt(Position);
				int32 UpperIndex = LowerIndex + 1;

				if ((LowerIndex >= 0) && (UpperIndex < OutOffsetArray.Num()))
				{
					OutOffsetArray[LowerIndex] = 1.f - (Position - static_cast<float>(LowerIndex));
					OutOffsetArray[UpperIndex] = 1.f - OutOffsetArray[LowerIndex];

					// Need to update digital bandwidth to be bandwidth of FFTbin
					DigitalBandWidth = 1.f / static_cast<float>(InSettings.FFTSize);
				}
				else if ((LowerIndex >= 0) && (LowerIndex < OutOffsetArray.Num()))
				{
					OutOffsetArray[LowerIndex] = 1.f;
					DigitalBandWidth = 1.f / static_cast<float>(InSettings.FFTSize);
				}
				else if ((UpperIndex >= 0) && (UpperIndex < OutOffsetArray.Num()))
				{
					OutOffsetArray[UpperIndex] = 1.f;
					DigitalBandWidth = 1.f / static_cast<float>(InSettings.FFTSize);
				}
			}
		}

		// Normalize window
		float NormDenom = 1.f;
		switch (InSettings.Normalization)
		{
			case EPseudoConstantQNormalization::EqualAmplitude:
				NormDenom = 1.f;
				break;

			case EPseudoConstantQNormalization::EqualEuclideanNorm:
				NormDenom = FMath::Sqrt(DigitalBandWidth * InSettings.FFTSize * FMath::Sqrt(PI  * 2.f));
				break;

			case EPseudoConstantQNormalization::EqualEnergy:
			default:
				NormDenom = DigitalBandWidth * InSettings.FFTSize * FMath::Sqrt(PI  * 2.f);
				break;
		}

		if ((NormDenom > 0.f) && (NormDenom != 1.f))
		{
			ArrayMultiplyByConstantInPlace(OutOffsetArray, 1.f / NormDenom);
		}

		// Truncate if necessary
		if (OutOffsetIndex >= NumUsefulFFTBins)
		{
			OutOffsetIndex = 0;
			OutOffsetArray.Empty();
		}
		else if ((OutOffsetIndex + OutOffsetArray.Num()) > NumUsefulFFTBins)
		{
			OutOffsetArray.SetNum(NumUsefulFFTBins - OutOffsetIndex);
		}
	}

	TUniquePtr<FContiguousSparse2DKernelTransform> NewPseudoConstantQKernelTransform(const FPseudoConstantQKernelSettings& InSettings, const int32 InFFTSize, const float InSampleRate)
	{
		check(InSampleRate > 0.f);
		check(InFFTSize > 0);

		const int32 NumUsefulFFTBins = (InFFTSize / 2) + 1;

		TUniquePtr<FContiguousSparse2DKernelTransform> Transform = MakeUnique<FContiguousSparse2DKernelTransform>(NumUsefulFFTBins, InSettings.NumBands);

		FPseudoConstantQBandSettings BandSettings;
		BandSettings.FFTSize = InFFTSize;
		BandSettings.SampleRate = InSampleRate;
		BandSettings.Normalization = InSettings.Normalization;

		for (int32 CQTBandIndex = 0; CQTBandIndex < InSettings.NumBands; CQTBandIndex++)
		{
			// Determine band center and width for this CQT band
			BandSettings.CenterFreq = FPseudoConstantQ::GetConstantQCenterFrequency(CQTBandIndex, InSettings.KernelLowestCenterFreq, InSettings.NumBandsPerOctave);
			BandSettings.BandWidth = FPseudoConstantQ::GetConstantQBandWidth(BandSettings.CenterFreq, InSettings.NumBandsPerOctave, InSettings.BandWidthStretch);

			if ((BandSettings.CenterFreq - BandSettings.BandWidth) > InSampleRate)
			{
				continue;
			}

			if (BandSettings.CenterFreq > (2.f * InSampleRate))
			{
				continue;
			}

			// Create gaussian centered around center freq and with appropriate bandwidth
			FAlignedFloatBuffer OffsetBandWeights;
			int32 OffsetBandWeightsIndex = 0;

			FPseudoConstantQ::FillArrayWithConstantQBand(BandSettings, OffsetBandWeights, OffsetBandWeightsIndex);

			// Store row in transform
			Transform->SetRow(CQTBandIndex, OffsetBandWeightsIndex, OffsetBandWeights);
		}

		return Transform;
	}
}
