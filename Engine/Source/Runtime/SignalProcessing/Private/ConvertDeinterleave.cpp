// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/ConvertDeinterleave.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "DSP/ChannelMap.h"
#include "DSP/MultichannelBuffer.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	namespace ConvertDeinterleavePrivate
	{
		// Template class for generic deinterleave/convert operation.  It uses
		// mixing gains provided by `Audio::Create2DChannelMap` which are AC3 
		// downmixing values. 
		//
		// Using a template class with the channel counts as template arguments
		// allows the compiler to use  hard coded channel counts when compiling 
		// the sample loop and/or for optimizations to be hand introduced using 
		// class template specialization.
		template<int32 NumInputChannels, int32 NumOutputChannels>
		struct TConvertDeinterleave : IConvertDeinterleave
		{
			static_assert(NumInputChannels > 0);
			static_assert(NumOutputChannels > 0);

			TConvertDeinterleave(const FConvertDeinterleaveParams& InParams)
			{
				using namespace Audio;

				check(InParams.NumInputChannels == NumInputChannels);
				check(InParams.NumOutputChannels == NumOutputChannels);

				FChannelMapParams ChannelMapParams;
				ChannelMapParams.NumInputChannels = NumInputChannels;
				ChannelMapParams.NumOutputChannels = NumOutputChannels;
				ChannelMapParams.Order = EChannelMapOrder::InputMajorOrder;
				ChannelMapParams.MonoUpmixMethod = InParams.MonoUpmixMethod;
				ChannelMapParams.bIsCenterChannelOnly = false;

				bool bSuccess = Create2DChannelMap(ChannelMapParams, ChannelGains);
				check(bSuccess);
				checkf(ChannelGains.Num() == (NumInputChannels * NumOutputChannels), TEXT("Expected channel array of %d elements. Actual num elements: %d"), (NumInputChannels * NumOutputChannels), ChannelGains.Num());
			}

			virtual ~TConvertDeinterleave() = default;

			const float* GetChannelGains(int32 InOutputChannelIndex) const
			{
				return &ChannelGains.GetData()[InOutputChannelIndex * NumInputChannels];
			}

			virtual void ProcessAudio(TArrayView<const float> InSamples, const TArray<TArrayView<float>>& OutMultiChannelAudio) const override
			{
				checkf((InSamples.Num() % NumInputChannels) == 0, TEXT("Input sample buffer contains partial audio frame."));
				checkf((InSamples.Num() / NumInputChannels) == GetMultichannelBufferNumFrames(OutMultiChannelAudio), TEXT("Audio buffer frame count mismatch."));
				checkf(NumOutputChannels == OutMultiChannelAudio.Num(), TEXT("Output audio buffer not initialized to expected channel count."));

				const int32 NumFrames = GetMultichannelBufferNumFrames(OutMultiChannelAudio);
				const float* InSampleData = InSamples.GetData();

				float* OutputChannelDataPtrs[NumOutputChannels];
				for (int32 OutChannelIndex = 0; OutChannelIndex < NumOutputChannels; OutChannelIndex++)
				{
					OutputChannelDataPtrs[OutChannelIndex] = OutMultiChannelAudio[OutChannelIndex].GetData();
				}

				for (int32 Frame = 0; Frame < NumFrames; Frame++)
				{
					const float* InSampleFrameStart = &InSampleData[Frame * NumInputChannels];
					for (int32 OutChannelIndex = 0; OutChannelIndex < NumOutputChannels; OutChannelIndex++)
					{
						float Value = 0.f;
						const float* Gains = GetChannelGains(OutChannelIndex);

						// Apply gains to interleaved input samples for a single output
						// sample.
						for (int32 InChannelIndex = 0; InChannelIndex < NumInputChannels; InChannelIndex++)
						{
							Value += InSampleFrameStart[InChannelIndex] * Gains[InChannelIndex];
						}

						*OutputChannelDataPtrs[OutChannelIndex] = Value;
						OutputChannelDataPtrs[OutChannelIndex]++;
					}
				}
			}
			
		private:
			TArray<float> ChannelGains;
		};

		// Specialization of TConvertDeinterleave for mono sources. This avoids
		// the need for deinterleaving.
		template<int32 NumOutputChannels>
		struct TConvertDeinterleave<1, NumOutputChannels> : IConvertDeinterleave
		{
			static_assert(NumOutputChannels > 0);

			TConvertDeinterleave(const FConvertDeinterleaveParams& InParams)
			{
				using namespace Audio;

				check(InParams.NumInputChannels == 1);
				check(InParams.NumOutputChannels == NumOutputChannels);

				FChannelMapParams ChannelMapParams;
				ChannelMapParams.NumInputChannels = 1;
				ChannelMapParams.NumOutputChannels = NumOutputChannels;
				ChannelMapParams.Order = EChannelMapOrder::InputMajorOrder;
				ChannelMapParams.MonoUpmixMethod = InParams.MonoUpmixMethod;
				ChannelMapParams.bIsCenterChannelOnly = false;

				bool bSuccess = Create2DChannelMap(ChannelMapParams, ChannelGains);
				check(bSuccess);
				checkf(ChannelGains.Num() == NumOutputChannels, TEXT("Expected channel array of %d elements. Actual num elements: %d"), NumOutputChannels, ChannelGains.Num());
			}

			virtual ~TConvertDeinterleave() = default;

			virtual void ProcessAudio(TArrayView<const float> InSamples, const TArray<TArrayView<float>>& OutMultiChannelAudio) const override
			{
				checkf(InSamples.Num() == GetMultichannelBufferNumFrames(OutMultiChannelAudio), TEXT("Audio buffer frame count mismatch."));
				checkf(NumOutputChannels == OutMultiChannelAudio.Num(), TEXT("Output audio buffer not initialized to expected channel count."));

				const int32 NumFrames = GetMultichannelBufferNumFrames(OutMultiChannelAudio);

				for (int32 OutChannelIndex = 0; OutChannelIndex < NumOutputChannels; OutChannelIndex++)
				{
					const float Gain = ChannelGains[OutChannelIndex];
					if (Gain == 1.f)
					{
						// Simply copy the audio to the output channels when gain is 1
						FMemory::Memcpy(OutMultiChannelAudio[OutChannelIndex].GetData(), InSamples.GetData(), NumFrames * sizeof(float));
					}
					else if (Gain != 0.f)
					{
						// Only need to multiply if gain is non-zero
						Audio::ArrayMultiplyByConstant(InSamples, ChannelGains[OutChannelIndex], OutMultiChannelAudio[OutChannelIndex]);
					}
					else
					{
						// If gain is zero, we can set the output channel to zero.
						FMemory::Memset(OutMultiChannelAudio[OutChannelIndex].GetData(), 0, NumFrames * sizeof(float));
					}
				}
			}

		private:
			TArray<float> ChannelGains;
		};

		// Template specialization for converting mono to mono. No need to adjust
		// gains or deinterleave.
		template<>
		struct TConvertDeinterleave<1, 1> : IConvertDeinterleave
		{
			TConvertDeinterleave(const FConvertDeinterleaveParams& InParams)
			{
				check(InParams.NumInputChannels == 1);
				check(InParams.NumOutputChannels == 1);
			}

			virtual ~TConvertDeinterleave() = default;
			virtual void ProcessAudio(TArrayView<const float> InSamples, const TArray<TArrayView<float>>& OutMultiChannelAudio) const override
			{
				checkf(InSamples.Num() == GetMultichannelBufferNumFrames(OutMultiChannelAudio), TEXT("Audio buffer frame count mismatch."));
				checkf(1 == OutMultiChannelAudio.Num(), TEXT("Output audio buffer not initialized to expected channel count."));

				const int32 NumFrames = GetMultichannelBufferNumFrames(OutMultiChannelAudio);
				const float* InSampleData = InSamples.GetData();
				FMemory::Memcpy(OutMultiChannelAudio[0].GetData(), InSampleData, NumFrames * sizeof(float));
			}
		};

		template<int32 NumInputChannels>
		TUniquePtr<IConvertDeinterleave> CreateConvertDeinterleave(const FConvertDeinterleaveParams& InParams)
		{
			// IConvertDeinterleave defines conversion operations channel counts 
			// between 1 and 8. This range mirrors the supported channel map channel
			// counts. If the supported set of channel maps is altered, the supported
			// set of defined TConvertDeinterleave classes should also be updated.
			static_assert(ChannelMapMaxNumChannels == 8);

			// Find the appropriate instantiation given the number of input and output channels.
			switch (InParams.NumOutputChannels)
			{
				case 1:
					return MakeUnique<TConvertDeinterleave<NumInputChannels, 1>>(InParams);
					break;
				case 2:
					return MakeUnique<TConvertDeinterleave<NumInputChannels, 2>>(InParams);
					break;
				case 3:
					return MakeUnique<TConvertDeinterleave<NumInputChannels, 3>>(InParams);
					break;
				case 4:
					return MakeUnique<TConvertDeinterleave<NumInputChannels, 4>>(InParams);
					break;
				case 5:
					return MakeUnique<TConvertDeinterleave<NumInputChannels, 5>>(InParams);
					break;
				case 6:
					return MakeUnique<TConvertDeinterleave<NumInputChannels, 6>>(InParams);
					break;
				case 7:
					return MakeUnique<TConvertDeinterleave<NumInputChannels, 7>>(InParams);
					break;
				case 8:
					return MakeUnique<TConvertDeinterleave<NumInputChannels, 8>>(InParams);
					break;
				default:
				{
					checkf(false, TEXT("Unsupported output channel count: %d"), InParams.NumOutputChannels);
					return TUniquePtr<IConvertDeinterleave>(nullptr);
				}
			}
		}
	}

	
	TUniquePtr<IConvertDeinterleave> IConvertDeinterleave::Create(const FConvertDeinterleaveParams& InParams)
	{
		using namespace ConvertDeinterleavePrivate;

		// IConvertDeinterleave defines conversion operations channel counts 
		// between 1 and 8. This range mirrors the supported channel map channel
		// counts. If the supported set of channel maps is altered, the supported
		// set of defined TConvertDeinterleave classes should also be updated.
		static_assert(ChannelMapMaxNumChannels == 8);

		// Find the appropriate instantiation given the number of input and output channels.
		switch (InParams.NumInputChannels)
		{
			case 1:
				return CreateConvertDeinterleave<1>(InParams);

			case 2:
				return CreateConvertDeinterleave<2>(InParams);

			case 3:
				return CreateConvertDeinterleave<3>(InParams);

			case 4:
				return CreateConvertDeinterleave<4>(InParams);

			case 5:
				return CreateConvertDeinterleave<5>(InParams);

			case 6:
				return CreateConvertDeinterleave<6>(InParams);

			case 7:
				return CreateConvertDeinterleave<7>(InParams);

			case 8:
				return CreateConvertDeinterleave<8>(InParams);

			default:
			{
				checkf(false, TEXT("Unsupported input channel count: %d"), InParams.NumInputChannels);
				return TUniquePtr<IConvertDeinterleave>(nullptr);
			}
		}
	}
}
