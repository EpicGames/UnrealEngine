// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/ConvolutionAlgorithm.h"
#include "SignalProcessingModule.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

namespace Audio
{
	IConvolutionAlgorithmFactory::~IConvolutionAlgorithmFactory()
	{}

	const FName IConvolutionAlgorithmFactory::GetModularFeatureName()
	{
		static const FName ModularFeatureName = FName("AudioConvolutionAlgorithmFactory");
		return ModularFeatureName;
	}

	const FName FConvolutionFactory::AnyAlgorithmFactory = FName("AnyAlgorithmFactory");

	TUniquePtr<IConvolutionAlgorithm> FConvolutionFactory::NewConvolutionAlgorithm(const FConvolutionSettings& InSettings, const FName& InAlgorithmFactoryName)
	{
		// Get all IConvolutionAlgorithm factories. 
		IModularFeatures::Get().LockModularFeatureList();
		TArray<IConvolutionAlgorithmFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<IConvolutionAlgorithmFactory>(IConvolutionAlgorithmFactory::GetModularFeatureName());
		IModularFeatures::Get().UnlockModularFeatureList();

		// Remove null factories
		Factories = Factories.FilterByPredicate([InAlgorithmFactoryName](const IConvolutionAlgorithmFactory* Factory) 
		{
			return (nullptr != Factory);
		});

		if (0 == Factories.Num())
		{
			UE_LOGF(LogSignalProcessing, Warning, "No registered IConvolutionAlgorithmFactories exist.");
			return TUniquePtr<IConvolutionAlgorithm>();
		}

		if (AnyAlgorithmFactory != InAlgorithmFactoryName)
		{
			// Filter factories if one has been specified. 
			Factories = Factories.FilterByPredicate([InAlgorithmFactoryName](const IConvolutionAlgorithmFactory* Factory) 
			{
				return (InAlgorithmFactoryName == Factory->GetFactoryName());
			});

			if (0 == Factories.Num())
			{
				UE_LOGF(LogSignalProcessing, Warning, "No IConvolutionAlgorithmFactories named \"%ls\" found.", *InAlgorithmFactoryName.ToString());
				return TUniquePtr<IConvolutionAlgorithm>();
			}
		}

		if (!InSettings.bEnableHardwareAcceleration)
		{
			// Remove factories with hardware acceleration if it has been disabled. 
			Factories = Factories.FilterByPredicate([InAlgorithmFactoryName](const IConvolutionAlgorithmFactory* Factory) 
			{
				return !Factory->IsHardwareAccelerated();
			});
		}

		// Sort factories by preference. 
		Factories.Sort([](const IConvolutionAlgorithmFactory& InFactoryA, const IConvolutionAlgorithmFactory& InFactoryB) 
		{
			// If InFactoryA has hardware acceleration and InFactoryB does not, then InFactoryA goes earlier.
			if (InFactoryA.IsHardwareAccelerated() && !InFactoryB.IsHardwareAccelerated())
			{
				return true;
			}

			// InFactoryA does not need to be placed before InFactoryB
			return false;
		});


		// Find first algorithm that can handle input settings
		for (IConvolutionAlgorithmFactory* Factory : Factories)
		{
			if (Factory->AreConvolutionSettingsSupported(InSettings))
			{
				TUniquePtr<IConvolutionAlgorithm> ConvolutionAlgorithm = Factory->NewConvolutionAlgorithm(InSettings);

				if (!ConvolutionAlgorithm.IsValid())
				{
					UE_LOGF(LogSignalProcessing, Warning, "IConvolutionAlgorithmFactory failed to create IConvolutionAlgorithm despite supporting ConvolutionReverb Settings.");
				}
				else
				{
					return ConvolutionAlgorithm;
				}
			}
		}

		UE_LOGF(LogSignalProcessing, Warning, "No IConvolutionAlgorithmFactories can create IConvolutionAlgorithm for given settings.");
		return TUniquePtr<IConvolutionAlgorithm>();
	}
}
