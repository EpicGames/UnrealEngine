// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Delay.h"
#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"
#include "HAL/IConsoleManager.h"

static float FDelayInitialAllocationSecondsCVar = -1.0f;
FAutoConsoleVariableRef CVarFDelayInitialAllocationSeconds(
	TEXT("au.DSP.InitialFDelayAllocationSeconds"),
	FDelayInitialAllocationSecondsCVar,
	TEXT("Override the inital delay line allocation in seconds, it will grow up to InBufferLengthSec.\n"),
	//TEXT("The default is -1.  A value less than zero will allocate the full InBufferLengthSec\n"),
	ECVF_Default);
	
	

static int32 DelayResetFadeLengthSamplesCVar = 64;
FAutoConsoleVariableRef CVarDelayResetFadeLengthSamples(
	TEXT("au.DSP.DelayResetFadeLengthSamples"),
	DelayResetFadeLengthSamplesCVar,
	TEXT("Controls fade length (in samples) when clearing internal memory with ResetWithFade().\n"),
	ECVF_Default);

namespace Audio
{
	FDelay::FDelay()
		: AudioBufferSize(0)
		, ReadIndex(0)
		, WriteIndex(0)
		, SampleRate(0)
		, DelayInSamples(0.0f)
		, MaxBufferLengthSamples(0)
		, EaseDelayMsec(0.0f, 0.0001f)
		, OutputAttenuation(1.0f)
		, OutputAttenuationDB(0.0f)
	{
		Reset();
	}

	FDelay::~FDelay() = default;

	// update metadata, call Reset()
	void FDelay::Init(const float InSampleRate, const float InBufferLengthSec)
	{
		SampleRate = InSampleRate;
		
		// we cache this value because it is where we cap resizing the buffer
		MaxBufferLengthSamples = InBufferLengthSec * (float)InSampleRate + 1;

		float InitialBufferSizeSeconds = InBufferLengthSec;

		if (FDelayInitialAllocationSecondsCVar > 0.f)
		{
			InitialBufferSizeSeconds = FMath::Min(InBufferLengthSec, FDelayInitialAllocationSecondsCVar);
		}
		
		AudioBufferSize = (int32)(InitialBufferSizeSeconds * (float)InSampleRate) + 1;
		Reset();
	}

	// resize AudioBuffer, zero-out delay line, reset indicies
	void FDelay::Reset()
	{
		AudioBuffer.Reset(AudioBufferSize);
		AudioBuffer.AddZeroed(AudioBufferSize);

		WriteIndex = 0;
		ReadIndex = 0;

		Update(true);
	}

	void FDelay::ResetWithFade()
	{
		const int32 NumSamplesToFade = FMath::Min(DelayResetFadeLengthSamplesCVar,  static_cast<int32>(DelayInSamples - 1));
		
		if (DelayResetFadeLengthSamplesCVar == 0 || NumSamplesToFade < 2)
		{
			Reset();
			return;
		}
		
		const bool bIsReadRegionSplit = WriteIndex < ReadIndex;
		
		// we don't want to accept new input while the delay is faded
		// (this could cause the two fades to create a small chirp)
		// so we start the input gain negative and rely on the clamping to mute (gain = 0)
		InputFadeGainStep = 1.f / static_cast<float>(NumSamplesToFade);
		InputAttenuation = -1.0f - InputFadeGainStep;


		// simple case
		if (!bIsReadRegionSplit)
		{
			const int32 NumSamplesToZero = WriteIndex - ReadIndex - NumSamplesToFade;
			
			const TArrayView<float> FadeRegion(AudioBuffer.GetData() + ReadIndex, NumSamplesToFade);
			const TArrayView<float> ZeroRegion(AudioBuffer.GetData() + ReadIndex + NumSamplesToFade, NumSamplesToZero);
			
			ArrayFade(FadeRegion, 1.0f, 0.0f);
			FMemory::Memzero(ZeroRegion.GetData(), ZeroRegion.NumBytes());
			
			return;
		}

		// complex case, two regions to worry about
		const int32 NumSamplesToZero = (AudioBufferSize - ReadIndex) + WriteIndex - NumSamplesToFade;
		const int32 ReadRegion1Num = AudioBufferSize - ReadIndex;
		const int32 ReadRegion2Num = WriteIndex;

		// 3 complex cases:
		// 1.) single fade region and two zero regions (fade region < read region 1)
		// 2.) two fade regions and a single zero region (fade region > read region 2)
		// 3.) [else] read region 1 is exactly the fade region (unlikely)
		// note: we should have at least on sample to zero out
		//       (see -1 in calculation of NumSamplesToFade)
		//       so technically we don't have an "all fade" case

		// case 1:
		if (NumSamplesToFade < ReadRegion1Num)
		{
			const int32 ZeroRegion1Size = NumSamplesToZero - ReadRegion2Num;
			const int32 ZeroRegion2Size = NumSamplesToZero - ZeroRegion1Size;

			const TArrayView<float> FadeRegion(AudioBuffer.GetData() + ReadIndex, NumSamplesToFade);
			const TArrayView<float> ZeroRegion1(AudioBuffer.GetData() + ReadIndex + NumSamplesToFade, ZeroRegion1Size);
			const TArrayView<float> ZeroRegion2(AudioBuffer.GetData(), ZeroRegion2Size);

			ArrayFade(FadeRegion, 1.0f, 0.0f);
			FMemory::Memzero(ZeroRegion1.GetData(), ZeroRegion1.NumBytes());
			FMemory::Memzero(ZeroRegion2.GetData(), ZeroRegion2.NumBytes());

			return;
		}
		
		// case 2:
		if (NumSamplesToFade > ReadRegion1Num)
		{
			const int32 FadeRegion1Size = AudioBufferSize - ReadIndex;
			const int32 FadeRegion2Size = NumSamplesToFade - FadeRegion1Size;
			const int32 ZeroRegionSize = WriteIndex - FadeRegion2Size;

			const TArrayView<float> FadeRegion1(AudioBuffer.GetData() + ReadIndex, FadeRegion1Size);
			const TArrayView<float> FadeRegion2(AudioBuffer.GetData(), FadeRegion2Size);
			const TArrayView<float> ZeroRegion(AudioBuffer.GetData() + FadeRegion2Size, ZeroRegionSize);

			// technically there will be 2 samples with this gain value applied
			// (should be imperceptible)
			const float MidGain = FadeRegion2Size / static_cast<float>(NumSamplesToFade);

			ArrayFade(FadeRegion1, 1.0f, MidGain);
			ArrayFade(FadeRegion2, MidGain, 0.0f);
			FMemory::Memzero(ZeroRegion.GetData(), ZeroRegion.NumBytes());

			return;
		}

		// case 3 (rare):
		const TArrayView<float> FadeRegion(AudioBuffer.GetData() + ReadIndex, AudioBufferSize - ReadIndex);
		const TArrayView<float> ZeroRegion(AudioBuffer.GetData(), WriteIndex);

		ArrayFade(FadeRegion, 1.0f, 0.0f);
		FMemory::Memzero(ZeroRegion.GetData(), ZeroRegion.NumBytes());
	}
	 
	void FDelay::SetDelayMsec(const float InDelayMsec)
	{
		// Directly set the delay
		const float NewDelayInSamples = InDelayMsec * SampleRate * 0.001f;
		DelayInSamples = FMath::Min(NewDelayInSamples, MaxBufferLengthSamples - 1);
		ResizeIfNeeded(DelayInSamples);
		Update(true);
	}

	void FDelay::SetDelaySamples(const float InDelaySamples)
	{
		DelayInSamples = FMath::Min(InDelaySamples, MaxBufferLengthSamples - 1);
		ResizeIfNeeded(DelayInSamples);
		Update(true);
	}

	void FDelay::SetEasedDelayMsec(const float InDelayMsec, const bool bIsInit)
	{
		const float DesiredDelayInSamples = InDelayMsec * SampleRate * 0.001f;
		const float TargetDelayInSamples = FMath::Min(DesiredDelayInSamples, MaxBufferLengthSamples - 1);

		if (FMath::IsNearlyEqual(TargetDelayInSamples, DesiredDelayInSamples))
		{
			SetDelaySamples(TargetDelayInSamples);
			return;
		}
		
		ResizeIfNeeded(TargetDelayInSamples);

		EaseDelayMsec.SetValue(InDelayMsec, bIsInit);
		if (bIsInit)
		{
			const float NewDelayInSamples = InDelayMsec * SampleRate * 0.001f;
			DelayInSamples = TargetDelayInSamples;
		}
		Update(bIsInit);
	}

	void FDelay::SetEaseFactor(const float InEaseFactor)
	{
		EaseDelayMsec.SetEaseFactor(InEaseFactor);
	}

	void FDelay::SetOutputAttenuationDB(const float InDelayAttenDB)
	{
		OutputAttenuationDB = InDelayAttenDB;

		// Compute linear output attenuation based on DB attenuation settings
		OutputAttenuation = FMath::Pow(10.0f, OutputAttenuationDB / 20.0f);
	}

	float FDelay::Read() const
	{
		// Read the output of the delay at ReadIndex
		const float Yn = AudioBuffer[ReadIndex];

		// Read the location ONE BEHIND yn at y(n-1)
		int32 ReadIndexPrev = ReadIndex - 1;
		if (ReadIndexPrev < 0)
		{
			ReadIndexPrev = AudioBufferSize - 1;
		}

		// Set y(n-1)
		const float YnPrev = AudioBuffer[ReadIndexPrev];

		// Get the amount of fractional delay between previous and next read indices
		const float Fraction = DelayInSamples - (int32)DelayInSamples;

		return FMath::Lerp(Yn, YnPrev, Fraction);
	}

	float FDelay::ReadDelayAt(const float InReadMsec) const
	{
		const float ReadAtDelayInSamples = InReadMsec*((float)SampleRate) / 1000.0f;

		// Subtract to make read index
		int32 ReadAtReadIndex = WriteIndex - (int32)ReadAtDelayInSamples;

		if (ReadAtReadIndex < 0)
		{
			ReadAtReadIndex += AudioBufferSize;	// amount of wrap is Read + Length
		}

		// Read the output of the delay at ReadAtReadIndexs
		float Yn = AudioBuffer[ReadAtReadIndex];

		// Read the location ONE BEHIND yn at y(n-1)
		int32 ReadAtReadIndexPrev = ReadAtReadIndex - 1;
		if (ReadAtReadIndexPrev < 0)
		{
			ReadAtReadIndexPrev = AudioBufferSize - 1;
		}

		// get y(n-1)
		const float YnPrev = AudioBuffer[ReadAtReadIndexPrev];

		// interpolate: (0, yn) and (1, yn_1) by the amount fracDelay
		float Fraction = ReadAtDelayInSamples - (int32)ReadAtDelayInSamples;

		return FMath::Lerp(Yn, YnPrev, Fraction);
	}

	void FDelay::WriteToBuffer(const float InInput)
	{
		InputAttenuation = FMath::Min(InputAttenuation + InputFadeGainStep, 1.f); // Ceil to 1.0
		AudioBuffer[WriteIndex] = InInput * FMath::Max(InputAttenuation, 0.f);   // Floor to 0.0
	}

	void FDelay::AdvanceIndices()
	{
		if (++WriteIndex >= AudioBufferSize)
		{
			WriteIndex = 0;
		}
		
		if (++ReadIndex  >= AudioBufferSize)
		{
			ReadIndex  = 0;
		}
	}

	void FDelay::WriteDelayAndInc(const float InDelayInput)
	{
		WriteToBuffer(InDelayInput);
		AdvanceIndices();
	}

	float FDelay::ProcessAudioSample(const float InAudio)
	{
		Update();
		WriteToBuffer(InAudio);
		const float Yn = Read();
		AdvanceIndices();
		return OutputAttenuation * Yn;
	}

	void FDelay::ProcessAudioBuffer(const float* InAudio, int32 InNumSamples, float* OutAudio)
	{
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
		{
			Update();
			WriteToBuffer(InAudio[SampleIndex]);
			OutAudio[SampleIndex] = OutputAttenuation * Read();
			AdvanceIndices();
		}
	}

	void FDelay::Update(bool bForce)
	{
		if (!EaseDelayMsec.IsDone() || bForce)
		{
			// Compute the delay in samples based on msec delay line
			// If we're easing, then get the delay based on the current value of the ease
			if (!EaseDelayMsec.IsDone())
			{
				DelayInSamples = EaseDelayMsec.GetNextValue() * SampleRate * 0.001f;
			}

			DelayInSamples = FMath::Clamp(DelayInSamples, 0.0f, (float)(MaxBufferLengthSamples - 1));

			// Subtract from write index the delay in samples (will do interpolation during read)
			ReadIndex = WriteIndex - (int32)DelayInSamples;

			// If negative, wrap around
			if (ReadIndex < 0)
			{
				ReadIndex += AudioBufferSize;
			}
		}
	}

	void FDelay::ResizeIfNeeded(const int32 InNewNumSamples)
	{
		// should be clamped by callers
		ensure(InNewNumSamples <= MaxBufferLengthSamples);

		// already large enough
		if (InNewNumSamples <= AudioBufferSize)
		{
			return;
		}

		// resize the buffer
		const int32 OldBufferSize = AudioBufferSize;
		AudioBufferSize = FMath::Min(AudioBufferSize * 2, MaxBufferLengthSamples);
		AudioBuffer.SetNumUninitialized(AudioBufferSize);

		// see if we need to copy data to the end
		if (ReadIndex < WriteIndex)
		{
			// no action needed, we will write over the uninitialzed data
			// before we read from it.
			return;
		}

		// (WriteIndex <= ReadIndex): our soon-to-be-read-data is in two chunks.
		// we need to copy the second chunk to the end of the now resized array
		// and update the read index.

		// note: we can leave alone the old data since we will write to it before its read
		const int32 SamplesToCopy = OldBufferSize - WriteIndex;
		const int32 OldReadIndex = ReadIndex;
		ReadIndex = AudioBufferSize - SamplesToCopy;
		FMemory::Memmove(&AudioBuffer[WriteIndex], &AudioBuffer[AudioBufferSize - OldBufferSize], SamplesToCopy * sizeof(float));
	}
}
