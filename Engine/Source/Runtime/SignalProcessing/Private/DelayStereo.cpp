// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/DelayStereo.h"

namespace Audio
{
	FDelayStereo::FDelayStereo()
	{
	}

	FDelayStereo::~FDelayStereo()
	{
	}

	void FDelayStereo::SetMode(const EStereoDelayMode::Type InMode)
	{
		DelayMode = InMode;
	}

	void FDelayStereo::SetDelayTimeMsec(const float InDelayTimeMsec)
	{
		DelayTimeMsec = InDelayTimeMsec;
		UpdateDelays();
	}

	void FDelayStereo::SetFeedback(const float InFeedback)
	{
		Feedback = FMath::Clamp(InFeedback, 0.0f, 1.0f);
	}

	void FDelayStereo::SetDelayRatio(const float InDelayRatio)
	{
		DelayRatio = FMath::Clamp(InDelayRatio, -1.0f, 1.0f);
		UpdateDelays();
	}

	void FDelayStereo::SetWetLevel(const float InWetLevel)
	{
		WetLevel = FMath::Clamp(InWetLevel, 0.0f, 1.0f);;
	}

	void FDelayStereo::SetDryLevel(const float InDryLevel)
	{
		DryLevel = FMath::Clamp(InDryLevel, 0.0f, 1.0f);;
	}

	void FDelayStereo::SetFilterEnabled(bool bInEnabled)
	{
		bIsFilterEnabled = bInEnabled;
	}

	void FDelayStereo::SetFilterSettings(EBiquadFilter::Type InFilterType, const float InCutoffFrequency, const float InQ)
	{
		FilterFreq = InCutoffFrequency;
		FilterQ = InQ;

		for (FBiquadFilter& Filter : BiquadFilters)
		{
			Filter.SetParams(InFilterType, InCutoffFrequency, GetBandwidthFromQ(FilterQ), 0.0f);
		}
	}

	void FDelayStereo::Init(const float InSampleRate, int32 InNumChannels, const float InDelayLengthSec)
	{
		NumChannels = InNumChannels;

		// Init the delay lines
		for (int32 Channel = 0; Channel < InNumChannels; ++Channel)
		{
			int32 Index = Delays.Add(FDelay());
			Delays[Index].Init(InSampleRate, 2.0f * InDelayLengthSec);

			Index = BiquadFilters.Add(FBiquadFilter());
			BiquadFilters[Index].Init(InSampleRate, 1, FilterType, FilterFreq, GetBandwidthFromQ(FilterQ));
		}

		Reset();
	}

	void FDelayStereo::Reset()
	{
		bIsInit = true;
		for (FDelay& Delay : Delays)
		{
			Delay.Reset();
		}
	}

	void FDelayStereo::UpdateDelays()
	{
		// As delay ratio goes to zero, the delay times are the same
		if (NumChannels == 1)
		{
			Delays[0].SetEasedDelayMsec(DelayTimeMsec * (1.0f + DelayRatio), bIsInit);
		}
		else
		{
			Delays[0].SetEasedDelayMsec(DelayTimeMsec * (1.0f + DelayRatio), bIsInit);
			Delays[1].SetEasedDelayMsec(DelayTimeMsec * (1.0f - DelayRatio), bIsInit);
		}
	}

	void FDelayStereo::ProcessAudioFrame(const float* InFrame, float* OutFrame)
	{
		bIsInit = false;

		// 1-channel audio just does a simple delay
		if (NumChannels == 1)
		{
			FDelay& Delay = Delays[0];
			float DelayOut = Delay.Read();

			float DelayIn = InFrame[0] + DelayOut * Feedback;

			// Feed delay out through the filter
			if (bIsFilterEnabled)
			{
				FBiquadFilter& Biquad = BiquadFilters[0];
				Biquad.ProcessAudioFrame(&DelayIn, &DelayIn);
			}

			DelayOut = Delay.ProcessAudioSample(DelayIn);
			OutFrame[0] = DryLevel * InFrame[0] + WetLevel * DelayOut;
		}
		else
		{
			float LeftDelayOut = Delays[0].Read();
			float RightDelayOut = Delays[1].Read();

			float LeftDelayIn = 0.0f;
			float RightDelayIn = 0.0f;

			if (DelayMode == EStereoDelayMode::Normal)
			{
				LeftDelayIn = InFrame[0] + LeftDelayOut * Feedback;
				RightDelayIn = InFrame[1] + RightDelayOut * Feedback;
			}
			else if (DelayMode == EStereoDelayMode::Cross)
			{
				LeftDelayIn = InFrame[1] + LeftDelayOut * Feedback;
				RightDelayIn = InFrame[0] + RightDelayOut * Feedback;
			}
			else if (DelayMode == EStereoDelayMode::PingPong)
			{
				LeftDelayIn = InFrame[1] + RightDelayOut * Feedback;
				RightDelayIn = InFrame[0] + LeftDelayOut * Feedback;
			}

			// Feed delay out through the filter
			if (bIsFilterEnabled)
			{
				FBiquadFilter& BiquadLeft = BiquadFilters[0];
				BiquadLeft.ProcessAudioFrame(&LeftDelayIn, &LeftDelayIn);

				FBiquadFilter& BiquadRight = BiquadFilters[1];
				BiquadLeft.ProcessAudioFrame(&RightDelayIn, &RightDelayIn);
			}

			float WetLeftOut = 0.0f;
			float WetRightOut = 0.0f;
			WetLeftOut = Delays[0].ProcessAudioSample(LeftDelayIn);
			WetRightOut = Delays[1].ProcessAudioSample(RightDelayIn);

			OutFrame[0] = DryLevel * InFrame[0] + WetLevel * WetLeftOut;
			OutFrame[1] = DryLevel * InFrame[1] + WetLevel * WetRightOut;
		}
	}

	void FDelayStereo::ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer)
	{
		bIsInit = false;

		if (NumChannels == 1)
		{
			FDelay& Delay = Delays[0];

			if (bIsFilterEnabled)
			{
				FBiquadFilter& BiquadLeft = BiquadFilters[0];

				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
				{
					float DelayOut = Delay.Read();
					float DelayIn = InBuffer[SampleIndex] + DelayOut * Feedback;

					BiquadLeft.ProcessAudioFrame(&DelayIn, &DelayIn);

					DelayOut = Delay.ProcessAudioSample(DelayIn);
					OutBuffer[SampleIndex] = DryLevel * InBuffer[SampleIndex] + WetLevel * DelayOut;
				}
}
			else
			{
				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
				{
					float DelayOut = Delay.Read();
					float DelayIn = InBuffer[SampleIndex] + DelayOut * Feedback;
					DelayOut = Delay.ProcessAudioSample(DelayIn);
					OutBuffer[SampleIndex] = DryLevel * InBuffer[SampleIndex] + WetLevel * DelayOut;
				}
			}
		}
		else
		{
			if (bIsFilterEnabled)
			{
				FBiquadFilter& BiquadLeft = BiquadFilters[0];
				FBiquadFilter& BiquadRight = BiquadFilters[1];

				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
				{
					float LeftDelayOut = Delays[0].Read();
					float RightDelayOut = Delays[1].Read();

					float LeftDelayIn = 0.0f;
					float RightDelayIn = 0.0f;

					if (DelayMode == EStereoDelayMode::Normal)
					{
						LeftDelayIn = InBuffer[SampleIndex] + LeftDelayOut * Feedback;
						RightDelayIn = InBuffer[SampleIndex + 1] + RightDelayOut * Feedback;
					}
					else if (DelayMode == EStereoDelayMode::Cross)
					{
						LeftDelayIn = InBuffer[SampleIndex + 1] + LeftDelayOut * Feedback;
						RightDelayIn = InBuffer[SampleIndex] + RightDelayOut * Feedback;
					}
					else if (DelayMode == EStereoDelayMode::PingPong)
					{
						LeftDelayIn = InBuffer[SampleIndex + 1] + RightDelayOut * Feedback;
						RightDelayIn = InBuffer[SampleIndex] + LeftDelayOut * Feedback;
					}

					BiquadLeft.ProcessAudioFrame(&LeftDelayIn, &LeftDelayIn);
					BiquadRight.ProcessAudioFrame(&RightDelayIn, &RightDelayIn);

					float WetLeftOut = 0.0f;
					float WetRightOut = 0.0f;
					WetLeftOut = Delays[0].ProcessAudioSample(LeftDelayIn);
					WetRightOut = Delays[1].ProcessAudioSample(RightDelayIn);

					OutBuffer[SampleIndex] = DryLevel * InBuffer[SampleIndex] + WetLevel * WetLeftOut;
					OutBuffer[SampleIndex + 1] = DryLevel * InBuffer[SampleIndex + 1] + WetLevel * WetRightOut;
				}
			}
			else
			{
				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
				{
					float LeftDelayOut = Delays[0].Read();
					float RightDelayOut = Delays[1].Read();

					float LeftDelayIn = 0.0f;
					float RightDelayIn = 0.0f;

					if (DelayMode == EStereoDelayMode::Normal)
					{
						LeftDelayIn = InBuffer[SampleIndex] + LeftDelayOut * Feedback;
						RightDelayIn = InBuffer[SampleIndex + 1] + RightDelayOut * Feedback;
					}
					else if (DelayMode == EStereoDelayMode::Cross)
					{
						LeftDelayIn = InBuffer[SampleIndex + 1] + LeftDelayOut * Feedback;
						RightDelayIn = InBuffer[SampleIndex] + RightDelayOut * Feedback;
					}
					else if (DelayMode == EStereoDelayMode::PingPong)
					{
						LeftDelayIn = InBuffer[SampleIndex + 1] + RightDelayOut * Feedback;
						RightDelayIn = InBuffer[SampleIndex] + LeftDelayOut * Feedback;
					}

					float WetLeftOut = 0.0f;
					float WetRightOut = 0.0f;
					WetLeftOut = Delays[0].ProcessAudioSample(LeftDelayIn);
					WetRightOut = Delays[1].ProcessAudioSample(RightDelayIn);

					OutBuffer[SampleIndex] = DryLevel * InBuffer[SampleIndex] + WetLevel * WetLeftOut;
					OutBuffer[SampleIndex + 1] = DryLevel * InBuffer[SampleIndex + 1] + WetLevel * WetRightOut;
				}
			}
		}
	}


}
