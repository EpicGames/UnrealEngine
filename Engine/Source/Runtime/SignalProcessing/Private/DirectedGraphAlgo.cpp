// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/DirectedGraphAlgo.h"

// This entire file has been deprecated/moved to MathCore (Graph/DirectedGraphUtils.h)
PRAGMA_DISABLE_DEPRECATION_WARNINGS

namespace Audio
{

	namespace DirectedGraphAlgoPrivate
	{
		// Used to track which vertices have been visited and vertex children.
		struct FDepthFirstSortInfo
		{
			bool TemporaryMark = false;
			bool PermanentMark = false;
			TArray<int32> Children;
		};

		// Recursive function for depth first topological sorting.
		bool DepthFirstTopologicalSortVisit(int32 InVertex, TMap<int32, FDepthFirstSortInfo>& InSortInfos, TArray<int32>& OutReverseOrder)
		{
			FDepthFirstSortInfo& Info = InSortInfos[InVertex];

			if (Info.PermanentMark)
			{
				// Exit depth first traversal
				return true;
			}
			else if (Info.TemporaryMark)
			{
				// Found a cycle in the DAG.
				return false;
			}

			Info.TemporaryMark = true;

			for (int32 Child : Info.Children)
			{
				if (!DepthFirstTopologicalSortVisit(Child, InSortInfos, OutReverseOrder))
				{
					// cycle found in children.
					return false;
				}
			}

			Info.TemporaryMark = false;
			Info.PermanentMark = true;

			OutReverseOrder.Add(InVertex);

			// subgraph successfully sorted.
			return true;
		}
	} // End namespace DirectedAlgoGraphCorePrivate


	void FDirectedGraphAlgo::BuildDirectedTree(TArrayView<const FDirectedEdge> InEdges, FDirectedTree& OutTree)
	{
		for (const FDirectedEdge& Edge : InEdges)
		{
			FDirectedTreeElement& Element = OutTree.FindOrAdd(Edge.Get<0>());

			Element.Children.AddUnique(Edge.Get<1>());
		}
	}

	void FDirectedGraphAlgo::BuildTransposeDirectedTree(TArrayView<const FDirectedEdge> InEdges, FDirectedTree& OutTree)
	{
		for (const FDirectedEdge& Edge : InEdges)
		{
			FDirectedTreeElement& Element = OutTree.FindOrAdd(Edge.Get<1>());

			Element.Children.AddUnique(Edge.Get<0>());
		}
	}

	void FDirectedGraphAlgo::DepthFirstTraversal(int32 InInitialVertex, const FDirectedTree& InTree, TFunctionRef<bool (int32)> InVisitFunc)
	{
		// Non recursive depth first traversal.

		TArray<int32> Stack({InInitialVertex});
		TSet<int32> Visited;

		while (Stack.Num() > 0)
		{
			int32 CurrentVertex = Stack.Pop();
			if (Visited.Contains(CurrentVertex))
			{
				continue;
			}

			bool bVisitChildren = InVisitFunc(CurrentVertex);

			Visited.Add(CurrentVertex);

			if (bVisitChildren)
			{
				const FDirectedTreeElement* Element = InTree.Find(CurrentVertex);

				if (nullptr != Element)
				{
					Stack.Append(Element->Children);
				}
			}
		}
	}


	void FDirectedGraphAlgo::BreadthFirstTraversal(int32 InInitialVertex, const FDirectedTree& InTree, TFunctionRef<bool (int32)> InVisitFunc)
	{
		// Non recursive breadth first traversal.

		TArray<int32> Queue({InInitialVertex});
		TSet<int32> Visited;

		bool bVisitChildren = true;

		while (Queue.Num() > 0)
		{
			int32 CurrentVertex = Queue[0];
			Queue.RemoveAt(0);

			if (Visited.Contains(CurrentVertex))
			{
				continue;
			}

			bVisitChildren = InVisitFunc(CurrentVertex);

			Visited.Add(CurrentVertex);

			if (bVisitChildren)
			{
				const FDirectedTreeElement* Element = InTree.Find(CurrentVertex);

				if (nullptr != Element)
				{
					Queue.Append(Element->Children);
				}
			}
		}
	}

	bool FDirectedGraphAlgo::DepthFirstTopologicalSort(TArrayView<const int32> InUniqueVertices, TArrayView<const FDirectedEdge> InUniqueEdges, TArray<int32>& OutVertexOrder)
	{
		using namespace DirectedGraphAlgoPrivate;

		// Initialize side info about which vertices have been visited and which
		// have not.
		TMap<int32, FDepthFirstSortInfo> SortInfos;

		const int32 MaxVertexNum = InUniqueVertices.Num();
		SortInfos.Reserve(SortInfos.Num() + MaxVertexNum);
		OutVertexOrder.Reserve(OutVertexOrder.Num() + MaxVertexNum);

		for (int32 Vertex : InUniqueVertices)
		{
			SortInfos.Add(Vertex);
		}

		for (const FDirectedEdge& Edge : InUniqueEdges)
		{
			FDepthFirstSortInfo& Info = SortInfos[Edge.Get<0>()];
			Info.Children.Add(Edge.Get<1>());
		}

		bool bSuccess = true;

		// Loop through each node to make sure they are all visited.
		for (int32 Vertex : InUniqueVertices)
		{
			FDepthFirstSortInfo& OuterInfo = SortInfos[Vertex];

			if (!(OuterInfo.TemporaryMark || OuterInfo.PermanentMark))
			{
				bSuccess = DepthFirstTopologicalSortVisit(Vertex, SortInfos, OutVertexOrder);

				if (!bSuccess)
				{
					break;
				}
			}
		}

		// Depth first sort generates it in reverse order. It is reversed here
		// so it is in forward order. 
		Algo::Reverse(OutVertexOrder);

		return bSuccess;
	}

	bool FDirectedGraphAlgo::KahnTopologicalSort(TArrayView<const int32> InUniqueVertices, TArrayView<const FDirectedEdge> InUniqueEdges, TArray<int32>& OutVertexOrder)
	{
		using FVertexMultiMap = TMultiMap<int32, int32>;
		using FVertexPair = TMultiMap<int32, int32>::ElementType;
		
		FVertexMultiMap Dependencies;

		for (const FDirectedEdge& Edge : InUniqueEdges)
		{
			// Destination vertices are dependent upon source vertices.
			// Create a multimap where each destination vertex refrences
			// all of the source vertices that have not been placed in the
			// output. 
			Dependencies.Add(Edge.Get<1>(), Edge.Get<0>());
		}

		TArray<int32> UniqueVertices(InUniqueVertices.GetData(), InUniqueVertices.Num());
		TArray<int32> IndependentVertices;

		// Sort graph so that vertices with no dependencies always go first.
		while (UniqueVertices.Num() > 0)
		{
			IndependentVertices = UniqueVertices.FilterByPredicate([&](int32 InVertex) { 
					return (Dependencies.Num(InVertex) == 0);
			});

			if (0 == IndependentVertices.Num())
			{
				// If there are no independent vertices, then likely there is a 
				// cycle in the graph.
				return false;
			}

			FVertexMultiMap UpdatedDependencies;
			
			// Remove independent vertices from dependency map.
			for (FVertexPair& Element : Dependencies)
			{
				if (!IndependentVertices.Contains(Element.Value))
				{
					UpdatedDependencies.AddUnique(Element.Key, Element.Value);
				}
			}
			Dependencies = UpdatedDependencies;

			// Remove independent vertices from node list
			for (int32 Vertex : IndependentVertices)
			{
				UniqueVertices.RemoveSwap(Vertex);
			}

			// Add independent vertices to output.
			OutVertexOrder.Append(IndependentVertices);
		}

		return true;
	}

	namespace DirectedGraphAlgoPrivate
	{
		struct FTarjanVertexInfo
		{
			// Tag value given to a vertex.
			int32 Tag = INDEX_NONE;

			// Lowest tag in strongly connected components
			int32 LowLinkTag = INDEX_NONE;

			// Flag for global stack
			bool bIsOnStack = false;

			// Immediate children of vertex in DAG.
			TArray<int32> Children;
		};

		struct FTarjanAlgoImplSettings 
		{
			// Single vertices can be returned as strongly connected components
			// but usually they are not wanted by the caller. Setting this to
			// true will require all strongly connected components to contain
			// two or more vertices.
			bool bExcludeSingleVertex;
		};

		// Private implementation of algorithm.
		class FTarjanAlgoImpl
		{
			public:

			FTarjanAlgoImpl(const FTarjanAlgoImplSettings& InSettings)
			:	Settings(InSettings)
			{
			}

			bool FindStronglyConnectedComponents(const TSet<FDirectedEdge>& InEdges, TArray<FStronglyConnectedComponent>& OutComponents)
			{
				using FElement = TMap<int32, FTarjanVertexInfo>::ElementType;

				// Initialize internal data stores and tracking info.
				Init(InEdges);
				
				int32 OriginalNumOutComponents = OutComponents.Num();

				// Iterate through all vertices. Update(...) does a 
				// depth-first-search which may result some vertices getting 
				// inspected during recursive calls to Update(...).
				for (FElement& Element : VertexInfos)
				{
					// Only examine vertice if it hasn't already been inspected.
					if (Element.Value.Tag == INDEX_NONE)
					{
						Update(Element.Key, Element.Value, OutComponents);
					}
				}
				
				// Check if the size of OutComponents has grown.
				return OutComponents.Num() > OriginalNumOutComponents;
			}

			private:

			void Init(const TSet<FDirectedEdge>& InEdges)
			{
				CurrentTag = 0;

				VertexInfos.Reset();
				VertexStack.Reset();

				for (const FDirectedEdge& Edge : InEdges)
				{
					FTarjanVertexInfo& Info = VertexInfos.FindOrAdd(Edge.Get<0>());
					Info.Children.AddUnique(Edge.Get<1>());

					VertexInfos.FindOrAdd(Edge.Get<1>());
				}
			}

			void Update(int32 InVertex, FTarjanVertexInfo& InVertexInfo, TArray<FStronglyConnectedComponent>& OutComponents)
			{
				InVertexInfo.Tag = CurrentTag;
				InVertexInfo.LowLinkTag = CurrentTag;
				InVertexInfo.bIsOnStack = true;
				VertexStack.Add(InVertex);

				// Tag needs to always be increasing during subsequent calls to
				// Update(...).
				CurrentTag++;

				// Perform depth first traversal through the graph looking for 
				// children that are already on the stack (which denotes a 
				// strongly connected component)
				for (int32 ChildVertex : InVertexInfo.Children)
				{
					FTarjanVertexInfo& ChildInfo = VertexInfos[ChildVertex];

					if (ChildInfo.Tag == INDEX_NONE)
					{
						// If ChildInfo.Tag == INDEX_NONE, then this child is 
						// not on the stack because it has never been inspected.

						// Continue in depth-first-traversal. 
						Update(ChildVertex, ChildInfo, OutComponents);

						// Update to use the lowest link tag found in children.
						InVertexInfo.LowLinkTag = FMath::Min(InVertexInfo.LowLinkTag, ChildInfo.LowLinkTag);
					}
					else if(ChildInfo.bIsOnStack)
					{
						// This child is on the stack and represents a strongly 
						// connected component. 
						InVertexInfo.LowLinkTag = FMath::Min(InVertexInfo.LowLinkTag, ChildInfo.Tag);
					}
				}

				if (InVertexInfo.LowLinkTag == InVertexInfo.Tag)
				{
					// Create a strongly connected component object if this is 
					// the root vertex of a strongly connected component.
					AddStronglyConnectedComponent(InVertex, OutComponents);
				}
			}

			// Adds the current strongly connected component to the 
			// OutComponents array.
			void AddStronglyConnectedComponent(int32 InRootVertex, TArray<FStronglyConnectedComponent>& OutComponents)
			{
				// When a single vertex represents the entire strongly connected
				// component, the root vertex is the same as the vertex at the
				// top of the stack.
				bool bIsSingleVertex = (VertexStack.Top() == InRootVertex);

				if (bIsSingleVertex && Settings.bExcludeSingleVertex)
				{
					// This strongly connected component is skipped because it 
					// consists of a single vertex. 
					//
					// The stack still needs to be updated as well as the vertex
					// info for the root vertex.
					int32 TopVertex = VertexStack.Pop();	
					FTarjanVertexInfo& TopInfo = VertexInfos[TopVertex];
					TopInfo.bIsOnStack = false;
				}
				else
				{
					bool bDidCompleteLoop = false;

					// Create a new strongly connected component subgraph.
					FStronglyConnectedComponent& StronglyConnectedComponent = OutComponents.AddDefaulted_GetRef();

					// Pop off the top of the vertex stack until we reach the
					// root vertex.
					while (!bDidCompleteLoop)
					{
						int32 TopVertex = VertexStack.Pop();

						FTarjanVertexInfo& TopInfo = VertexInfos[TopVertex];
						TopInfo.bIsOnStack = false;

						StronglyConnectedComponent.Vertices.Add(TopVertex);

						// The unwinding continues until the current vertex is 
						// removed from the stack.
						bDidCompleteLoop = (TopVertex == InRootVertex);
					}

					// Add the edges associated with the strongly connected
					// component. 
					for (int32 ComponentVertex : StronglyConnectedComponent.Vertices)
					{
						FTarjanVertexInfo& ComponentInfo = VertexInfos[ComponentVertex];

						for (int32 ChildVertex : ComponentInfo.Children)
						{
							if (StronglyConnectedComponent.Vertices.Contains(ChildVertex))
							{
								StronglyConnectedComponent.Edges.Emplace(ComponentVertex, ChildVertex);
							}
						}
					}
				}
			}

			FTarjanAlgoImplSettings Settings;

			int32 CurrentTag = 0;

			TMap<int32, FTarjanVertexInfo> VertexInfos;

			TArray<int32> VertexStack;
		};
	}

	bool FDirectedGraphAlgo::TarjanStronglyConnectedComponents(const TSet<FDirectedEdge>& InEdges, TArray<FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex)
	{
		using namespace DirectedGraphAlgoPrivate;

		// Use private implementation of Tarjan algorithm.
		FTarjanAlgoImplSettings Settings;
		Settings.bExcludeSingleVertex = bExcludeSingleVertex;

		FTarjanAlgoImpl TarjanAlgo(Settings);

		return TarjanAlgo.FindStronglyConnectedComponents(InEdges, OutComponents);
	}

}

PRAGMA_ENABLE_DEPRECATION_WARNINGS