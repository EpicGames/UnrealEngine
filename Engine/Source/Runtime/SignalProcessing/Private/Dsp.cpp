// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Dsp.h"

#include "DSP/FloatArrayMath.h"

namespace Audio
{
	void EncodeMidSide(
		const FAlignedFloatBuffer& InLeftChannel, 
		const FAlignedFloatBuffer& InRightChannel, 
		FAlignedFloatBuffer& OutMidChannel, 
		FAlignedFloatBuffer& OutSideChannel)
	{
		const int32 LeftInNumElements = InLeftChannel.Num();
		const int32 RightInNumElements = InRightChannel.Num();
		const bool bInputChannelsEqual = LeftInNumElements == RightInNumElements;
		const bool bOutputChannelsBigEnough = (OutMidChannel.Num() >= LeftInNumElements) && (OutSideChannel.Num() >= LeftInNumElements);

		if (!bInputChannelsEqual || !bOutputChannelsBigEnough)
		{
			return;
		}

		OutMidChannel.Reset(LeftInNumElements);
		OutSideChannel.Reset(LeftInNumElements);
		OutMidChannel.AddUninitialized(LeftInNumElements);
		OutSideChannel.AddUninitialized(LeftInNumElements);

		ArraySubtract(InLeftChannel, InRightChannel, OutSideChannel);

		//Output
		ArraySum(InLeftChannel, InRightChannel, OutMidChannel);

	}

	void DecodeMidSide(
		const FAlignedFloatBuffer& InMidChannel,
		const FAlignedFloatBuffer& InSideChannel,
		FAlignedFloatBuffer& OutLeftChannel,
		FAlignedFloatBuffer& OutRightChannel)
	{
		const int32 LeftInNumElements = InMidChannel.Num();
		const int32 RightInNumElements = InSideChannel.Num();
		const bool bInputChannelsEqual = LeftInNumElements == RightInNumElements;
		const bool bOutputChannelsBigEnough = (OutLeftChannel.Num() >= LeftInNumElements) && (OutRightChannel.Num() >= LeftInNumElements);

		if (!bInputChannelsEqual || !bOutputChannelsBigEnough)
		{
			return;
		}

		OutLeftChannel.Reset(LeftInNumElements);
		OutRightChannel.Reset(LeftInNumElements);
		OutLeftChannel.AddUninitialized(InMidChannel.Num());
		OutRightChannel.AddUninitialized(InSideChannel.Num());

		// Calculate the average value between the two signals at each sample
		const float AverageScale = 0.5f;
		ArrayWeightedSum(InMidChannel, AverageScale, InSideChannel, AverageScale, OutLeftChannel);

		ArraySubtract(InMidChannel, InSideChannel, OutRightChannel);
		ArrayMultiplyByConstantInPlace(OutRightChannel, AverageScale);

	}

}
