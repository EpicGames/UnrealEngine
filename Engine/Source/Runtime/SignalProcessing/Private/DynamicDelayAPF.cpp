// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/DynamicDelayAPF.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FDynamicDelayAPF::FDynamicDelayAPF(float InG, int32 InMinDelay, int32 InMaxDelay, int32 InMaxNumInternalBufferSamples, float InSampleRate)
	:	EaseTimeInSec(1.f)
	,	MinDelay(InMinDelay)
	,	MaxDelay(InMaxDelay)
	,	NumDelaySamples(InMinDelay - 1)
	,	NumInternalBufferSamples(InMaxNumInternalBufferSamples)
	{
		G.Init(InSampleRate);
		G.SetValue(InG);

		checkf(NumDelaySamples >= 0, TEXT("Minimum delay must be atleast 1"));
		// NumInternalBufferSamples must be less than the length of the delay to support buffer indexing logic.
		int32 MaxBufferSamples = FMath::Min(NumDelaySamples, InMaxNumInternalBufferSamples);
		if (NumInternalBufferSamples > MaxBufferSamples)
		{
			NumInternalBufferSamples = MaxBufferSamples;
			// Block length must be divisible by simd alignment to support simd operations.
			NumInternalBufferSamples -= (NumInternalBufferSamples % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		}

		checkf(NumInternalBufferSamples > 0, TEXT("Invalid internal buffer length"));

		// Check that delays are sane.
		checkf(MinDelay <= MaxDelay, TEXT("invalid delay range"));


		// Allocate delay line.
		IntegerDelayLine = MakeUnique<FAlignedBlockBuffer>((2 * MinDelay) + NumInternalBufferSamples, MinDelay + NumInternalBufferSamples);
		IntegerDelayLine->AddZeros(NumDelaySamples);
		//FractionalDelayLine = MakeUnique<FAllPassFractionalDelay>(MaxDelay - MinDelay + 1, NumInternalBufferSamples);
		FractionalDelayLine = MakeUnique<FLinearInterpFractionalDelay>(MaxDelay - MinDelay + 1, NumInternalBufferSamples);

		// Allocate internal buffers.
		FractionalDelays.Reset(NumInternalBufferSamples);
		FractionalDelays.AddUninitialized(NumInternalBufferSamples);
		WorkBufferA.Reset(NumInternalBufferSamples);
		WorkBufferA.AddUninitialized(NumInternalBufferSamples);
		WorkBufferB.Reset(NumInternalBufferSamples);
		WorkBufferB.AddUninitialized(NumInternalBufferSamples);
	}

	FDynamicDelayAPF::~FDynamicDelayAPF()
	{}

	void FDynamicDelayAPF::ProcessAudio(const FAlignedFloatBuffer& InSamples, const FAlignedFloatBuffer& InSampleDelays, FAlignedFloatBuffer& OutSamples)
	{
		const int32 InNum = InSamples.Num();
		checkf(InNum == InSampleDelays.Num(), TEXT("InSamples [%d], InSampleDelays [%d] length mismatch"), InNum, InSampleDelays.Num());

		OutSamples.Reset(InNum);
		OutSamples.AddUninitialized(InNum);

		if (InNum != InSampleDelays.Num())
		{
			// Output silence in the case of an error.
			FMemory::Memset(OutSamples.GetData(), 0, sizeof(float) * InNum);
			return;
		}

		const float* InSampleData = InSamples.GetData();
		float* OutSampleData = OutSamples.GetData();

		int32 LeftOver = InNum;
		int32 BufferPos = 0;
		while (LeftOver != 0)
		{
			const int32 NumToProcess = FMath::Min(LeftOver, NumInternalBufferSamples);

			FractionalDelays.Reset(NumToProcess);
			FractionalDelays.AddUninitialized(NumToProcess);

			float* FractionalDelayData = FractionalDelays.GetData();
			FMemory::Memcpy(FractionalDelayData, &InSampleDelays.GetData()[BufferPos], NumToProcess * sizeof(float));
			
			for (int32 i = 0; i < NumToProcess; i++)
			{
				FractionalDelayData[i] -= NumDelaySamples;
			}
			
			ProcessAudioBlock(&InSampleData[BufferPos], FractionalDelays, NumToProcess, &OutSampleData[BufferPos]);

			LeftOver -= NumToProcess;
			BufferPos += NumToProcess;
		}
	}

	void FDynamicDelayAPF::ProcessAudioBlock(const float* InSamples, const FAlignedFloatBuffer& InFractionalDelays, const int32 InNum, float* OutSamples)
	{
		// Make a copy of the delay line w[n - d_int]
		WorkBufferA.Reset(InNum);
		WorkBufferA.AddUninitialized(InNum);
		FMemory::Memcpy(WorkBufferA.GetData(), IntegerDelayLine->InspectSamples(InNum), InNum * sizeof(float));

		// Apply fractional delay
		// w[n - d]
		FractionalDelayLine->ProcessAudio(WorkBufferA, InFractionalDelays, WorkBufferB);

		DelayLineInput.Reset(InNum);
		DelayLineInput.AddUninitialized(InNum);

		TArrayView<float> WorkBufferAView(WorkBufferA.GetData(), InNum);
		TArrayView<float> WorkBufferBView(WorkBufferB.GetData(), InNum);
		TArrayView<float> DelayLineInputView(DelayLineInput.GetData(), InNum);
		TArrayView<const float> InSamplesView(InSamples, InNum);
		TArrayView<float> OutSamplesView(OutSamples, InNum);

		// Get G values to interpolate over
		const float LastG = G.GetNextValue();
		const float CurrG = G.GetNextValue(InNum - 1);

		// WorkBufferA = G * WorkBufferB
		FMemory::Memcpy(WorkBufferA.GetData(), WorkBufferB.GetData(), InNum * sizeof(float));
		ArrayFade(WorkBufferAView, LastG, CurrG);

		// DelayLineInput = InSamples + WorkBufferA
		// w[n] = x[n] + G * w[n - d]
		ArraySum(InSamplesView, WorkBufferAView, DelayLineInputView);

		// Update delay line
		IntegerDelayLine->RemoveSamples(InNum);
		IntegerDelayLine->AddSamples(DelayLineInput.GetData(), InNum);

		// y[n] = -G w[n] + w[n - d]
		ArrayFade(DelayLineInputView, -LastG, -CurrG);
		ArraySum(DelayLineInputView, WorkBufferBView, OutSamplesView);
	}

	void FDynamicDelayAPF::Reset() 
	{
		IntegerDelayLine->ClearSamples();
		IntegerDelayLine->AddZeros(MinDelay - 1);
		FractionalDelayLine->Reset();
	}
}
