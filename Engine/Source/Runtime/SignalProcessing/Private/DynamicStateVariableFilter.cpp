// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/DynamicStateVariableFilter.h"

#include "AudioDefines.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"

namespace Audio
{
	// To check if a number is divisible by 64 without %
	constexpr int32 SmallBlockSizeBitCheck = (1 << 6) - 1;
	constexpr int32 SmallBlockSize = 64;

	void FDynamicStateVariableFilter::Init(const float InSampleRate, const int32 InNumChannels)
	{
		SampleRate = InSampleRate;
		OneOverSampleRate = 1.f / InSampleRate;
		NumChannels = InNumChannels;

		FilterState.Reset();
		FilterState.AddDefaulted(InNumChannels);

		KneePoints.Init(FVector2D(), 2);

		Reset();

		FInlineEnvelopeFollowerInitParams EnvelopeParams;
		EnvelopeParams.SampleRate = InSampleRate;

		Envelope.Init(EnvelopeParams);
	}

	void FDynamicStateVariableFilter::Reset()
	{
		FMemory::Memzero(FilterState.GetData(), sizeof(FSVFState) * NumChannels);
		KeyFilterState = FSVFState();
	}

	void FDynamicStateVariableFilter::UpdateSettings()
	{
		UpdatePreFilterVariables();
		UpdatePostFilterVariables();

		bNeedsUpdate = false;
	}

	void FDynamicStateVariableFilter::UpdatePreFilterVariables()
	{
		KeyVars.K = OneOverQ;

		KeyVars.A1 = 1.f / (1.f + KeyVars.G * (KeyVars.G + KeyVars.K));
		KeyVars.A2 = KeyVars.G * KeyVars.A1;
		KeyVars.A3 = KeyVars.G * KeyVars.A2;
	}

	void FDynamicStateVariableFilter::UpdatePostFilterVariables()
	{
		OutputVars.A0 = FMath::Max(ScaledGainLinear, KINDA_SMALL_NUMBER);

		if (FilterType != EDynamicFilterType::Bell)
		{
			// LowShelf + HighShelf get the same g & k
			OutputVars.G = KeyVars.G / FMath::Sqrt(OutputVars.A0);
			OutputVars.K = OneOverQ;
		}
		else
		{
			OutputVars.G = KeyVars.G;
			OutputVars.K = OneOverQ / (OutputVars.A0);
		}

		OutputVars.A1 = 1.f / (1 + OutputVars.G * (OutputVars.G + OutputVars.K));
		OutputVars.A2 = OutputVars.G * OutputVars.A1;
		OutputVars.A3 = OutputVars.G * OutputVars.A2;

		switch (FilterType)
		{
		default:
		case EDynamicFilterType::Bell:
			OutputVars.M0 = 1.f;
			OutputVars.M1 = OutputVars.K * (OutputVars.A0 * OutputVars.A0 - 1.f);
			OutputVars.M2 = OutputVars.A0 * OutputVars.A0 - 1.f;
			break;

		case EDynamicFilterType::LowShelf:
			OutputVars.M0 = 1.f;
			OutputVars.M1 = OutputVars.K * (OutputVars.A0 - 1.f);
			OutputVars.M2 = OutputVars.A0 * OutputVars.A0 - 1.f;
			break;

		case EDynamicFilterType::HighShelf:
			OutputVars.M0 = OutputVars.A0 * OutputVars.A0;
			OutputVars.M1 = OutputVars.K * (1.f - OutputVars.A0) * OutputVars.A0;
			OutputVars.M2 = 1.f - (OutputVars.A0 * OutputVars.A0);
			break;
		}
	}

	void FDynamicStateVariableFilter::ProcessAudio(const float* InSamples, float* OutSamples, const int32 InNumSamples)
	{
		ProcessAudio(InSamples, OutSamples, InSamples, InNumSamples);
	}

	void FDynamicStateVariableFilter::ProcessAudio(const float* InSamples, float* OutSamples, const float* KeySamples, const int32 InNumSamples)
	{
		if (bNeedsUpdate)
		{
			UpdateSettings();
		}

		switch (FilterType)
		{
		default:
		case Bell:
			ProcessBell(InSamples, OutSamples, KeySamples, InNumSamples);
			break;
		case LowShelf:
			ProcessLowShelf(InSamples, OutSamples, KeySamples, InNumSamples);
			break;
		case HighShelf:
			ProcessHighShelf(InSamples, OutSamples, KeySamples, InNumSamples);
			break;
		}
	}

	void FDynamicStateVariableFilter::ProcessBell(const float* InSamples, float* OutSamples, const float* KeySamples, const int32 InNumSamples)
	{
		const int32 NumFramesInBlock = 64;
		const int32 NumSamplesInBlock = NumFramesInBlock * NumChannels;

		for (int32 StartSampleIndex = 0; StartSampleIndex < InNumSamples; StartSampleIndex += NumSamplesInBlock)
		{
			const int32 EndSampleIndex = FMath::Min(StartSampleIndex + NumSamplesInBlock, InNumSamples);

			float EnvelopeVal = 0.f;

			for (int32 SampleIndex = StartSampleIndex; SampleIndex < EndSampleIndex; SampleIndex += NumChannels)
			{
				float MonoSum = 0.f;

				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					MonoSum += KeySamples[SampleIndex + Channel];
				}

				KeyFilterState.ProcessSample(KeyVars, MonoSum);
				EnvelopeVal = Envelope.ProcessSample(KeyFilterState.V1);
			}

			ScaledGainLinear = CalcGain(EnvelopeVal);
			UpdatePostFilterVariables();

			for (int32 SampleIndex = StartSampleIndex; SampleIndex < EndSampleIndex; SampleIndex += NumChannels)
			{
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					const float InSample = InSamples[SampleIndex + Channel];
					FSVFState& State = FilterState[Channel];

					State.ProcessSample(OutputVars, InSample);

					OutSamples[SampleIndex + Channel] = InSample + (OutputVars.M1 * State.V1);
				}
			}
		}
	}

	void FDynamicStateVariableFilter::ProcessLowShelf(const float* InSamples, float* OutSamples, const float* KeySamples, const int32 InNumSamples)
	{
		const int32 NumFramesInBlock = 64;
		const int32 NumSamplesInBlock = NumFramesInBlock * NumChannels;

		for (int32 StartSampleIndex = 0; StartSampleIndex < InNumSamples; StartSampleIndex += NumSamplesInBlock)
		{
			const int32 EndSampleIndex = FMath::Min(StartSampleIndex + NumSamplesInBlock, InNumSamples);

			float EnvelopeVal = 0.f;

			for (int32 SampleIndex = StartSampleIndex; SampleIndex < EndSampleIndex; SampleIndex += NumChannels)
			{
				float MonoSum = 0.f;

				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					MonoSum += KeySamples[SampleIndex + Channel];
				}

				KeyFilterState.ProcessSample(KeyVars, MonoSum);
				EnvelopeVal = Envelope.ProcessSample(KeyFilterState.V2);
			}

			ScaledGainLinear = CalcGain(EnvelopeVal);
			UpdatePostFilterVariables();

			for (int32 SampleIndex = StartSampleIndex; SampleIndex < EndSampleIndex; SampleIndex += NumChannels)
			{
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					const float InSample = InSamples[SampleIndex + Channel];
					FSVFState& State = FilterState[Channel];

					State.ProcessSample(OutputVars, InSample);

					OutSamples[SampleIndex + Channel] = InSample + (OutputVars.M1 * State.V1) + (OutputVars.M2 * State.V2);
				}
			}
		}
	}

	void FDynamicStateVariableFilter::ProcessHighShelf(const float* InSamples, float* OutSamples, const float* KeySamples, const int32 InNumSamples)
	{
		const int32 NumFramesInBlock = 64;
		const int32 NumSamplesInBlock = NumFramesInBlock * NumChannels;

		for (int32 StartSampleIndex = 0; StartSampleIndex < InNumSamples; StartSampleIndex += NumSamplesInBlock)
		{
			const int32 EndSampleIndex = FMath::Min(StartSampleIndex + NumSamplesInBlock, InNumSamples);

			float EnvelopeVal = 0.f;

			for (int32 SampleIndex = StartSampleIndex; SampleIndex < EndSampleIndex; SampleIndex += NumChannels)
			{
				float MonoSum = 0.f;

				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					MonoSum += KeySamples[SampleIndex + Channel];
				}

				KeyFilterState.ProcessSample(KeyVars, MonoSum);
				EnvelopeVal = Envelope.ProcessSample((KeyVars.K * KeyFilterState.V1) - KeyFilterState.V2);
			}

			ScaledGainLinear = CalcGain(EnvelopeVal);
			UpdatePostFilterVariables();

			for (int32 SampleIndex = StartSampleIndex; SampleIndex < EndSampleIndex; SampleIndex += NumChannels)
			{
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					const float InSample = InSamples[SampleIndex + Channel];
					FSVFState& State = FilterState[Channel];

					State.ProcessSample(OutputVars, InSample);

					OutSamples[SampleIndex + Channel] = (OutputVars.M0 * InSample) + (OutputVars.M1 * State.V1) + (OutputVars.M2 * State.V2);
				}
			}
		}
	}

	float FDynamicStateVariableFilter::CalcGain(const float KeyEnvelope)
	{
		const float EnvDb = ConvertToDecibels(KeyEnvelope);

		float DynamicGain = 0.f;

		if (EnvDb > ThresholdDb - HalfKnee)
		{
			float LocalSlopeFactor = SlopeFactor;

			if (HalfKnee > 0.0f && EnvDb < ThresholdDb + HalfKnee)
			{
				// Setup the knee for interpolation. Don't allow the top knee point to exceed 0.0
				KneePoints[0].X = ThresholdDb - HalfKnee;
				KneePoints[1].X = FMath::Min(ThresholdDb + HalfKnee, 0.0f);

				KneePoints[0].Y = 0.0f;
				KneePoints[1].Y = LocalSlopeFactor;

				LocalSlopeFactor = LagrangianInterpolation(KneePoints, EnvDb);
			}

			DynamicGain = FMath::Min(LocalSlopeFactor * (ThresholdDb - EnvDb), 0.f);

			// Positive dynamic gain is equivalent to just taking whatever gain reduction we'd do in a compressor and flipping it
			DynamicGain = FMath::Max(DynamicGain, -FMath::Abs(DynamicRangeDb)) * FMath::Sign(-DynamicRangeDb);
		}

		// scale gain by .5 here to avoid having to do Sqrt later
		return ConvertToLinear(0.5f * (GainDb + DynamicGain));
	}

	void FDynamicStateVariableFilter::SetFrequency(const float InFrequency)
	{
		if (FMath::IsNearlyEqual(Frequency, InFrequency) == false)
		{
			Frequency = FMath::Clamp(InFrequency, MIN_FILTER_FREQUENCY, (SampleRate * 0.5f) - 1.0f);

			KeyVars.G = FMath::Tan(PI * Frequency * OneOverSampleRate);

			bNeedsUpdate = true;
		}
	}

	void FDynamicStateVariableFilter::SetQ(const float InQ)
	{
		if (FMath::IsNearlyEqual(InQ, Q) == false)
		{
			constexpr float Max_Q = 50.f;
			Q = FMath::Clamp(InQ, KINDA_SMALL_NUMBER, Max_Q);
			OneOverQ = 1.f / Q;
			bNeedsUpdate = true;
		}
	}

	void FDynamicStateVariableFilter::SetGain(const float InGain)
	{
		if (FMath::IsNearlyEqual(GainDb, InGain) == false)
		{
			GainDb = FMath::Clamp(InGain, MIN_VOLUME_DECIBELS, -(MIN_VOLUME_DECIBELS));
			bNeedsUpdate = true;
		}
	}

	void FDynamicStateVariableFilter::SetFilterType(const EDynamicFilterType InFilterType)
	{
		if (FilterType != InFilterType)
		{
			bNeedsUpdate = true;
			FilterType = InFilterType;
		}
	}

	void FDynamicStateVariableFilter::SetAnalog(const bool bInAnalog)
	{
		Envelope.SetAnalog(bInAnalog);
	}

	void FDynamicStateVariableFilter::SetAttackTime(const float InAttackTime)
	{
		Envelope.SetAttackTime(FMath::Clamp(InAttackTime, 0.f, UE_BIG_NUMBER));
	}

	void FDynamicStateVariableFilter::SetReleaseTime(const float InReleaseTime)
	{
		Envelope.SetReleaseTime(FMath::Clamp(InReleaseTime, 0.f, UE_BIG_NUMBER));
	}

	void FDynamicStateVariableFilter::SetThreshold(const float InThresholdDb)
	{
		ThresholdDb = FMath::Clamp(InThresholdDb, MIN_VOLUME_DECIBELS, 0.f);
	}

	void FDynamicStateVariableFilter::SetEnvMode(const EPeakMode::Type InMode)
	{
		Envelope.SetMode(InMode);
	}

	void FDynamicStateVariableFilter::SetDynamicRange(const float InDynamicRange)
	{
		DynamicRangeDb = FMath::Clamp(InDynamicRange, MIN_VOLUME_DECIBELS, -(MIN_VOLUME_DECIBELS));
	}

	void FDynamicStateVariableFilter::SetRatio(const float InRatio)
	{
		Ratio = FMath::Clamp(InRatio, 1.f + KINDA_SMALL_NUMBER, 100.f);
		SlopeFactor = 1.0f - 1.0f / Ratio;
	}

	void FDynamicStateVariableFilter::SetKnee(const float InKnee)
	{
		Knee = FMath::Clamp(InKnee, 0.f, -(MIN_VOLUME_DECIBELS));
		HalfKnee = Knee * 0.5f;
	}

	float FDynamicStateVariableFilter::GetGainReduction()
	{
		return ScaledGainLinear * 2.f;
	}

	void FDynamicStateVariableFilter::FSVFState::ProcessSample(const FSVFCoefficients& Coeffs, const float InSample)
	{
		V3 = InSample - Z2;
		V1 = (Coeffs.A1 * Z1) + (Coeffs.A2 * V3);
		V2 = Z2 + (Coeffs.A2 * Z1) + Coeffs.A3 * V3;

		Z1 = 2.f * V1 - Z1;
		Z2 = 2.f * V2 - Z2;
	}
}