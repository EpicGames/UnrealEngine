// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/DynamicsProcessor.h"
#include "DSP/FloatArrayMath.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SignalProcessingModule.h"

namespace Audio
{
	FDynamicsProcessor::FDynamicsProcessor()
		: ProcessingMode(EDynamicsProcessingMode::Compressor)
		, SlopeFactor(0.0f)
		, EnvelopeFollowerPeakMode(EPeakMode::Peak)
		, LookaheadDelayMsec(10.0f)
		, AttackTimeMsec(20.0f)
		, ReleaseTimeMsec(1000.0f)
		, ThresholdDb(-6.0f)
		, Ratio(1.0f)
		, HalfKneeBandwidthDb(5.0f)
		, InputGain(1.0f)
		, OutputGain(1.0f)
		, KeyGain(1.0f)
		, SampleRate(48000.f)
		, LinkMode(EDynamicsProcessorChannelLinkMode::Disabled)
		, bIsAnalogMode(true)
		, bKeyAuditionEnabled(false)
		, bKeyHighshelfEnabled(false)
		, bKeyLowshelfEnabled(false)
	{
		// The knee will have 2 points
		KneePoints.Init(FKneePoint(), 2);
		CalculateSlope();
	}

	FDynamicsProcessor::~FDynamicsProcessor()
	{
	}

	void FDynamicsProcessor::Init(const float InSampleRate, const int32 InNumChannels)
	{
		check(InSampleRate > 0.f);
		SampleRate = InSampleRate;

		SetNumChannels(InNumChannels);
		SetKeyNumChannels(InNumChannels);

		for (int32 Channel = 0; Channel < InNumChannels; ++Channel)
		{
			EnvFollower[Channel].Init(FInlineEnvelopeFollowerInitParams{SampleRate, AttackTimeMsec, ReleaseTimeMsec, EnvelopeFollowerPeakMode, bIsAnalogMode});
		}

		// Initialize key filters
		InputLowshelfFilter.Init(SampleRate, InNumChannels, EBiquadFilter::LowShelf);
		InputHighshelfFilter.Init(SampleRate, InNumChannels, EBiquadFilter::HighShelf);

		DetectorOuts.Reset();
		DetectorOuts.AddZeroed(InNumChannels);

		Gain.Reset();
		Gain.AddZeroed(InNumChannels);
	}

	int32 FDynamicsProcessor::GetNumChannels() const
	{
		return Gain.Num();
	}

	int32 FDynamicsProcessor::GetKeyNumChannels() const
	{
		return EnvFollower.Num();
	}

	float Audio::FDynamicsProcessor::GetMaxLookaheadMsec() const
	{
		return MaxLookaheadMsec;
	}
	
	void FDynamicsProcessor::SetLookaheadMsec(const float InLookAheadMsec)
	{
		LookaheadDelayMsec = FMath::Min(InLookAheadMsec, MaxLookaheadMsec);
		const int32 NumDelayFrames = GetNumDelayFrames();
		for (int32 Channel = 0; Channel < LookaheadDelay.Num(); ++Channel)
		{
			LookaheadDelay[Channel].SetDelayLengthSamples(NumDelayFrames);
		}
	}

	void FDynamicsProcessor::SetAttackTime(const float InAttackTimeMsec)
	{
		AttackTimeMsec = InAttackTimeMsec;
		for (int32 Channel = 0; Channel < EnvFollower.Num(); ++Channel)
		{
			EnvFollower[Channel].SetAttackTime(InAttackTimeMsec);
		}
	}

	void FDynamicsProcessor::SetReleaseTime(const float InReleaseTimeMsec)
	{
		ReleaseTimeMsec = InReleaseTimeMsec;
		for (int32 Channel = 0; Channel < EnvFollower.Num(); ++Channel)
		{
			EnvFollower[Channel].SetReleaseTime(InReleaseTimeMsec);
		}
	}

	void FDynamicsProcessor::SetThreshold(const float InThresholdDb)
	{
		ThresholdDb = InThresholdDb;
		CalculateKnee();
	}

	void FDynamicsProcessor::SetRatio(const float InCompressionRatio)
	{
		// Don't let the compression ratio be 0.0!
		Ratio = FMath::Max(InCompressionRatio, SMALL_NUMBER);
		CalculateSlope();
	}

	void FDynamicsProcessor::SetKneeBandwidth(const float InKneeBandwidthDb)
	{
		HalfKneeBandwidthDb = 0.5f * InKneeBandwidthDb;
		CalculateKnee();
	}

	void FDynamicsProcessor::SetInputGain(const float InInputGainDb)
	{
		InputGain = ConvertToLinear(InInputGainDb);
	}

	void FDynamicsProcessor::SetKeyAudition(const bool InAuditionEnabled)
	{
		bKeyAuditionEnabled = InAuditionEnabled;
	}

	void FDynamicsProcessor::SetKeyGain(const float InKeyGain)
	{
		KeyGain = ConvertToLinear(InKeyGain);
	}

	void FDynamicsProcessor::SetKeyHighshelfCutoffFrequency(const float InCutoffFreq)
	{
		InputHighshelfFilter.SetFrequency(InCutoffFreq);
	}

	void FDynamicsProcessor::SetKeyHighshelfEnabled(const bool bInEnabled)
	{
		bKeyHighshelfEnabled = bInEnabled;
	}

	void FDynamicsProcessor::SetKeyHighshelfGain(const float InGainDb)
	{
		InputHighshelfFilter.SetGainDB(InGainDb);
	}

	void FDynamicsProcessor::SetKeyLowshelfCutoffFrequency(const float InCutoffFreq)
	{
		InputLowshelfFilter.SetFrequency(InCutoffFreq);
	}

	void FDynamicsProcessor::SetKeyLowshelfEnabled(const bool bInEnabled)
	{
		bKeyLowshelfEnabled = bInEnabled;
	}

	void FDynamicsProcessor::SetKeyLowshelfGain(const float InGainDb)
	{
		InputLowshelfFilter.SetGainDB(InGainDb);
	}

	void FDynamicsProcessor::SetKeyNumChannels(const int32 InNumChannels)
	{
		if (InNumChannels != EnvFollower.Num())
		{
			EnvFollower.Reset();

			for (int32 Channel = 0; Channel < InNumChannels; ++Channel)
			{
				EnvFollower.Emplace(FInlineEnvelopeFollowerInitParams{SampleRate, AttackTimeMsec, ReleaseTimeMsec, EnvelopeFollowerPeakMode, bIsAnalogMode});
			}
		}

		if (InNumChannels != InputLowshelfFilter.GetNumChannels())
		{
			InputLowshelfFilter.Init(SampleRate, InNumChannels, EBiquadFilter::LowShelf);
		}

		if (InNumChannels != InputHighshelfFilter.GetNumChannels())
		{
			InputHighshelfFilter.Init(SampleRate, InNumChannels, EBiquadFilter::HighShelf);
		}

		if (InNumChannels != DetectorOuts.Num())
		{
			DetectorOuts.Reset();
			DetectorOuts.AddZeroed(InNumChannels);
		}
	}

	void FDynamicsProcessor::SetOutputGain(const float InOutputGainDb)
	{
		OutputGain = ConvertToLinear(InOutputGainDb);
	}

	void FDynamicsProcessor::SetChannelLinkMode(const EDynamicsProcessorChannelLinkMode InLinkMode)
	{
		LinkMode = InLinkMode;
	}

	void FDynamicsProcessor::SetAnalogMode(const bool bInIsAnalogMode)
	{
		bIsAnalogMode = bInIsAnalogMode;
		for (int32 Channel = 0; Channel < EnvFollower.Num(); ++Channel)
		{
			EnvFollower[Channel].SetAnalog(bInIsAnalogMode);
		}
	}

	void FDynamicsProcessor::SetNumChannels(const int32 InNumChannels)
	{
		if (InNumChannels != Gain.Num())
		{
			Gain.Reset();
			Gain.AddZeroed(InNumChannels);
		}

		if (InNumChannels != LookaheadDelay.Num())
		{
			// Initialize lookahead buffers
			constexpr int32 DelayInternalBufferSize = 32;
			const int32 MaxNumDelayFrames = FMath::Max(1, FMath::CeilToInt(MaxLookaheadMsec * SampleRate / 1000.0f));
			const int32 NumDelayFrames = GetNumDelayFrames();

			LookaheadDelay.Reset(InNumChannels);

			for (int32 Channel = 0; Channel < InNumChannels; ++Channel)
			{
				LookaheadDelay.Emplace(MaxNumDelayFrames, NumDelayFrames, DelayInternalBufferSize);
			}
		}
	}

	void FDynamicsProcessor::SetPeakMode(const EPeakMode::Type InEnvelopeFollowerModeType)
	{
		EnvelopeFollowerPeakMode = InEnvelopeFollowerModeType;
		for (int32 Channel = 0; Channel < EnvFollower.Num(); ++Channel)
		{
			EnvFollower[Channel].SetMode(EnvelopeFollowerPeakMode);
		}
	}

	void FDynamicsProcessor::SetProcessingMode(const EDynamicsProcessingMode::Type InProcessingMode)
	{
		ProcessingMode = InProcessingMode;
		CalculateSlope();
	}

	void FDynamicsProcessor::ProcessAudioFrame(const float* InFrame, float* OutFrame, const float* InKeyFrame)
	{
		const bool bKeyIsInput = InFrame == InKeyFrame;
		if (ProcessKeyFrame(InKeyFrame, OutFrame, bKeyIsInput))
		{
			const int32 NumChannels = GetNumChannels();
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				// Write and read into the look ahead delay line.
				// We apply the compression output of the direct input to the output of this delay line
				// This way sharp transients can be "caught" with the gain.
				float LookaheadOutput = LookaheadDelay[Channel].ProcessAudioSample(InFrame[Channel]);

				// Write into the output with the computed gain value
				OutFrame[Channel] = Gain[Channel] * LookaheadOutput * OutputGain * InputGain;
			}
		}
	}

	void FDynamicsProcessor::ProcessAudioFrame(const float* InFrame, float* OutFrame, const float* InKeyFrame, float* OutGain)
	{
		check(OutGain != nullptr);

		const bool bKeyIsInput = InFrame == InKeyFrame;
		if (ProcessKeyFrame(InKeyFrame, OutFrame, bKeyIsInput))
		{
			const int32 NumChannels = GetNumChannels();
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				// Write and read into the look ahead delay line.
				// We apply the compression output of the direct input to the output of this delay line
				// This way sharp transients can be "caught" with the gain.
				float LookaheadOutput = LookaheadDelay[Channel].ProcessAudioSample(InFrame[Channel]);

				// Write into the output with the computed gain value
				OutFrame[Channel] = Gain[Channel] * LookaheadOutput * OutputGain * InputGain;
				// Also write the output gain value
				OutGain[Channel] = Gain[Channel];
			}
		}
	}


	void FDynamicsProcessor::ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer, const float* InKeyBuffer, float* OutEnvelope)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicsProcessor::ProcessAudio_Interleaved);

		constexpr int32 MaxNumChannels = 8;
		constexpr int32 MinNumSubbufferFrames = 32;
		constexpr int32 NumSubbufferSamples = MaxNumChannels * MinNumSubbufferFrames;
		static_assert((NumSubbufferSamples * sizeof(float) * 4) <= 4096, "Statically allocated deinterleave buffers are clamped to be 4096 to protect against running out of stack memory.");

		// Stack allocated intermediate buffers to hold deinterleave audio
		float* DeinterleaveInput[MaxNumChannels];
		float InputWorkBuffer[NumSubbufferSamples];

		float* DeinterleaveOutput[MaxNumChannels];
		float OutputWorkBuffer[NumSubbufferSamples];

		float* DeinterleaveKey[MaxNumChannels];
		float KeyWorkBuffer[NumSubbufferSamples];

		float* DeinterleaveEnvelope[MaxNumChannels];
		float EnvelopeWorkBuffer[NumSubbufferSamples];


		const int32 NumChannels = GetNumChannels();
		const int32 KeyNumChannels = GetKeyNumChannels();
		const int32 NumInputFrames = InNumSamples / NumChannels;
		const int32 NumSubbufferFrames = (NumSubbufferSamples / FMath::Max(1, FMath::Max(NumChannels, KeyNumChannels)));

		check(nullptr != InBuffer);
		check(nullptr != OutBuffer);
		check(KeyNumChannels <= MaxNumChannels);
		check(NumChannels <= MaxNumChannels);
		check(NumSubbufferFrames >= MinNumSubbufferFrames);

		// Initialize deinterleave buffer pointers
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
		{
			DeinterleaveInput[ChannelIndex] = &InputWorkBuffer[ChannelIndex * NumSubbufferFrames];
			DeinterleaveOutput[ChannelIndex] = &OutputWorkBuffer[ChannelIndex * NumSubbufferFrames];
			DeinterleaveEnvelope[ChannelIndex] = &EnvelopeWorkBuffer[ChannelIndex * NumSubbufferFrames];
		};
		for (int32 KeyChannelIndex = 0; KeyChannelIndex < KeyNumChannels; KeyChannelIndex++)
		{
			DeinterleaveKey[KeyChannelIndex] = &KeyWorkBuffer[KeyChannelIndex * NumSubbufferFrames];
		}

		// Run dyanmics processor on deinterleaved subbuffers. 
		for (int32 FrameIndex = 0; FrameIndex < NumInputFrames; FrameIndex += NumSubbufferFrames)
		{
			const int32 NumFramesThisIteration = FMath::Min(NumSubbufferFrames, NumInputFrames - FrameIndex);
			const int32 InterleavedSampleIndex = FrameIndex * NumChannels;

			ArrayDeinterleave(&InBuffer[InterleavedSampleIndex], DeinterleaveInput, NumFramesThisIteration, NumChannels);

			if (InKeyBuffer)
			{
				const int32 KeyInterleavedSampleIndex = FrameIndex * KeyNumChannels;
				ArrayDeinterleave(&InKeyBuffer[KeyInterleavedSampleIndex], DeinterleaveKey, NumFramesThisIteration, KeyNumChannels);
				
				ProcessAudio(DeinterleaveInput, NumFramesThisIteration, DeinterleaveOutput, DeinterleaveKey, DeinterleaveEnvelope);
			}
			else
			{
				ProcessAudio(DeinterleaveInput, NumFramesThisIteration, DeinterleaveOutput, nullptr /* InKeyBuffers */, DeinterleaveEnvelope);
			}

			// Interleave output results
			if (OutEnvelope)
			{
				ArrayInterleave(DeinterleaveEnvelope, &OutEnvelope[InterleavedSampleIndex], NumFramesThisIteration, NumChannels);
			}

			ArrayInterleave(DeinterleaveOutput, &OutBuffer[InterleavedSampleIndex], NumFramesThisIteration, NumChannels);
		}
	}

	void FDynamicsProcessor::ProcessAudio(const float* const* const InBuffers, const int32 InNumFrames, float* const* OutBuffers, const float* const* const InKeyBuffers, float* const* OutEnvelopes)
	{
		// There are rare cases where the number of output channels is mutated but the key count is
		// not (such as during device swap, adding & removing endpoints such as controllers, etc.).
		// For now, ensure key number of channels matches that of the output number of channels.
		const int32 NumChannels = GetNumChannels();
		if (GetKeyNumChannels() > NumChannels)
		{
			SetKeyNumChannels(NumChannels);
		}

		ProcessAudioDeinterleaveInternal(InBuffers, InNumFrames, OutBuffers, InKeyBuffers, OutEnvelopes);
	}

	void FDynamicsProcessor::ProcessAudioDeinterleaveInternal(const float* const* const InBuffers, const int32 InNumFrames, float* const* OutBuffers, const float* const* const InKeyBuffers, float* const* OutEnvelopes)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicsProcessor::ProcessAudioDeinterleaveInternal);
		// NOTE: This is a useful test, but it isn't comprehensive. It can't be sure the passed 
		// in arrays of pointers have a matching number of pointers to the channel counts. It
		// will have to do for now.
		check(nullptr != InBuffers && nullptr != InBuffers[0]);
		check(nullptr != OutBuffers && nullptr != OutBuffers[0]);
		check(nullptr != OutEnvelopes && nullptr != OutEnvelopes[0]);

		// NOTE: The OutBuffers are used as scratch buffers for any processing applied
		// to the key. Callers must provide 
		const int32 KeyNumChannels = GetKeyNumChannels();
		const int32 NumChannels = GetNumChannels();

		const float* const* KeySources = InKeyBuffers ? InKeyBuffers : InBuffers;
		float* const* KeyWorkBuffers = OutBuffers;

		// Process Key
		if (bKeyLowshelfEnabled)
		{
			InputLowshelfFilter.ProcessAudio(KeySources, InNumFrames, KeyWorkBuffers);
			KeySources = KeyWorkBuffers;
		}

		if (bKeyHighshelfEnabled)
		{
			InputHighshelfFilter.ProcessAudio(KeySources, InNumFrames, KeyWorkBuffers);
			KeySources = KeyWorkBuffers;
		}

		const bool bKeyIsInput = !InKeyBuffers || InKeyBuffers == InBuffers;
		float DetectorGain = InputGain;

		// Apply key gain only if detector is key (not input)
		if (!bKeyIsInput)
		{
			DetectorGain *= KeyGain;
		}

		if (bKeyAuditionEnabled)
		{
			if (KeySources != KeyWorkBuffers)
			{
				// This means we have not done either filter above, so we have not  
				// "moved" the key input to the output buffer. We have to do it here. 
				for (int32 Channel = 0; Channel < KeyNumChannels; ++Channel)
				{
					FMemory::Memcpy(OutBuffers[Channel], KeySources[Channel], sizeof(float) * InNumFrames);
				}
			}
			// we just have to zero out any channels that are not in the key signal
			for (int32 Channel = KeyNumChannels; Channel < NumChannels; ++Channel)
			{
				FMemory::Memset(OutBuffers[Channel], 0, sizeof(float) * InNumFrames);
			}

			// I WOULD zero or 1 the "OutEnvelope", but that is not what the original does!
			return;
		}

		for (int32 Channel = 0; Channel < KeyNumChannels; ++Channel)
		{
			// apply detector gain to key
			Audio::ArrayMultiplyByConstant(TArrayView<const float>(KeySources[Channel], InNumFrames), DetectorGain, TArrayView<float>(KeyWorkBuffers[Channel], InNumFrames));
			// EnvelopeFollow key
			TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicsProcessor::ProcessAudioDeinterleaveInternal_EnvFollower);
			EnvFollower[Channel].ProcessBuffer(KeyWorkBuffers[Channel], InNumFrames, OutEnvelopes[Channel]);
		}

		int32 NumGainChannels = 1;
		switch (LinkMode)
		{
		case EDynamicsProcessorChannelLinkMode::Average:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicsProcessor::ProcessAudioDeinterleaveInternal_LinkModeAverage);
			// average all key channels
			TArrayView<float> Out = TArrayView<float>(OutEnvelopes[0], InNumFrames);
			for (int32 KeyChannel = 1; KeyChannel < KeyNumChannels; ++KeyChannel)
			{
				ArraySum(TArrayView<const float>(OutEnvelopes[KeyChannel], InNumFrames), Out, Out);
			}
			ArrayMultiplyByConstant(Out, 1.0f / (float)KeyNumChannels, Out);
			// convert magnitude to db
			ArrayMagnitudeToDecibelInPlace(Out, -96.0f);
			// compute 1 gain and use for all channels ...	const float ComputedGain = ComputeGain(DetectorOutLinkedDb);
			ComputeGains(OutEnvelopes[0], InNumFrames);
			NumGainChannels = 1;
		}
		break;

		case EDynamicsProcessorChannelLinkMode::Peak:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicsProcessor::ProcessAudioDeinterleaveInternal_LinkModePeak);
			// detect peak across all key channels
			TArrayView<float> Out = TArrayView<float>(OutEnvelopes[0], InNumFrames);
			for (int32 KeyChannel = 1; KeyChannel < KeyNumChannels; ++KeyChannel)
			{
				ArrayMax(TArrayView<const float>(OutEnvelopes[KeyChannel], InNumFrames), Out, Out);
			}
			// convert magnitude to db
			ArrayMagnitudeToDecibelInPlace(Out, -96.0f);
			// compute 1 gain and use for all channels ...	const float ComputedGain = ComputeGain(DetectorOutLinkedDb);
			ComputeGains(OutEnvelopes[0], InNumFrames);
			NumGainChannels = 1;
		}
		break;

		case EDynamicsProcessorChannelLinkMode::Disabled:
		default:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicsProcessor::ProcessAudioDeinterleaveInternal_LinkModeDisabled);
			for (int32 Channel = 0; Channel < KeyNumChannels; ++Channel)
			{
				// convert magnitude to db
				ArrayMagnitudeToDecibelInPlace(TArrayView<float>(OutEnvelopes[Channel],InNumFrames), -96.0f);
				// compute 1 gain and use for all channels ...	const float ComputedGain = ComputeGain(DetectorOutLinkedDb);
				ComputeGains(OutEnvelopes[Channel], InNumFrames);
			}
			NumGainChannels = KeyNumChannels;
		}
		break;
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicsProcessor::ProcessAudioDeinterleaveInternal_ApplyGain);

			const float OutAndInGain = OutputGain * InputGain;
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				int32 GainChannel = Channel % NumGainChannels;

				TArrayView<const float> InBufferView = MakeArrayView<const float>(InBuffers[Channel], InNumFrames);
				TArrayView<float> OutBufferView = MakeArrayView<float>(OutBuffers[Channel], InNumFrames);
				TArrayView<const float> EnvelopeBufferView = MakeArrayView<const float>(OutEnvelopes[GainChannel], InNumFrames);

				// copy input to look ahead delay write position
				LookaheadDelay[Channel].ProcessAudio(InBufferView, OutBufferView);
				
				// apply compression gain & output gain & input gain to look ahead delay read position and save to output
				ArrayMultiplyInPlace(EnvelopeBufferView, OutBufferView);
				ArrayMultiplyByConstantInPlace(OutBufferView, OutAndInGain);
			}
		}
	}

	bool FDynamicsProcessor::ProcessKeyFrame(const float* InKeyFrame, float* OutFrame, bool bKeyIsInput)
	{
		// Get detector outputs
		const float* KeyIn = InKeyFrame;

		const int32 KeyNumChannels = GetKeyNumChannels();
		const int32 NumChannels = GetNumChannels();
		if (KeyNumChannels > 0)
		{
			if (bKeyLowshelfEnabled)
			{
				InputLowshelfFilter.ProcessAudioFrame(KeyIn, DetectorOuts.GetData());
				KeyIn = DetectorOuts.GetData();
			}

			if (bKeyHighshelfEnabled)
			{
				InputHighshelfFilter.ProcessAudioFrame(KeyIn, DetectorOuts.GetData());
				KeyIn = DetectorOuts.GetData();
			}
		}

		float DetectorGain = InputGain;

		// Apply key gain only if detector is key (not input)
		if (!bKeyIsInput)
		{
			DetectorGain *= KeyGain;
		}

		if (bKeyAuditionEnabled)
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const int32 KeyIndex = Channel % KeyNumChannels;
				OutFrame[Channel] = DetectorGain * KeyIn[KeyIndex];
			}

			return false;
		}

		for (int32 Channel = 0; Channel < KeyNumChannels; ++Channel)
		{
			DetectorOuts[Channel] = EnvFollower[Channel].ProcessSample(DetectorGain * KeyIn[Channel]);
		}

		switch (LinkMode)
		{
			case EDynamicsProcessorChannelLinkMode::Average:
			{
				float KeyOutLinked = 0.0f;
				for (int32 Channel = 0; Channel < KeyNumChannels; ++Channel)
				{
					KeyOutLinked += DetectorOuts[Channel];
				}
				KeyOutLinked /= static_cast<float>(KeyNumChannels);
				const float DetectorOutLinkedDb = ConvertToDecibels(KeyOutLinked);
				const float ComputedGain = ComputeGain(DetectorOutLinkedDb);
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					Gain[Channel] = ComputedGain;
				}
			}
			break;

			case EDynamicsProcessorChannelLinkMode::Peak:
			{
				float KeyOutLinked = FMath::Max<float>(DetectorOuts);
				const float KeyOutLinkedDb = ConvertToDecibels(KeyOutLinked);
				const float ComputedGain = ComputeGain(KeyOutLinkedDb);
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					Gain[Channel] = ComputedGain;
				}
			}
			break;

			case EDynamicsProcessorChannelLinkMode::Disabled:
			default:
			{
				// Compute gain individually per channel and wrap if
				// channel count is greater than key channel count.
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					const int32 KeyIndex = Channel % KeyNumChannels;
					float ChannelGain = DetectorOuts[KeyIndex];

					const float KeyOutDb = ConvertToDecibels(ChannelGain);
					const float ComputedGain = ComputeGain(KeyOutDb);
					Gain[KeyIndex] = ComputedGain;
				}
			}
			break;
		}

		return true;
	}

	bool FDynamicsProcessor::IsInProcessingThreshold(const float InEnvFollowerDb) const
	{
		if (ProcessingMode == EDynamicsProcessingMode::UpwardsCompressor)
		{
			return HalfKneeBandwidthDb >= 0.0f
				&& InEnvFollowerDb < (ThresholdDb - HalfKneeBandwidthDb)
				&& InEnvFollowerDb > (ThresholdDb + HalfKneeBandwidthDb);
		}

		return HalfKneeBandwidthDb >= 0.0f
			&& InEnvFollowerDb > (ThresholdDb - HalfKneeBandwidthDb)
			&& InEnvFollowerDb < (ThresholdDb + HalfKneeBandwidthDb);
	}

	float FDynamicsProcessor::ComputeGain(const float InEnvFollowerDb)
	{
		// If we are in the range of compression
		float SlopeThisSample = SlopeFactor;
		if (IsInProcessingThreshold(InEnvFollowerDb))
		{
			// The knee calculation adjusts the slope to use via lagrangian interpolation through the slope
			float Lagrangian = (InEnvFollowerDb - KneePoints[1].X) / Denominator0Minus1;
			SlopeThisSample = Lagrangian * KneePoints[0].Y;
			Lagrangian = (InEnvFollowerDb - KneePoints[0].X) / Denominator1Minus0;
			SlopeThisSample += Lagrangian * KneePoints[1].Y;

		}

		float OutputGainDb = SlopeThisSample * (ThresholdDb - InEnvFollowerDb);

		if (ProcessingMode == EDynamicsProcessingMode::UpwardsCompressor)
		{
			// if left unchecked Upwards compression will try to apply infinite gain
			OutputGainDb = FMath::Clamp(OutputGainDb, 0.f, UpwardsCompressionMaxGain);
		}
		else
		{
			OutputGainDb = FMath::Min(0.f, OutputGainDb);
		}

		return ConvertToLinear(OutputGainDb);
	}

	
	void FDynamicsProcessor::ComputeGains(float* InEnvFollowerDbOutGain, const int32 InNumSamples)
	{
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
		{
			float InEnvFollowerDb = InEnvFollowerDbOutGain[SampleIndex];

			float SlopeThisSample = SlopeFactor;
			if (IsInProcessingThreshold(InEnvFollowerDb))
			{
				float Lagrangian = (InEnvFollowerDb - KneePoints[1].X) / Denominator0Minus1;
				SlopeThisSample = Lagrangian * KneePoints[0].Y;
				Lagrangian = (InEnvFollowerDb - KneePoints[0].X) / Denominator1Minus0;
				SlopeThisSample += Lagrangian * KneePoints[1].Y;
			};

			float OutputGainDb = SlopeThisSample * (ThresholdDb - InEnvFollowerDb);

			if (ProcessingMode == EDynamicsProcessingMode::UpwardsCompressor)
			{
				// if left unchecked Upwards compression will try to apply infinite gain
				OutputGainDb = FMath::Clamp(OutputGainDb, 0.f, UpwardsCompressionMaxGain);
				InEnvFollowerDbOutGain[SampleIndex] = ConvertToLinear(OutputGainDb);
			}
			else
			{
				InEnvFollowerDbOutGain[SampleIndex] = OutputGainDb > -UE_SMALL_NUMBER ? 1.0f : ConvertToLinear(OutputGainDb);
			}
		}
	}

	int32 FDynamicsProcessor::GetNumDelayFrames() const
	{
		checkf(LookaheadDelayMsec <= MaxLookaheadMsec, TEXT("An lookahead delay of %fms exceeds maximum lookahead delay of %fms"), LookaheadDelayMsec, MaxLookaheadMsec)
		return FMath::Max(0, FMath::CeilToInt(LookaheadDelayMsec * SampleRate / 1000.0f));
	}

	void FDynamicsProcessor::CalculateSlope()
	{
		SlopeFactor = 0.0f;
		// Depending on the mode, we define the "slope". 
		switch (ProcessingMode)
		{
		default:

			// Compressors smoothly reduce the gain as the gain gets louder
			// CompressionRatio -> Inifinity is a limiter
			// Upwards compression applies gain when below a threshold, but uses the same slope
		case EDynamicsProcessingMode::UpwardsCompressor:
		case EDynamicsProcessingMode::Compressor:
			SlopeFactor = 1.0f - 1.0f / Ratio;
			break;

			// Limiters do nothing until it hits the threshold then clamps the output hard
		case EDynamicsProcessingMode::Limiter:
			SlopeFactor = 1.0f;
			break;

			// Expanders smoothly increase the gain as the gain gets louder
			// CompressionRatio -> Infinity is a gate
		case EDynamicsProcessingMode::Expander:
			SlopeFactor = 1.0f / Ratio - 1.0f;
			break;

			// Gates are opposite of limiter. They stop sound (stop gain) until the threshold is hit
		case EDynamicsProcessingMode::Gate:
			SlopeFactor = -1.0f;
			break;
		}
		CalculateKnee();
	}

	void FDynamicsProcessor::CalculateKnee()
	{
		// Setup the knee for interpolation. Don't allow the top knee point to exceed 0.0
		KneePoints[0].X = ThresholdDb - HalfKneeBandwidthDb;
		KneePoints[1].X = FMath::Min(ThresholdDb + HalfKneeBandwidthDb, 0.0f);

		KneePoints[0].Y = 0.0f;
		KneePoints[1].Y = SlopeFactor;

		// These next few calculations let us optimize out the call to 
		// the LagrangeInterpolation that used to be in there. We precalculate 
		// some coefficients that remain the same unless the KneePoints change.
		Denominator0Minus1 = KneePoints[0].X - KneePoints[1].X;
		if (FMath::Abs(Denominator0Minus1) < UE_SMALL_NUMBER)
		{
			Denominator0Minus1 = UE_SMALL_NUMBER;
		}
		Denominator1Minus0 = KneePoints[1].X - KneePoints[0].X;
		if (FMath::Abs(Denominator1Minus0) < UE_SMALL_NUMBER)
		{
			Denominator1Minus0 = UE_SMALL_NUMBER;
		}
	}
}
