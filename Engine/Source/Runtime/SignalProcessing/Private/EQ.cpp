// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/EQ.h"
#include "DSP/Dsp.h"

namespace Audio
{
	FEqualizer::FEqualizer()
		: NumChannels(0)
	{
	}

	FEqualizer::~FEqualizer()
	{
	}

	void FEqualizer::Init(const float InSampleRate, const int32 InNumBands, const int32 InNumChannels)
	{
		NumChannels = InNumChannels;

		WorkBuffer.SetNum(InNumChannels);

		FilterBands.SetNum(InNumBands);

		for (FBiquadFilter& Filter : FilterBands)
		{
			Filter.Init(InSampleRate, InNumChannels, EBiquadFilter::ParametricEQ, 500.0f, 1.0f, 0.0f);
		}

	}

	void FEqualizer::SetBandEnabled(const int32 InBand, const bool bInEnabled)
	{
		FilterBands[InBand].SetEnabled(bInEnabled);
	}

	void FEqualizer::SetBandParams(const int32 InBand, const float InFrequency, const float InBandwidth, const float InGainDB)
	{
		FilterBands[InBand].SetParams(EBiquadFilter::ParametricEQ, InFrequency, InBandwidth, InGainDB);
	}

	void FEqualizer::SetBandFrequency(const int32 InBand, const float InFrequency)
	{
		FilterBands[InBand].SetFrequency(InFrequency);
	}

	void FEqualizer::SetBandBandwidth(const int32 InBand, const float InBandwidth)
	{
		FilterBands[InBand].SetBandwidth(InBandwidth);
	}

	void FEqualizer::SetBandGainDB(const int32 InBand, const float InGainDB)
	{
		FilterBands[InBand].SetGainDB(InGainDB);
	}

	void FEqualizer::ProcessAudioFrame(const float* InAudio, float* OutAudio)
	{
		if (NumChannels < 1)
		{
			return;
		}

		// Copy input to output in case there are no bands to process.
		FMemory::Memcpy(OutAudio, InAudio, sizeof(float) * NumChannels);

		// Get pointers to working buffers
		float* WorkInput = OutAudio;
		float* WorkOutput = WorkBuffer.GetData();

		for (FBiquadFilter& Filter : FilterBands)
		{
			Filter.ProcessAudioFrame(WorkInput, WorkOutput);

			// Swap pointers for internal buffers
			Swap<float*>(WorkInput, WorkOutput);
		}

		if (WorkInput != OutAudio)
		{
			// Note: the pointer comparison is done after a Swap
			// If pointers do not point to same buffer, then copy  to output.
			// Whether or not this gets called is dependent upon number of channels.
			FMemory::Memcpy(OutAudio, WorkInput, sizeof(float) * NumChannels);
		}
	}
}
