// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/EarlyReflectionsFast.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	namespace EarlyReflectionsPrivate
	{
		float NormalizedLinearToLog(float InLinear)
		{
			check(InLinear >= 0.f);
			check(InLinear <= 1.f);

			static const float InvLn2 = 1.f / FMath::Loge(2.f);
			return FMath::Loge(1.f + InLinear) * InvLn2;
		}
	}

	FEarlyReflectionsFastSettings::FEarlyReflectionsFastSettings()
		: Gain(1.0f)
		, PreDelayMsec(0.0f)
		, Bandwidth(0.8f)
		, Decay(0.5f)
		, Absorption(0.7f)
	{}

	bool FEarlyReflectionsFastSettings::operator==(const FEarlyReflectionsFastSettings& Other) const
	{
		bool bIsEqual = (
			(Other.Gain == Gain) &&
			(Other.PreDelayMsec == PreDelayMsec) &&
			(Other.Bandwidth == Bandwidth) &&
			(Other.Decay == Decay) &&
			(Other.Absorption == Absorption));
		return bIsEqual;
	}

	bool FEarlyReflectionsFastSettings::operator!=(const FEarlyReflectionsFastSettings& Other) const
	{
		return !(*this == Other);
	}

	const float FEarlyReflectionsFast::MinGain       = 0.0f;
	const float FEarlyReflectionsFast::MaxGain       = 0.9999f;
	const float FEarlyReflectionsFast::MaxPreDelay   = 1000.0f;
	const float FEarlyReflectionsFast::MinPreDelay   = 0.0f;
	const float FEarlyReflectionsFast::MaxBandwidth  = 0.99999f;
	const float FEarlyReflectionsFast::MinBandwidth  = 0.0f;
	const float FEarlyReflectionsFast::MaxDecay      = 1.0f;
	const float FEarlyReflectionsFast::MinDecay      = 0.0001f;
	const float FEarlyReflectionsFast::MaxAbsorption = 0.99999f;
	const float FEarlyReflectionsFast::MinAbsorption = 0.0f;

	const FEarlyReflectionsFastSettings FEarlyReflectionsFast::DefaultSettings;

	FEarlyReflectionsFast::FEarlyReflectionsFast(float InSampleRate, int32 InMaxNumInternalBufferSamples, const FEarlyReflectionsFastSettings& InSettings)
	:	Settings(InSettings),
		SampleRate(InSampleRate),
		LeftFDN(InMaxNumInternalBufferSamples, FFDNDelaySettings::DefaultLeftDelays(InSampleRate)),
		RightFDN(InMaxNumInternalBufferSamples, FFDNDelaySettings::DefaultRightDelays(InSampleRate)),
		LeftPreDelay((int32)InSampleRate * 2.0f, 0, InMaxNumInternalBufferSamples),
		RightPreDelay((int32)InSampleRate * 2.0f, 0, InMaxNumInternalBufferSamples)

	{
		LeftCoefficients.InputScale = 0.25f;
		RightCoefficients.InputScale = 0.25f;

		ClampSettings(Settings);
		ApplySettings();
	}

	FEarlyReflectionsFast::~FEarlyReflectionsFast()
	{
	}

	void FEarlyReflectionsFast::ClampSettings(FEarlyReflectionsFastSettings& InOutSettings)
	{
		InOutSettings.Gain = FMath::Clamp(
			InOutSettings.Gain,
			FEarlyReflectionsFast::MinGain,
			FEarlyReflectionsFast::MaxGain);
		InOutSettings.PreDelayMsec = FMath::Clamp(
			InOutSettings.PreDelayMsec,
			FEarlyReflectionsFast::MinPreDelay,
			FEarlyReflectionsFast::MaxPreDelay);
		InOutSettings.Bandwidth = FMath::Clamp(
			InOutSettings.Bandwidth,
			FEarlyReflectionsFast::MinBandwidth,
			FEarlyReflectionsFast::MaxBandwidth);
		InOutSettings.Decay = FMath::Clamp(
			InOutSettings.Decay,
			FEarlyReflectionsFast::MinDecay,
			FEarlyReflectionsFast::MaxDecay);
		InOutSettings.Absorption = FMath::Clamp(
			InOutSettings.Absorption,
			FEarlyReflectionsFast::MinAbsorption,
			FEarlyReflectionsFast::MaxAbsorption);
	}

	void FEarlyReflectionsFast::SetSettings(const FEarlyReflectionsFastSettings& InSettings)
	{
		Settings = InSettings;
		ClampSettings(Settings);
		ApplySettings();
	}

	void FEarlyReflectionsFast::ApplySettings()
	{
		using namespace EarlyReflectionsPrivate;

		int32 DelaySamples = (int32)SampleRate * Settings.PreDelayMsec / 1000.0f;
		LeftPreDelay.SetDelayLengthSamples(DelaySamples);
		RightPreDelay.SetDelayLengthSamples(DelaySamples);

		// Convert from linear 0-1 scale to logarithmic 0-1 scale. 
		const float LPFG = NormalizedLinearToLog(Settings.Bandwidth);

		LeftInputLPF.SetG(LPFG);
		RightInputLPF.SetG(LPFG);

		LeftCoefficients.LPFB[0] = FMath::Min(Settings.Absorption + 0.1f, 0.9999f);
		LeftCoefficients.LPFB[1] = FMath::Min(Settings.Absorption - 0.12f, 0.9999f);
		LeftCoefficients.LPFB[2] = FMath::Min(Settings.Absorption + 0.08f, 0.9999f);
		LeftCoefficients.LPFB[3] = FMath::Min(Settings.Absorption - 0.07f, 0.9999f);
		LeftCoefficients.LPFA[0] = 1.0f - LeftCoefficients.LPFB[0];
		LeftCoefficients.LPFA[1] = 1.0f - LeftCoefficients.LPFB[1];
		LeftCoefficients.LPFA[2] = 1.0f - LeftCoefficients.LPFB[2];
		LeftCoefficients.LPFA[3] = 1.0f - LeftCoefficients.LPFB[3];
		LeftCoefficients.APFG[0] = 0.1f;
		LeftCoefficients.APFG[1] = 0.2f;
		LeftCoefficients.APFG[2] = 0.3f;
		LeftCoefficients.APFG[3] = 0.4f;

		RightCoefficients.LPFB[0] = FMath::Min(Settings.Absorption + 0.17f, 0.999f);
		RightCoefficients.LPFB[1] = FMath::Min(Settings.Absorption - 0.07f, 0.999f);
		RightCoefficients.LPFB[2] = FMath::Min(Settings.Absorption + 0.05f, 0.999f);
		RightCoefficients.LPFB[3] = FMath::Min(Settings.Absorption - 0.11f, 0.999f);
		RightCoefficients.LPFA[0] = 1.0f - RightCoefficients.LPFB[0];
		RightCoefficients.LPFA[1] = 1.0f - RightCoefficients.LPFB[1];
		RightCoefficients.LPFA[2] = 1.0f - RightCoefficients.LPFB[2];
		RightCoefficients.LPFA[3] = 1.0f - RightCoefficients.LPFB[3];
		RightCoefficients.APFG[0] = 0.1f;
		RightCoefficients.APFG[1] = 0.2f;
		RightCoefficients.APFG[2] = 0.3f;
		RightCoefficients.APFG[3] = 0.4f;

		// 1/sqrt(2) * Decay
		//float Feedback = (1.0f - Settings.Decay) * 0.707f;
		float Feedback = (1.0f - Settings.Decay) * 0.5f;
		LeftCoefficients.Feedback = Feedback;
		RightCoefficients.Feedback = Feedback;

		LeftFDN.SetCoefficients(LeftCoefficients);
		RightFDN.SetCoefficients(RightCoefficients);
	}

	void FEarlyReflectionsFast::ProcessAudio(const FAlignedFloatBuffer& InSamples, const int32 InNumChannels, FAlignedFloatBuffer& OutLeftSamples, FAlignedFloatBuffer& OutRightSamples)
	{
		checkf((InNumChannels == 1) || (InNumChannels == 2), TEXT("EarlyReflections only supports one or two channel input samples."));

		const int32 InNum = InSamples.Num();
		const int32 InNumFrames = InNum / InNumChannels;

		// Resize internal buffers
		LeftInputBuffer.Reset(InNumFrames);
		LeftWorkBufferA.Reset(InNumFrames);
		LeftWorkBufferB.Reset(InNumFrames);
		RightInputBuffer.Reset(InNumFrames);
		RightWorkBufferA.Reset(InNumFrames);
		RightWorkBufferB.Reset(InNumFrames);

		LeftInputBuffer.AddUninitialized(InNumFrames);
		LeftWorkBufferA.AddUninitialized(InNumFrames);
		LeftWorkBufferB.AddUninitialized(InNumFrames);
		RightInputBuffer.AddUninitialized(InNumFrames);
		RightWorkBufferA.AddUninitialized(InNumFrames);
		RightWorkBufferB.AddUninitialized(InNumFrames);

		// Resize output buffers
		OutLeftSamples.Reset(InNumFrames);
		OutRightSamples.Reset(InNumFrames);

		OutLeftSamples.AddUninitialized(InNumFrames);
		OutRightSamples.AddUninitialized(InNumFrames);

		if (1 == InNumChannels)
		{
			// Copy mono to both left and right
			FMemory::Memcpy(LeftInputBuffer.GetData(), InSamples.GetData(), sizeof(float) * InNumFrames);
			FMemory::Memcpy(RightInputBuffer.GetData(), InSamples.GetData(), sizeof(float) * InNumFrames);
		}
		else if (2 == InNumChannels)
		{
			BufferDeinterleave2ChannelFast(InSamples, LeftInputBuffer, RightInputBuffer);
		}
		else 
		{
			// This class only supports 1 and 2 channel input audio.
			FMemory::Memset(OutLeftSamples.GetData(), 0, sizeof(float) * InNumFrames);
			FMemory::Memset(OutRightSamples.GetData(), 0, sizeof(float) * InNumFrames);
			return;
		}

		// predelay
		LeftPreDelay.ProcessAudio(LeftInputBuffer, LeftWorkBufferB);
		RightPreDelay.ProcessAudio(RightInputBuffer, RightWorkBufferB);

		// lpf
		LeftInputLPF.ProcessAudio(LeftWorkBufferB, LeftWorkBufferA);
		RightInputLPF.ProcessAudio(RightWorkBufferB, RightWorkBufferA);

		// feedback delay network
		LeftFDN.ProcessAudio(LeftWorkBufferA, OutLeftSamples);
		RightFDN.ProcessAudio(RightWorkBufferA, OutRightSamples);
		
		const float OneMinusGain = 1.0 - Settings.Gain;

		// Apply Gain
		ArrayMultiplyByConstantInPlace(OutLeftSamples, Settings.Gain);
		ArrayMultiplyByConstantInPlace(OutRightSamples, Settings.Gain);
	}

	void FEarlyReflectionsFast::FlushAudio()
	{
		// predelay
		LeftPreDelay.Reset();
		RightPreDelay.Reset();

		// lpf
		LeftInputLPF.FlushAudio();
		RightInputLPF.FlushAudio();

		// feedback delay network
		LeftFDN.FlushAudio();
		RightFDN.FlushAudio();
	}
}
