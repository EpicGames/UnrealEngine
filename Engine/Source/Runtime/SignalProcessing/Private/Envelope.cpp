// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Envelope.h"
#include "DSP/Dsp.h"

namespace Audio
{
	FEnvelope::FEnvelope()
		: VoiceId(0)
		, CurrentEnvelopeValue(0.0f)
		, CurrentEnvelopeBiasValue(0.0f)
		, SampleRate(44100.0f)
		, AttackTimeMSec(100.0f)
		, DecayTimeMsec(100.0f)
		, SustainGain(0.7f)
		, ReleaseTimeMsec(2000.0f)
		, ShutdownTimeMsec(10.0f)
		, ShutdownDelta(0.0f)
		, Depth(1.0f)
		, BiasDepth(1.0f)
		, CurrentState(EEnvelopeState::Off)
		, ModMatrix(nullptr)
		, bIsSimulatingAnalog(true)
		, bIsLegatoMode(false)
		, bIsRetriggerMode(false)
		, bChanged(true)
		, bInvert(false)
		, bBiasInvert(false)
	{
	}

	FEnvelope::~FEnvelope()
	{
	}

	void FEnvelope::Init(const float InSampleRate, const int32 InVoiceId, FModulationMatrix* InModMatrix, const bool bInSimulateAnalog)
	{
		VoiceId = InVoiceId;
		SampleRate = InSampleRate;
		SetSimulateAnalog(bInSimulateAnalog);
		bChanged = true;

		ModMatrix = InModMatrix;
		if (ModMatrix)
		{
			EnvSource = ModMatrix->CreatePatchSource(VoiceId);
			BiasedEnvSource = ModMatrix->CreatePatchSource(VoiceId);

#if MOD_MATRIX_DEBUG_NAMES
			EnvSource.Name = TEXT("EnvSource");
			BiasedEnvSource.Name = TEXT("BiasedEnvSource");
#endif
		}
	}

	FEnvelope::EEnvelopeState FEnvelope::GetState() const
	{
		return CurrentState;
	}

	void FEnvelope::SetSimulateAnalog(const bool bInSimulatingAnalog)
	{
		bIsSimulatingAnalog = bInSimulatingAnalog;
		bChanged = true;
	}

	void FEnvelope::Start()
	{
		bCurrentCycleIsADOnly = SustainGain <= SMALL_NUMBER;

		// Don't reset the envelope if we're in legato mode and we're not in release or off
		if (bIsLegatoMode && CurrentState != EEnvelopeState::Off && CurrentState != EEnvelopeState::Release)
		{
			return;
		}

		// Reset the envelope data
		Reset();

		// Set the state back to attack no matter where it is
		CurrentState = FEnvelope::EEnvelopeState::Attack;
	}

	// logic for one mono note interrupting another mono note (same voice)
	void FEnvelope::StartLegato(const float InNewDepth)
	{
		// Envelope is not being used. Don't do the work (and don't divide by zero)
		if (Depth <= SMALL_NUMBER && InNewDepth <= SMALL_NUMBER)
		{
			return;
		}

		bCurrentCycleIsADOnly = SustainGain <= SMALL_NUMBER;

		switch (CurrentState)
		{
			case EEnvelopeState::Attack:
			{
				if (InNewDepth > Depth)
				{
					CurrentEnvelopeValue *= Depth / InNewDepth;
					Depth = InNewDepth;

					bChanged = true;
				}
			}
			break;

			case EEnvelopeState::Decay:
			{
				if (InNewDepth > Depth * CurrentEnvelopeValue)
				{
					CurrentState = EEnvelopeState::Attack;
				}

				CurrentEnvelopeValue *= Depth / InNewDepth;
				Depth = InNewDepth;

				bChanged = true;
			}
			break;

			case EEnvelopeState::Sustain:
			{
				// new sustain gain is higher
				if (InNewDepth > Depth * SustainGain)
				{
					CurrentEnvelopeValue *= Depth / InNewDepth;
					Depth = InNewDepth;
					CurrentState = EEnvelopeState::Attack;
				}

				bChanged = true;
			}
			break;

			case EEnvelopeState::Release:
			{
				// "attack up to" a larger new depth or "decay down to" a lower new depth
				CurrentState = EEnvelopeState::Attack;

				if(InNewDepth < Depth * CurrentEnvelopeValue)
				{
					CurrentState = EEnvelopeState::Decay;
				}

				CurrentEnvelopeValue *= Depth / InNewDepth;
				Depth = InNewDepth;

				bChanged = true;
			}
			break;

			default:
			{
				// previous behavior
				Depth = InNewDepth;
				Start();
			}
		}
	}

	void FEnvelope::Stop()
	{
		if (CurrentEnvelopeValue == 0.0f)
		{
			// already finished (jump to off)
			CurrentState = EEnvelopeState::Off;
		}
		else if (!bCurrentCycleIsADOnly)
		{
			// normal envelope mode (jump to release)
			CurrentState = EEnvelopeState::Release;
		}
		else if (CurrentState == EEnvelopeState::Attack)
		{
			// AD only envelope mode (jump to decay)
			CurrentState = EEnvelopeState::Decay;
		}
	}

	void FEnvelope::Shutdown()
	{
		if (bIsLegatoMode)
		{
			return;
		}

		// If we're forcing off or if we're already off, then set state to off
		if (CurrentEnvelopeValue == 0.0f)
		{
			CurrentState = EEnvelopeState::Off;
		}
		else
		{
			// If we actually have an envelope value now, go to release
			CurrentState = EEnvelopeState::Shutdown;

			ShutdownDelta = -(1000.0f * CurrentEnvelopeValue) / ShutdownTimeMsec / SampleRate;
		}
	}

	void FEnvelope::Kill()
	{
		CurrentState = EEnvelopeState::Off;
	}

	bool FEnvelope::IsDone() const
	{
		return CurrentState == EEnvelopeState::Off;
	}

	void FEnvelope::Reset()
	{
		// Set the envelope state to off when reset
		CurrentState = EEnvelopeState::Off;

		// Recompute the envelopes if needed
		SetSimulateAnalog(bIsSimulatingAnalog);

		bChanged = true;

		// If set to reset the envelope value to 0.0, set the envelope back to 0
		// Otherwise the envelope will continue to the target value from where it currently is
		if (bIsRetriggerMode)
		{
			CurrentEnvelopeValue = 0.0f;
		}
	}

	void FEnvelope::Update()
	{
		if (bChanged)
		{
			bChanged = false;

			if (bIsSimulatingAnalog)
			{
				// If in analog mode, we're going to emulate capacitor charging
				// Q = 1 - e^(-t/RC) for charging (attack)
				// Q = e^(-t/RC) for discharging
				AttackData.TCO = FMath::Exp(-1.5f);
				DecayData.TCO = FMath::Exp(-4.95f);
			}
			else
			{
				AttackData.TCO = 0.99999f;
				DecayData.TCO = FMath::Exp(-11.05f);
			}

			ReleaseData.TCO = DecayData.TCO;

			AttackData.TimeSamples = 0.001f * SampleRate * AttackTimeMSec;;
			DecayData.TimeSamples = 0.001f * SampleRate * DecayTimeMsec;
			ReleaseData.TimeSamples = 0.001f * SampleRate * ReleaseTimeMsec;

			AttackData.Coefficient = FMath::Exp(-FMath::Loge((1.0f + AttackData.TCO) / AttackData.TCO) / AttackData.TimeSamples);
			AttackData.Offset = (1.0f + AttackData.TCO) * (1.0f - AttackData.Coefficient);

			DecayData.Coefficient = FMath::Exp(-FMath::Loge((1.0f + DecayData.TCO) / DecayData.TCO) / DecayData.TimeSamples);
			DecayData.Offset = (bCurrentCycleIsADOnly? 0.0f : SustainGain - DecayData.TCO)*(1.0f - DecayData.Coefficient);

			ReleaseData.Coefficient = FMath::Exp(-FMath::Loge((1.0f + ReleaseData.TCO) / ReleaseData.TCO) / ReleaseData.TimeSamples);
			ReleaseData.Offset = -ReleaseData.TCO*(1.0f - ReleaseData.Coefficient);
		}
	}

	float FEnvelope::Generate(float* BiasedOutput)
	{
		// Update the envelope if it changed
		Update();

		// Evaluate the finite state machine
		switch (CurrentState)
		{
			case EEnvelopeState::Off:
			{
				if (bIsRetriggerMode)
				{
					CurrentEnvelopeValue = 0.0f;
				}
			}
			break;

			case EEnvelopeState::Attack:
			{
				CurrentEnvelopeValue = AttackData.Offset + CurrentEnvelopeValue * AttackData.Coefficient;
				if (CurrentEnvelopeValue >= 1.0f || AttackTimeMSec <= 0.0f)
				{
					CurrentEnvelopeValue = 1.0f;
					CurrentState = EEnvelopeState::Decay;
					break;
				}
			}
			break;

			case EEnvelopeState::Decay:
			{
				// --- render value
				CurrentEnvelopeValue = DecayData.Offset + CurrentEnvelopeValue * DecayData.Coefficient;
				if ((CurrentEnvelopeValue <= SustainGain) || DecayTimeMsec <= 0.0f)
				{
					
					if(!bCurrentCycleIsADOnly)
					{
						CurrentEnvelopeValue = SustainGain;
						CurrentState = EEnvelopeState::Sustain;
						break;
					}
					else if (CurrentEnvelopeValue <= SMALL_NUMBER)
					{
						CurrentState = EEnvelopeState::Off;
					}
				}
			}
			break;

			case EEnvelopeState::Sustain:
			{
				// live-update sustain level (to hear changes made during sustain phase)
				CurrentEnvelopeValue = SustainGain;

				if (bCurrentCycleIsADOnly && SustainGain <= SMALL_NUMBER)
				{
					// Check if envelope was being used as AD only
					CurrentState = EEnvelopeState::Off;
				}
			}
			break;

			case EEnvelopeState::Release:
			{
				CurrentEnvelopeValue = ReleaseData.Offset + CurrentEnvelopeValue * ReleaseData.Coefficient;
				if (CurrentEnvelopeValue <= 0.0f || ReleaseTimeMsec <= 0.0f || SustainGain <= SMALL_NUMBER)
				{
					CurrentEnvelopeValue = 0.0f;
					CurrentState = EEnvelopeState::Off;
					break;
				}
			}
			break;

			case EEnvelopeState::Shutdown:
			{
				if (bIsRetriggerMode)
				{
					CurrentEnvelopeValue += ShutdownDelta;
					if (CurrentEnvelopeValue <= 0)
					{
						CurrentState = EEnvelopeState::Off;
						CurrentEnvelopeValue = 0.0f;
						break;
					}
				}
				else
				{
					CurrentState = EEnvelopeState::Off;
				}
			}
			break;
		}

		// Send the bias output (i.e. scale envelope by offset by sustain gain)
		float CurrentBiasedOutput = bBiasInvert ? 1.0f - CurrentEnvelopeValue : CurrentEnvelopeValue;
		CurrentBiasedOutput -= SustainGain;
		CurrentBiasedOutput *= BiasDepth;

		float OutputEnvValue = bInvert ? 1.0f - CurrentEnvelopeValue : CurrentEnvelopeValue;
		OutputEnvValue *= Depth;

		if (BiasedOutput)
		{
			*BiasedOutput = CurrentBiasedOutput;
		}

		if (ModMatrix)
		{
			ModMatrix->SetSourceValue(VoiceId, EnvSource, OutputEnvValue);
			ModMatrix->SetSourceValue(VoiceId, BiasedEnvSource, CurrentBiasedOutput);
		}

		return OutputEnvValue;
	}

	void FEnvelope::SetAttackTime(const float InAttackTimeMsec)
	{
		bChanged |= !FMath::IsNearlyEqual(AttackTimeMSec, InAttackTimeMsec);
		AttackTimeMSec = InAttackTimeMsec;
	}

	void FEnvelope::SetDecayTime(const float InDecayTimeMsec)
	{
		bChanged |= !FMath::IsNearlyEqual(DecayTimeMsec, InDecayTimeMsec);
		DecayTimeMsec = InDecayTimeMsec;
	}

	void FEnvelope::SetSustainGain(const float InSustainGain)
	{
		bChanged |= !FMath::IsNearlyEqual(SustainGain, InSustainGain);
		SustainGain = InSustainGain;
	}

	void FEnvelope::SetReleaseTime(const float InReleaseTimeMsec)
	{
		bChanged |= !FMath::IsNearlyEqual(ReleaseTimeMsec, InReleaseTimeMsec);
		ReleaseTimeMsec = InReleaseTimeMsec;
	}

	void FEnvelope::SetInvert(const bool bInInvert)
	{
		bInvert = bInInvert;
	}

	void FEnvelope::SetBiasInvert(const bool bInBiasInvert)
	{
		bBiasInvert = bInBiasInvert;
	}

	void FEnvelope::SetDepth(const float InDepth)
	{
		Depth = InDepth;
	}

	void FEnvelope::SetBiasDepth(const float InDepth)
	{
		BiasDepth = InDepth;
	}

	void FADEnvelope::Init(int32 InSampleRate)
	{
		SampleRate = (float)InSampleRate;
		SetAttackTimeSeconds(AttackTimeSeconds);
		SetDecayTimeSeconds(DecayTimeSeconds);
	}


	void FADEnvelope::SetAttackTimeSeconds(float InAttackTimeSeconds)
	{
		AttackTimeSeconds = FMath::Max(0.0f, InAttackTimeSeconds);
		AttackSampleCount = SampleRate * AttackTimeSeconds;
	}

	void FADEnvelope::SetDecayTimeSeconds(float InDecayTimeSeconds)
	{
		DecayTimeSeconds = FMath::Max(0.0f, InDecayTimeSeconds);
		DecaySampleCount = SampleRate * DecayTimeSeconds;
	}

	void FADEnvelope::SetAttackCurveFactor(float InAttackCurveFactor)
	{
		AttackCurveFactor = FMath::Max(KINDA_SMALL_NUMBER, AttackCurveFactor);
	}

	void FADEnvelope::SetDecayCurveFactor(float InDecayCurveFactor)
	{
		DecayCurveFactor = FMath::Max(KINDA_SMALL_NUMBER, InDecayCurveFactor);
	}

	void FADEnvelope::Attack()
	{
		// Resets the envelope state
		CurrentSampleIndex = 0;
		StartingEnvelopeValue = CurrentEnvelopeValue;
	}

	void FADEnvelope::GetNextEnvelopeOut(int32 StartFrame, int32 EndFrame, TArray<int32>& OutFinishedFrames, Audio::AlignedFloatBuffer& OutEnvelope)
	{
		if (CurrentSampleIndex == INDEX_NONE)
		{
			return;
		}

		float* OutEnvPtr = OutEnvelope.GetData();
		int32 FrameIndex = StartFrame;

		for (int32 i = StartFrame; i < EndFrame; ++i)
		{
			// In attack
			if (CurrentSampleIndex <= AttackSampleCount)
			{
				float EnvelopeLeft = 1.0f - StartingEnvelopeValue;
				float AttackFraction = (float)CurrentSampleIndex++ / AttackSampleCount;
				CurrentEnvelopeValue = StartingEnvelopeValue + EnvelopeLeft * FMath::Pow(AttackFraction, AttackCurveFactor);
				OutEnvPtr[i] = CurrentEnvelopeValue;
			}
			else
			{
				int32 TotalSampleCount = AttackSampleCount + DecaySampleCount;

				// In decay
				if (CurrentSampleIndex < TotalSampleCount)
				{
					int32 SampleCountInDecayState = CurrentSampleIndex++ - AttackSampleCount;
					float DecayFraction = (float)SampleCountInDecayState / DecaySampleCount;
					CurrentEnvelopeValue = FMath::Pow(DecayFraction, DecayCurveFactor);
					OutEnvPtr[i] = CurrentEnvelopeValue;
				}
				// We're looping, so we reset the envelope state and continue on, which will go into attack phase
				else if (bIsLooping)
				{
					Attack();
					// We still want to render this frame of audio for the restart envelope. Our AD envelope always starts from 0.0, so we know the first sample will be 0.0.
					OutEnvPtr[i] = 0.0f;
					// Still want to increment the sample index to avoid off-by one error with AD timing
					CurrentSampleIndex++;
					// Output that we "finished" this envelope. This may result in multiple entries if the AD loop repeats faster than the render block.
					OutFinishedFrames.Add(i);
				}
				// we're done
				else
				{
					int32 NumSamplesLeft = EndFrame - i;
					if (NumSamplesLeft > 0)
					{
						FMemory::Memzero(&OutEnvPtr[i], sizeof(float)*NumSamplesLeft);
					}
					CurrentSampleIndex = INDEX_NONE;
					OutFinishedFrames.Add(i);
					break;
				}
			}

		}
	}

	bool FADEnvelope::GetNextEnvelopeOut(float& OutEnvelope)
	{
		TArray<int32> OutFinishedFrames;
		GetNextEnvelopeOut(0, 1, OutFinishedFrames, OutEnvelope);
		return OutFinishedFrames.Num() > 0;
	}

	void FADEnvelope::GetNextEnvelopeOut(int32 StartFrame, int32 EndFrame, TArray<int32>& OutFinishedFrames, float& OutEnvelope)
	{
		// Don't need to do anything if we're not generating the envelope at the top of the block since this is a block-rate envelope
		if (StartFrame > 0 || CurrentSampleIndex == INDEX_NONE)
		{
			OutEnvelope = 0.0f;
			return;
		}

		// We are in attack
		if (CurrentSampleIndex < AttackSampleCount)
		{
			float EnvelopeLeft = 1.0f - StartingEnvelopeValue;
			float AttackFraction = (float)CurrentSampleIndex++ / AttackSampleCount;

			CurrentEnvelopeValue = StartingEnvelopeValue + EnvelopeLeft * FMath::Pow(AttackFraction, AttackCurveFactor);
			OutEnvelope = CurrentEnvelopeValue;
		}
		else
		{
			int32 TotalEnvSampleCount = (AttackSampleCount + DecaySampleCount);

			// We are in Decay
			if (CurrentSampleIndex < TotalEnvSampleCount)
			{
				int32 SampleCountInDecayState = CurrentSampleIndex++ - AttackSampleCount;
				float DecayFraction = (float)SampleCountInDecayState / DecaySampleCount;
				CurrentEnvelopeValue = 1.0f - FMath::Pow(DecayFraction, DecayCurveFactor);
				OutEnvelope = CurrentEnvelopeValue;
			}
			// We are looping so reset the sample index
			else if (bIsLooping)
			{
				CurrentSampleIndex = 0;
				OutFinishedFrames.Add(0);
			}
			else
			{
				// Envelope is done
				CurrentSampleIndex = INDEX_NONE;
				OutEnvelope = 0.0f;
				OutFinishedFrames.Add(0);
			}
		}
	}
}
