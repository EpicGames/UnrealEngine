// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/EnvelopeFollower.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	namespace EnvelopeFollowerPrivate
	{
		FORCEINLINE float SmoothSample(float InSample, float InPriorSmoothedSample, float InAttackSamples, float InReleaseSamples)
		{
			float Value = InPriorSmoothedSample;
			float Diff = Value - InSample;

			if (Value <= InSample)
			{
				Value = (InAttackSamples * Diff) + InSample;
			}
			else
			{
				Value = (InReleaseSamples * Diff) + InSample;
			}

			return Value;
		}

		template<int32 NumChannels>
		void ProcessSmoothingFixedChannels(const float* InBuffer, const int32 InNumFrames, const float AttackSamples, const float ReleaseSamples, float* PriorEnvelopeValues)
		{
			check(InNumFrames > 0);
			float EnvelopeValues[NumChannels];
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				EnvelopeValues[Channel] = PriorEnvelopeValues[Channel];
			}

			const int32 NumSamples = InNumFrames * NumChannels;

			for (int32 Pos = 0; Pos < NumSamples; Pos += NumChannels)
			{
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					EnvelopeValues[Channel] = EnvelopeFollowerPrivate::SmoothSample(InBuffer[Pos + Channel], EnvelopeValues[Channel], AttackSamples, ReleaseSamples);
				}
			}

			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				PriorEnvelopeValues[Channel] = EnvelopeValues[Channel];
			}
		}

		/**
		 * Returns the maximum of the elements in the given vector
		 */
		FORCEINLINE float VectorReduceMax(const VectorRegister4Float& ABCD)
		{
			const VectorRegister4Float BADC = VectorSwizzle(ABCD, 1, 0, 3, 2);
			const VectorRegister4Float MaxAB_MaxCD = VectorMax(ABCD, BADC);
			const VectorRegister4Float MaxCD_MaxAB = VectorSwizzle(MaxAB_MaxCD, 2, 3, 0, 1);
			const VectorRegister4Float MaxABCD = VectorMax(MaxAB_MaxCD, MaxCD_MaxAB);
			return VectorGetComponent(MaxABCD, 0);
		}

		alignas(AUDIO_SIMD_BYTE_ALIGNMENT) static const float TruePeakPolyphaseCoefficients[] =
		{
			+0.0017089843750f, -0.0291748046875f, -0.0189208984375f, -0.0083007812500f,
			+0.0109863281250f, +0.0292968750000f, +0.0330810546875f, +0.0148925781250f,
			-0.0196533203125f, -0.0517578125000f, -0.0582275390625f, -0.0266113281250f,
			+0.0332031250000f, +0.0891113281250f, +0.1015625000000f, +0.0476074218750f,
			-0.0594482421875f, -0.1665039062500f, -0.2003173828125f, -0.1022949218750f,
			+0.1373291015625f, +0.4650878906250f, +0.7797851562500f, +0.9721679687500f,
			+0.9721679687500f, +0.7797851562500f, +0.4650878906250f, +0.1373291015625f,
			-0.1022949218750f, -0.2003173828125f, -0.1665039062500f, -0.0594482421875f,
			+0.0476074218750f, +0.1015625000000f, +0.0891113281250f, +0.0332031250000f,
			-0.0266113281250f, -0.0582275390625f, -0.0517578125000f, -0.0196533203125f,
			+0.0148925781250f, +0.0330810546875f, +0.0292968750000f, +0.0109863281250f,
			-0.0083007812500f, -0.0189208984375f, -0.0291748046875f, +0.0017089843750f,
		};

		constexpr uint32 TruePeakPhaseCount = 4; // == Over-sampling ratio
		constexpr uint32 TruePeakNumCoefficientsPerPhase = UE_ARRAY_COUNT(TruePeakPolyphaseCoefficients) / TruePeakPhaseCount;
		constexpr uint32 TruePeakFilterHistorySize = 1 << FMath::CeilLogTwo(TruePeakNumCoefficientsPerPhase - 1);
	}

	FAttackRelease::FAttackRelease(float InSampleRate, float InAttackTimeMsec, float InReleaseTimeMsec, bool bInIsAnalog)
	: SampleRate(InSampleRate)
	, AttackTimeSamples(1.f)
	, ReleaseTimeSamples(1.f)
	, bIsAnalog(bInIsAnalog)
	{
		if (!ensure(SampleRate > 0.f))
		{
			SampleRate = 48000.f;
		}
		// Set the attack and release times using the default values
		SetAttackTime(InAttackTimeMsec);
		SetReleaseTime(InReleaseTimeMsec);
	}

	void FAttackRelease::SetSampleRate(float InSampleRate)
	{
		if (ensure(InSampleRate > 0.f))
		{
			SampleRate = InSampleRate;
			SetAttackTime(AttackTimeMsec);
			SetReleaseTime(ReleaseTimeMsec);
		}
	}

	void FAttackRelease::SetAnalog(bool bInIsAnalog)
	{
		bIsAnalog = bInIsAnalog;
		SetAttackTime(AttackTimeMsec);
		SetReleaseTime(ReleaseTimeMsec);
	}

	void FAttackRelease::SetAttackTime(float InAttackTimeMsec)
	{
		AttackTimeMsec = InAttackTimeMsec;
		if (AttackTimeMsec > 0.f)
		{
			float TimeConstant = bIsAnalog ? AnalogTimeConstant : DigitalTimeConstant;
			AttackTimeSamples = FMath::Exp(-1000.0f * TimeConstant / (AttackTimeMsec * SampleRate));
		}
		else
		{
			AttackTimeSamples = 0.f;
		}
	}

	void FAttackRelease::SetReleaseTime(float InReleaseTimeMsec)
	{
		ReleaseTimeMsec = InReleaseTimeMsec;
		if (ReleaseTimeMsec > 0.f)
		{
			float TimeConstant = bIsAnalog ? AnalogTimeConstant : DigitalTimeConstant;
			ReleaseTimeSamples = FMath::Exp(-1000.0f * TimeConstant / (ReleaseTimeMsec * SampleRate));
		}
		else
		{
			ReleaseTimeSamples = 0.f;
		}
	}

	FAttackReleaseSmoother::FAttackReleaseSmoother(float InSampleRate, int32 InNumChannels, float InAttackTimeMsec, float InReleaseTimeMsec, bool bInIsAnalog)
	: FAttackRelease(InSampleRate, InAttackTimeMsec, InReleaseTimeMsec, bInIsAnalog)
	, NumChannels(0)
	{
		SetNumChannels(InNumChannels);
	}

	void FAttackReleaseSmoother::ProcessAudio(const float* InBuffer, int32 InNumFrames)
	{
		if (InNumFrames > 0)
		{
			const int32 NumSamples = InNumFrames * NumChannels;
			const float AttackSamples = GetAttackTimeSamples();
			const float ReleaseSamples = GetReleaseTimeSamples();

#define SMOOTHER_SWITCH_CASE(NumChannels)                                                                                                                                 \
			case NumChannels:                                                                                                                                             \
				EnvelopeFollowerPrivate::ProcessSmoothingFixedChannels<NumChannels>(InBuffer, InNumFrames, AttackSamples, ReleaseSamples, PriorEnvelopeValues.GetData()); \
				break

			// Call out to a function compiled with a fixed number of channels, so compiler and CPU can parallelize the work better.
			switch (NumChannels)
			{
				SMOOTHER_SWITCH_CASE(1);
				SMOOTHER_SWITCH_CASE(2);
				SMOOTHER_SWITCH_CASE(3);
				SMOOTHER_SWITCH_CASE(4);
				SMOOTHER_SWITCH_CASE(5);
				SMOOTHER_SWITCH_CASE(6);
				SMOOTHER_SWITCH_CASE(7);
				SMOOTHER_SWITCH_CASE(8);
			default:
				for (int32 Channel = 0; Channel < NumChannels; Channel++)
				{
					float EnvelopeValue = PriorEnvelopeValues[Channel];

					for (int32 Pos = Channel; Pos < NumSamples; Pos += NumChannels)
					{
						EnvelopeValue = EnvelopeFollowerPrivate::SmoothSample(InBuffer[Pos], EnvelopeValue, AttackSamples, ReleaseSamples);
					}

					PriorEnvelopeValues[Channel] = EnvelopeValue;
				}
				break;
			}
		}

#undef SMOOTHER_SWITCH_CASE
	}

	void FAttackReleaseSmoother::ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer)
	{
		if (InNumFrames > 0)
		{
			const int32 NumSamples = InNumFrames * NumChannels;
			const float AttackSamples = GetAttackTimeSamples();
			const float ReleaseSamples = GetReleaseTimeSamples();

			for (int32 Channel = 0; Channel < NumChannels; Channel++)
			{
				float EnvelopeValue = PriorEnvelopeValues[Channel];

				for (int32 Pos = Channel; Pos < NumSamples; Pos += NumChannels)
				{
					EnvelopeValue = EnvelopeFollowerPrivate::SmoothSample(InBuffer[Pos], EnvelopeValue, AttackSamples, ReleaseSamples);
					OutBuffer[Pos] = EnvelopeValue;
				}

				PriorEnvelopeValues[Channel] = EnvelopeValue;
			}
		}
	}

	const TArray<float>& FAttackReleaseSmoother::GetEnvelopeValues() const
	{
		return PriorEnvelopeValues;
	}

	void FAttackReleaseSmoother::SetNumChannels(int32 InNumChannels)
	{
		if (InNumChannels != NumChannels)
		{
			PriorEnvelopeValues.Reset(InNumChannels);
			if (ensure(InNumChannels > 0))
			{
				PriorEnvelopeValues.AddZeroed(InNumChannels);
			}
			NumChannels = InNumChannels;
		}
	}

	void FAttackReleaseSmoother::Reset()
	{
		if (NumChannels > 0)
		{
			FMemory::Memset(PriorEnvelopeValues.GetData(), 0, sizeof(float) * NumChannels);
		}
	}


	FMeanSquaredFIR::FMeanSquaredFIR(float InSampleRate, int32 InNumChannels, float InWindowTimeMsec)
	: SampleRate(InSampleRate)
	, NumChannels(InNumChannels)
	, WindowTimeSamples(256)
	, NormFactor(1.f)
	{
		if (!ensure(SampleRate > 0.f))
		{
			SampleRate = 48000.f;
		}

		if (ensure(NumChannels > 0))
		{
			ChannelValues.AddZeroed(NumChannels);
		}

		SetWindowSize(InWindowTimeMsec);
	}

	void FMeanSquaredFIR::SetWindowSize(float InWindowTimeMsec)
	{
		if (ensure(InWindowTimeMsec > 0.f))
		{
			WindowTimeFrames = FMath::Max(1, FMath::RoundToInt(InWindowTimeMsec * 1000.f / SampleRate));
			NormFactor = 1.f / static_cast<float>(WindowTimeFrames);
			WindowTimeSamples = WindowTimeFrames * NumChannels;
		}

		Reset();
	}

	void FMeanSquaredFIR::Reset()
	{
		if (NumChannels > 0)
		{
			ChannelValues.Reset();
			ChannelValues.AddZeroed(NumChannels);

			WindowTimeSamples = WindowTimeFrames * NumChannels;
			HistorySquared.Reset(FMath::Max(2 * WindowTimeSamples, DefaultHistoryCapacity));
			if (WindowTimeSamples > 0)
			{
				HistorySquared.PushZeros(WindowTimeSamples);
			}
		}
	}

	void FMeanSquaredFIR::SetNumChannels(int32 InNumChannels)
	{
		if (NumChannels != InNumChannels)
		{
			NumChannels = InNumChannels;
			Reset(); // Update channel buffers and window buffers. 
		}
	}

	void FMeanSquaredFIR::ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer)
	{
		const int32 NumSamples = InNumFrames * NumChannels;
		SquaredHistoryBuffer.Reset(NumSamples);
		SquaredInputBuffer.Reset(NumSamples);

		if (NumSamples > 0)
		{
			// Square the input data
			SquaredInputBuffer.AddUninitialized(NumSamples);
			ArraySquare(MakeArrayView(InBuffer, NumSamples), SquaredInputBuffer);	

			// Save the squared data for later
			HistorySquared.Push(SquaredInputBuffer);

			// Get the historic input so it can be subtracted from 
			// the accumulated value as it leaves the window.
			SquaredHistoryBuffer.AddUninitialized(NumSamples);
			HistorySquared.Pop(SquaredHistoryBuffer.GetData(), NumSamples);
		}

		const float* HistorySquaredData = SquaredHistoryBuffer.GetData();
		const float* InputSquaredData = SquaredInputBuffer.GetData();
		float* ValueData = ChannelValues.GetData();

		// MS[n] = MS[n - 1] + x^2[n] - x^2[n - N]
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
		{
			float Value = ValueData[ChannelIndex];

			for (int32 SampleIndex = ChannelIndex; SampleIndex < NumSamples; SampleIndex += NumChannels)
			{
				Value -= HistorySquaredData[SampleIndex];
				Value += InputSquaredData[SampleIndex];
				OutBuffer[SampleIndex] = Value * NormFactor;
			}

			ValueData[ChannelIndex] = Value;
		}
	}


	FMeanSquaredIIR::FMeanSquaredIIR(float InSampleRate, int32 InNumChannels, float InWindowTimeMsec)
	: SampleRate(InSampleRate)
	, NumChannels(0)
	, Alpha(0.f)
	, Beta(1.f)
	{
		if (!ensure(SampleRate > 0.f))
		{
			SampleRate = 48000.f;
		}

		SetNumChannels(InNumChannels);
		SetWindowSize(InWindowTimeMsec);
	}

	void FMeanSquaredIIR::SetWindowSize(float InWindowTimeMsec)
	{
		if (ensure(InWindowTimeMsec > 0.f))
		{
			// Exponential decay set so window is 1/e at end of window
			Alpha = FMath::Exp(-1000.f / (SampleRate * InWindowTimeMsec));
			Beta = 1.f - Alpha;
		}

		Reset();
	}

	void FMeanSquaredIIR::SetNumChannels(int32 InNumChannels)
	{
		if (NumChannels != InNumChannels)
		{
			ChannelValues.Reset();
			if (ensure(InNumChannels > 0))
			{
				ChannelValues.AddZeroed(InNumChannels);
			}
			NumChannels = InNumChannels;
		}
	}

	void FMeanSquaredIIR::Reset()
	{
		if (NumChannels > 0)
		{
			ChannelValues.Reset();
			ChannelValues.AddZeroed(NumChannels);
		}
	}

	// TODO: what's the convention. pass num frames or num samples?
	void FMeanSquaredIIR::ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer)
	{
		const int32 NumSamples = InNumFrames * NumChannels;

		// Square input and store in output.
		ArraySquare(MakeArrayView(InBuffer, NumSamples), MakeArrayView(OutBuffer, NumSamples));

		// MS[n] = Beta * x^2[n] + Alpha * MS[n - 1];
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
		{
			// Get last output value
			float Value = ChannelValues[ChannelIndex];

			for (int32 SampleIndex = ChannelIndex; SampleIndex < NumSamples; SampleIndex += NumChannels)
			{
				Value = Beta * OutBuffer[SampleIndex] + Alpha * Value;
				OutBuffer[SampleIndex] = Value;
			}

			// Store last output value for next call to ProcessAudio
			ChannelValues[ChannelIndex] = Value;
		}
	}

	void FTruePeak::Reset()
	{
		AllChannelSignalHistories.Reset();
		AllChannelSignalHistories.SetNumZeroed(NumChannels * EnvelopeFollowerPrivate::TruePeakFilterHistorySize);
		HistoryPosition = 0;
	}

	void FTruePeak::ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer)
	{
		using namespace EnvelopeFollowerPrivate;

		constexpr uint32 HistoryIndexMask = TruePeakFilterHistorySize - 1;

		const int32 NumSamples = InNumFrames * NumChannels;

		uint32 PrevHistoryWriteIndex = HistoryPosition;

		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
		{
			const TArrayView ChannelSignalHistory = MakeArrayView(AllChannelSignalHistories).Slice(ChannelIndex * TruePeakFilterHistorySize, TruePeakFilterHistorySize);
			
			// Reset PrevHistoryWriteIndex for each channel:
			PrevHistoryWriteIndex = HistoryPosition;

			for (int32 SampleIndex = ChannelIndex; SampleIndex < NumSamples; SampleIndex += NumChannels)
			{
				const float InputSample = InBuffer[SampleIndex];

				static_assert(TruePeakPhaseCount == sizeof(VectorRegister4Float) / sizeof(float)); // TruePeakPhaseCount matches SIMD size.
				const VectorRegister4Float PolyphaseCoefficients0 = VectorLoad(&TruePeakPolyphaseCoefficients[0]);

				VectorRegister4Float Accumulators = VectorMultiply(VectorSetFloat1(InputSample), PolyphaseCoefficients0);

				uint32 HistoryReadIndex = PrevHistoryWriteIndex;
				for (uint32 CoefficientReadIndex = TruePeakPhaseCount; CoefficientReadIndex < UE_ARRAY_COUNT(TruePeakPolyphaseCoefficients); CoefficientReadIndex += TruePeakPhaseCount)
				{
					const VectorRegister4Float PolyphaseCoefficients = VectorLoad(&TruePeakPolyphaseCoefficients[CoefficientReadIndex]);
					const VectorRegister4Float HistorySample = VectorLoadFloat1(&ChannelSignalHistory[HistoryReadIndex]);
					Accumulators = VectorMultiplyAdd(HistorySample, PolyphaseCoefficients, Accumulators);
					HistoryReadIndex = (HistoryReadIndex - 1) & HistoryIndexMask; // uint32 wraparound is fine here as a bitmask is applied.
				}

				const float MaxAbsValue = VectorReduceMax(VectorAbs(Accumulators));
				OutBuffer[SampleIndex] = FMath::Max(FMath::Abs(InputSample), MaxAbsValue);

				const uint32 HistoryWriteIndex = (PrevHistoryWriteIndex + 1) & HistoryIndexMask;
				ChannelSignalHistory[HistoryWriteIndex] = InputSample;
				PrevHistoryWriteIndex = HistoryWriteIndex;
			}
		}

		HistoryPosition = PrevHistoryWriteIndex;
	}


	// A simple utility that returns a smoothed value given audio input using an RC circuit.
	// Used for following the envelope of an audio stream.
	FEnvelopeFollower::FEnvelopeFollower()
	: FEnvelopeFollower(FEnvelopeFollowerInitParams{})
	{
	}

	FEnvelopeFollower::FEnvelopeFollower(const FEnvelopeFollowerInitParams& InParams)
	: MeanSquaredProcessor(InParams.SampleRate, InParams.NumChannels, InParams.AnalysisWindowMsec)
	, TruePeakProcessor((InParams.Mode == EPeakMode::TruePeak) ? InParams.NumChannels : 0)
	, Smoother(InParams.SampleRate, InParams.NumChannels, InParams.AttackTimeMsec, InParams.ReleaseTimeMsec, InParams.bIsAnalog)
	, NumChannels(InParams.NumChannels)
	, EnvMode(InParams.Mode)
	{
	}

	// Initialize the envelope follower
	void FEnvelopeFollower::Init(const FEnvelopeFollowerInitParams& InParams)
	{
		Smoother = FAttackReleaseSmoother(InParams.SampleRate, InParams.NumChannels, InParams.AttackTimeMsec, InParams.ReleaseTimeMsec, InParams.bIsAnalog);
		MeanSquaredProcessor = FMeanSquaredIIR(InParams.SampleRate, InParams.NumChannels, InParams.AnalysisWindowMsec);
		TruePeakProcessor = FTruePeak((InParams.Mode == EPeakMode::TruePeak) ? InParams.NumChannels : 0);
		NumChannels = InParams.NumChannels;
		EnvMode = InParams.Mode;
	}

	int32 FEnvelopeFollower::GetNumChannels() const
	{
		return NumChannels;
	}

	float FEnvelopeFollower::GetSampleRate() const
	{
		return Smoother.GetSampleRate();
	}

	float FEnvelopeFollower::GetAttackTimeMsec() const
	{
		return Smoother.GetAttackTimeMsec();
	}

	float FEnvelopeFollower::GetReleaseTimeMsec() const
	{
		return Smoother.GetReleaseTimeMsec();
	}

	bool FEnvelopeFollower::GetAnalog() const
	{
		return Smoother.GetAnalog();
	}

	EPeakMode::Type FEnvelopeFollower::GetMode() const
	{
		return EnvMode;
	}

	void FEnvelopeFollower::SetNumChannels(int32 InNumChannels)
	{
		if (InNumChannels != NumChannels)
		{
			Smoother.SetNumChannels(InNumChannels);
			MeanSquaredProcessor.SetNumChannels(InNumChannels);
			TruePeakProcessor.SetNumChannels((EnvMode == EPeakMode::TruePeak) ? InNumChannels : 0);
			NumChannels = InNumChannels;
		}
	}

	// Resets the state of the envelope follower
	void FEnvelopeFollower::Reset()
	{
		MeanSquaredProcessor.Reset();
		TruePeakProcessor.Reset();
		Smoother.Reset();
	}

	// Sets whether or not to use analog or digital time constants
	void FEnvelopeFollower::SetAnalog(bool bInIsAnalog)
	{
		Smoother.SetAnalog(bInIsAnalog);
	}

	// Sets the envelope follower attack time (how fast the envelope responds to input)
	void FEnvelopeFollower::SetAttackTime(float InAttackTimeMsec)
	{
		Smoother.SetAttackTime(InAttackTimeMsec);
	}

	// Sets the envelope follower release time (how slow the envelope dampens from input)
	void FEnvelopeFollower::SetReleaseTime(float InReleaseTimeMsec)
	{
		Smoother.SetReleaseTime(InReleaseTimeMsec);
	}

	// Sets the output mode of the envelope follower
	void FEnvelopeFollower::SetMode(EPeakMode::Type InMode)
	{
		EnvMode = InMode;

		TruePeakProcessor.SetNumChannels((InMode == EPeakMode::TruePeak) ? NumChannels : 0);
	}

	// Process the input audio buffer and returns the last envelope value
	void FEnvelopeFollower::ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer)
	{
		const int32 NumSamples = InNumFrames * NumChannels;
		if (NumSamples > 0)
		{
			ProcessWorkBuffer(InBuffer, InNumFrames);
			Smoother.ProcessAudio(WorkBuffer.GetData(), InNumFrames, OutBuffer);
		}
	}

	void FEnvelopeFollower::ProcessAudio(const float* InBuffer, int32 InNumFrames)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EnvelopeFollower::ProcessAudio);
		const int32 NumSamples = InNumFrames * NumChannels;
		if (NumSamples > 0)
		{
			ProcessWorkBuffer(InBuffer, InNumFrames);
			Smoother.ProcessAudio(WorkBuffer.GetData(), InNumFrames);
		}
	}

	void FEnvelopeFollower::ProcessWorkBuffer(const float* InBuffer, int32 InNumFrames)
	{
		const int32 NumSamples = InNumFrames * NumChannels;
		if (NumSamples > 0)
		{
			WorkBuffer.Reset();
			WorkBuffer.AddUninitialized(NumSamples);

			if (EPeakMode::RootMeanSquared == EnvMode)
			{
				MeanSquaredProcessor.ProcessAudio(InBuffer, InNumFrames, WorkBuffer.GetData());
				ArraySqrtInPlace(WorkBuffer);
			}
			else if (EPeakMode::MeanSquared == EnvMode)
			{
				MeanSquaredProcessor.ProcessAudio(InBuffer, InNumFrames, WorkBuffer.GetData());
			}
			else if (EPeakMode::Peak == EnvMode)
			{
				Audio::ArrayAbs(MakeArrayView(InBuffer, NumSamples), WorkBuffer);
			}
			else if (EPeakMode::TruePeak == EnvMode)
			{
				TruePeakProcessor.ProcessAudio(InBuffer, InNumFrames, WorkBuffer.GetData());
			}
			else
			{
				check(false);
			}
		}
	}

	const TArray<float>& FEnvelopeFollower::GetEnvelopeValues() const
	{
		return Smoother.GetEnvelopeValues();
	}
}
