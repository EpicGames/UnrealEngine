// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/EventQuantizer.h"

namespace Audio
{
	FEventQuantizer::FEventQuantizer()
		: FrameCount(0)
		, NumFramesPerBar(0)
		, NumFramesPerBeat(0)
		, EventQuantizationForSettingsChange(EEventQuantization::Bar)
		, bQuantizationSettingsSet(false)
		, bResetEventState(false)
	{
	}

	FEventQuantizer::~FEventQuantizer()
	{
	}

	void FEventQuantizer::SetQuantizationSettings(const FEventQuantizationSettings& InQuantizationSettings)
	{
		if (!FMemory::Memcmp(&InQuantizationSettings, &QuantizationSettings, sizeof(FEventQuantizationSettings)))
		{
			return;
		}

		if (!bQuantizationSettingsSet)
		{
			SetQuantizationSettingsInternal(InQuantizationSettings);
		}
		else
		{
			TFunction<void(uint32 NumFramesOffset)> Lambda = [this, InQuantizationSettings](uint32 NumFramesOffset)
			{
				SetQuantizationSettingsInternal(InQuantizationSettings);
			};

			BPMQuantizationState.QueuedEvents.Add(MoveTemp(Lambda));
		}
	}

	void FEventQuantizer::SetQuantizationSettingsInternal(const FEventQuantizationSettings& InQuantizationSettings)
	{
		QuantizationSettings = InQuantizationSettings;

		// Do some validation on input to make things a bit more bullet proof
		QuantizationSettings.BeatsPerBar = FMath::Max<uint32>(QuantizationSettings.BeatsPerBar, 1);
		QuantizationSettings.BeatsPerMinute = FMath::Max<uint32>(QuantizationSettings.BeatsPerMinute, 1);
		QuantizationSettings.NumChannels = FMath::Max<uint32>(QuantizationSettings.NumChannels, 1);
		QuantizationSettings.SampleRate = InQuantizationSettings.SampleRate;
		QuantizationSettings.BeatDivision = FMath::RoundUpToPowerOfTwo(InQuantizationSettings.BeatDivision);

		SetBPMInternal(QuantizationSettings.BeatsPerMinute);

		// allow these setting to cache, but do not reset event state until function is called with a valid sample rate
		if (!bQuantizationSettingsSet && (InQuantizationSettings.SampleRate > 0))
		{
			bQuantizationSettingsSet = true;
			ResetEventState();
		}
		else
		{
			bResetEventState = true;
		}

	}
	
	void FEventQuantizer::ResetEventState()
	{
		const float QuarterNoteTime = 60.0f / FMath::Max(1.0f, QuantizationSettings.BeatsPerMinute);
		const float BeatDivision = (float)QuantizationSettings.BeatDivision;
		const float BeatTimeSeconds = 4.0f * QuarterNoteTime / FMath::Max(1.0f, BeatDivision);

		// force a 32second note to be at least a single sample
		NumFramesPerBeat = (uint32)(BeatTimeSeconds * QuantizationSettings.SampleRate);
		NumFramesPerBar = FMath::Max(QuantizationSettings.BeatsPerBar * NumFramesPerBeat, 32u);

		for (int32 Index = 0; Index < (int32)EEventQuantization::Count; ++Index)
		{
			FEventQuantizationState& EventState = EventQuantizationStates[Index];
			EventState.EventFrameDuration = NumFramesPerBar;

			switch ((EEventQuantization)Index)
			{
				// No quantization means that it happens as soon as possible... 
			case EEventQuantization::None:
				EventState.EventFrameDuration = 1;
				break;

			case EEventQuantization::Bars8:
				EventState.EventFrameDuration *= 8;
				break;

			case EEventQuantization::Bars4:
				EventState.EventFrameDuration *= 4;
				break;

			case EEventQuantization::Bars2:
				EventState.EventFrameDuration *= 2;
				break;

			case EEventQuantization::Bar:
				// Already computed
				break;

			case EEventQuantization::HalfNote:
				EventState.EventFrameDuration /= 2;
				break;

			case EEventQuantization::HalfNoteTriplet:
				EventState.EventFrameDuration /= 3;
				break;

			case EEventQuantization::QuarterNote:
				EventState.EventFrameDuration = NumFramesPerBeat;
				break;

			case EEventQuantization::QuarterNoteTriplet:
				EventState.EventFrameDuration = NumFramesPerBeat * 2.0f / 3.0f;
				break;

			case EEventQuantization::EighthNote:
				EventState.EventFrameDuration = NumFramesPerBeat / 2.0f;
				break;

			case EEventQuantization::EighthNoteTriplet:
				EventState.EventFrameDuration = NumFramesPerBeat / 3.0f;
				break;

			case EEventQuantization::SixteenthNote:
				EventState.EventFrameDuration = NumFramesPerBeat / 4.0f;
				break;

			case EEventQuantization::SixteenthNoteTriplet:
				EventState.EventFrameDuration = NumFramesPerBeat / 6.0f;
				break;

			case EEventQuantization::ThirtySecondNote:
				EventState.EventFrameDuration = NumFramesPerBeat / 8.0f;
				break;

			default:
				checkf(false, TEXT("Need to update this loop for new quantization enumeration"));
				break;
			}

			// Set the events to trigger on the first sample
			EventState.FrameCount = EventState.EventFrameDuration - 1;
		}

		BPMQuantizationState.FrameCount = 0;
		BPMQuantizationState.EventFrameDuration = EventQuantizationStates[(int32)EventQuantizationForSettingsChange].EventFrameDuration;

		// Reset the global frame count on quantization change
		FrameCount = 0;
	}


	void FEventQuantizer::SetBPMInternal(const float InBPM)
	{
		// Store the BPM here in case it changes directly from public API call
		QuantizationSettings.BeatsPerMinute = InBPM;
	}

	void FEventQuantizer::SetBPM(const float InBPM)
	{
		if (!bQuantizationSettingsSet || FMath::IsNearlyEqual(QuantizationSettings.BeatsPerMinute, InBPM))
		{
			return;
		}

		TFunction<void(uint32 NumFramesOffset)> Lambda = [this, InBPM](uint32 NumFramesOffset)
		{
			SetBPMInternal(InBPM);
			bResetEventState = true;
		};

		BPMQuantizationState.QueuedEvents.Add(MoveTemp(Lambda));
	}

	void FEventQuantizer::SetBeatDivision(const uint16 InBeatDivision)
	{
		if (QuantizationSettings.BeatDivision == InBeatDivision)
		{
			return;
		}

		TFunction<void(uint32 NumFramesOffset)> Lambda = [this, InBeatDivision](uint32 NumFramesOffset)
		{
			QuantizationSettings.BeatDivision = (uint16)FMath::RoundUpToPowerOfTwo(InBeatDivision);

			SetBPMInternal(QuantizationSettings.BeatsPerMinute);
			bResetEventState = true;
		};

		BPMQuantizationState.QueuedEvents.Add(MoveTemp(Lambda));
	}

	void FEventQuantizer::NotifyEventForState(FEventQuantizationState& State, EEventQuantization Type, bool bIsQuantizationEvent, int32 NumFramesInCallback)
	{
		check(State.FrameCount != INDEX_NONE);
		check(State.EventFrameDuration != INDEX_NONE);

		uint32 NextFrameCount = State.FrameCount + NumFramesInCallback;

		bool bResetFrameCount = false;
		int32 NumEventsFired = 0;

		// Check the event trigger condition
		if (NextFrameCount >= State.EventFrameDuration)
		{
			// Compute the frame offset (basically where in this buffer the event starts)
			int32 NextEventFrameStart = (int32)(State.EventFrameDuration - State.FrameCount - 1);
			int32 PostEventFramesRemaining = NumFramesInCallback - NextEventFrameStart;
			check(PostEventFramesRemaining >= 0);

			// Support lambda callbacks (events) queuing more events.
			// Which may need to be executed within a single audio buffer. 
			// This allows arbitrarily fast event quantization execution.
			do
			{
				if (State.QueuedEvents.Num() > 0)
				{
					// Move the queued events to the copied events scratch list that things can queue up more events in the lambda
					CopiedEvents.Reset();
					CopiedEvents.Append(MoveTemp(State.QueuedEvents));

					State.QueuedEvents.Reset();

					// We use the copied events since this loop
					for (TFunction<void(uint32 NumFramesOffset)>& Event : CopiedEvents)
					{
						Event(NextEventFrameStart);
					}
				}

				// No support for interface delegate notifications for "none" quantization state
				if (Type == EEventQuantization::None || bIsQuantizationEvent)
				{
					bResetFrameCount = true;
					break;
				}

				// Do event listener notifications *with* the frame count with offset accounted for so 
				// callbacks will get exact frame counts of when this event happened.
				uint32 FrameCountWithOffset = FrameCount + NextEventFrameStart;
				int32 NumBars = FrameCountWithOffset / NumFramesPerBar;
				int32 NumBeatsFromStart = FrameCountWithOffset / (NumFramesPerBeat - 1);
				int32 NumBeatsInBar = NumBeatsFromStart % QuantizationSettings.BeatsPerBar;

				for (IQuantizedEventListener* EventListener : State.EventListeners)
				{
					if (EventListener)
					{
						EventListener->OnEvent(Type, NumBars, NumBeatsInBar);
					}
				}

				NextEventFrameStart += State.EventFrameDuration;
				PostEventFramesRemaining -= State.EventFrameDuration;
				++NumEventsFired;
			} while (PostEventFramesRemaining >= 0);

			// Wrap the frame count back to within the event frame duration range
			// but keep the phase of the frame
			int32 Temp = static_cast<int32>(NextFrameCount) - (NumEventsFired * static_cast<int32>(State.EventFrameDuration));
			State.FrameCount = FMath::Max(0, Temp);
		}
		else
		{
			// if no event, just accumulate the frame count
			State.FrameCount = NextFrameCount;
		}

		if (bResetFrameCount)
		{
			State.FrameCount = 0;
		}

		check(State.FrameCount >= 0 && State.FrameCount < State.EventFrameDuration);
	}

	void FEventQuantizer::NotifyEvents(int32 NumFrames)
	{
		// Can't do anything if no quantization information was set
		if (!bQuantizationSettingsSet)
		{
			return;
		}

		NotifyEventForState(BPMQuantizationState, EEventQuantization::Count, true, NumFrames);
	
		for (int32 StateIndex = 0; StateIndex < (int32)EEventQuantization::Count; ++StateIndex)
		{
			FEventQuantizationState& State = EventQuantizationStates[StateIndex];
			NotifyEventForState(State, (EEventQuantization)StateIndex, false, NumFrames);
		}

		// Check to see if we need to re-set up our event states. 
		// This can happen after a quantization setting has happened. Otherwise
		// we will be changing event state for the current quantization event (e.g. bar)
		// and that will cause mis-calculations for events which are queued to happen this callback.
		if (bResetEventState)
		{
			bResetEventState = false;
			ResetEventState();
		}

		// Increment the overall frame count here
		FrameCount += NumFrames;
	}

	float FEventQuantizer::GetPlaybacktimeSeconds() const
	{
		return (float) FrameCount / QuantizationSettings.SampleRate;
	}

	uint32 FEventQuantizer::GetDurationInFrames(int32 NumBars, float NumBeats) const
	{
		return QuantizationSettings.BeatsPerBar * NumBars * NumFramesPerBeat + NumBeats * NumFramesPerBeat;
	}

	void FEventQuantizer::EnqueueEvent(EEventQuantization InQuantization, TFunction<void(uint32 NumFramesOffset)> Lambda)
	{
		EventQuantizationStates[(int32)InQuantization].QueuedEvents.Add(MoveTemp(Lambda));
	}

	void FEventQuantizer::RegisterListenerForEvent(IQuantizedEventListener* InListener, EEventQuantization InQuantization)
	{
		if (InQuantization == EEventQuantization::None)
		{
			return;
		}

		EventQuantizationStates[(int32)InQuantization].EventListeners.AddUnique(InListener);
	}

	void FEventQuantizer::UnregisterListenerForEvent(IQuantizedEventListener* InListener)
	{
		for (int32 Index = 0; Index < (int32)EEventQuantization::Count; ++Index)
		{
			EventQuantizationStates[Index].EventListeners.Remove(InListener);
		}
	}

	void FEventQuantizer::UnregisterListenerForEvent(IQuantizedEventListener* InListener, EEventQuantization InQuantization)
	{
		EventQuantizationStates[(int32)InQuantization].EventListeners.Remove(InListener);
	}




}