// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/FFTAlgorithm.h"
#include "SignalProcessingModule.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

namespace Audio
{
	// Return factories which support settings and factory name.
	static const TArray<IFFTAlgorithmFactory*> GetPrioritizedSupportingFactories(const FFFTSettings& InSettings, const FName& InAlgorithmFactoryName)
	{

		// Get all IFFTAlgorithm factories. 
		IModularFeatures::Get().LockModularFeatureList();
		TArray<IFFTAlgorithmFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<IFFTAlgorithmFactory>(IFFTAlgorithmFactory::GetModularFeatureName());
		IModularFeatures::Get().UnlockModularFeatureList();

		// Remove null factories
		Factories = Factories.FilterByPredicate([InAlgorithmFactoryName](const IFFTAlgorithmFactory* Factory) 
		{
			return (nullptr != Factory);
		});

		if (0 == Factories.Num())
		{
			UE_LOGF(LogSignalProcessing, Warning, "No registered IFFTAlgorithmFactories exist.");
			return Factories;
		}

		if (FFFTFactory::AnyAlgorithmFactory != InAlgorithmFactoryName)
		{
			// Filter factories if one has been specified. 
			Factories = Factories.FilterByPredicate([InAlgorithmFactoryName](const IFFTAlgorithmFactory* Factory) 
			{
				return (InAlgorithmFactoryName == Factory->GetFactoryName());
			});

			if (0 == Factories.Num())
			{
				UE_LOGF(LogSignalProcessing, Warning, "No IFFTAlgorithmFactories named \"%ls\" found.", *InAlgorithmFactoryName.ToString());
				return Factories;
			}
		}

		if (!InSettings.bEnableHardwareAcceleration)
		{
			// Remove factories with hardware acceleration if it has been disabled. 
			Factories = Factories.FilterByPredicate([InAlgorithmFactoryName](const IFFTAlgorithmFactory* Factory) 
			{
				return !Factory->IsHardwareAccelerated();
			});
		}

		if (!InSettings.bArrays128BitAligned)
		{
			// Remove factories requiring aligned input/output arrays if it has been disabled.
			Factories = Factories.FilterByPredicate([InAlgorithmFactoryName](const IFFTAlgorithmFactory* Factory) 
			{
				return !Factory->Expects128BitAlignedArrays();
			});
		}

		// Filter by whether factory supports settings.
		Factories = Factories.FilterByPredicate([InSettings](const IFFTAlgorithmFactory* Factory)
		{
			return Factory->AreFFTSettingsSupported(InSettings);
		});

		// We have two software versions, but should prefer the vectorized version
		// over the non-vectorized version.
		static const TArray<FName> InternalFFTFActoryPriority = {
			FName("FVectorFFTFactory"),
			FName("OriginalFFT_Deprecated")
		};

		// Sort factories by preference. 
		Factories.Sort([](const IFFTAlgorithmFactory& InFactoryA, const IFFTAlgorithmFactory& InFactoryB) 
		{
			// If InFactoryA has hardware acceleration and InFactoryB does not, then InFactoryA goes earlier.
			if (InFactoryA.IsHardwareAccelerated() && !InFactoryB.IsHardwareAccelerated())
			{
				return true;
			}

			// If InFactoryA uses aligned arrays and InFactoryB does not, then InFactoryA goes earlier.
			if (InFactoryA.Expects128BitAlignedArrays() && !InFactoryB.Expects128BitAlignedArrays())
			{
				return true;
			}

			// Check internal priority of software implementations.
			int32 PriorityIndexA = InternalFFTFActoryPriority.Find(InFactoryA.GetFactoryName());
			int32 PriorityIndexB = InternalFFTFActoryPriority.Find(InFactoryB.GetFactoryName());

			if ((PriorityIndexA != INDEX_NONE) && (PriorityIndexB != INDEX_NONE))
			{
				return PriorityIndexA < PriorityIndexB;
				
			}

			// InFactoryA does not need to be placed before InFactoryB
			return false;
		});
		
		return Factories;
	}

	/** virtual destructor for inheritance. */
	IFFTAlgorithm::~IFFTAlgorithm()
	{
	}

	/** Name of modular feature for FFT factory.  */
	const FName IFFTAlgorithmFactory::GetModularFeatureName()
	{
		static const FName ModularFeatureName = FName("AudioFFTAlgorithmFactory");
		return ModularFeatureName;
	}

	/** This denotes that no specific IFFTAlgorithmFactory is desired. */
	const FName FFFTFactory::AnyAlgorithmFactory = FName("AnyAlgorithmFactory");

	/** NewFFTAlgorithm
	 *
	 * Creates and returns a new FFTAlgorithm. 
	 *
	 * @param InSettings - The settings used to create the FFT algorithm.
	 * @param InAlgorithmFactoryName - If not equal to FFFTFactory::AnyAlgorithmFactory, will only uses FFT algorithm facotry if IFFTAlgorithmFactory::GetFactoryName() equals InAlgorithmFactoryName.
	 * @return A TUniquePtr<IFFTAlgorithm> to the created FFT. This pointer can be invalid if an error occured or the fft algorithm could not be created.
	 */
	TUniquePtr<IFFTAlgorithm> FFFTFactory::NewFFTAlgorithm(const FFFTSettings& InSettings, const FName& InAlgorithmFactoryName)
	{
		TArray<IFFTAlgorithmFactory*> Factories = GetPrioritizedSupportingFactories(InSettings, InAlgorithmFactoryName);

		// Find first algorithm which creates the an algorithm from the input settings. 
		for (IFFTAlgorithmFactory* Factory : Factories)
		{
			TUniquePtr<IFFTAlgorithm> FFTAlgorithm = Factory->NewFFTAlgorithm(InSettings);

			if (!FFTAlgorithm.IsValid())
			{
				UE_LOGF(LogSignalProcessing, Warning, "IFFTAlgorithmFactory failed to create IFFTAlgorithm despite supporting FFT Settings.");
			}
			else
			{
				return FFTAlgorithm;
			}
		}


		UE_LOGF(LogSignalProcessing, Warning, "No IFFTAlgorithmFactories can create IFFTAlgorithm for given settings.");
		return TUniquePtr<IFFTAlgorithm>();
	}

	bool FFFTFactory::AreFFTSettingsSupported(const FFFTSettings& InSettings, const FName& InAlgorithmFactoryName)
	{
		TArray<IFFTAlgorithmFactory*> Factories = GetPrioritizedSupportingFactories(InSettings, InAlgorithmFactoryName);
		return Factories.Num() > 0;
	}
}
