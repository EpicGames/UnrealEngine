// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/FeedbackDelayNetwork.h"

namespace Audio
{
	// The presets for the late refelections reverb were based on a strange sample rate. This 
	// function helps translate delay values in the paper to delay values for this reverb.
	// https://ccrma.stanford.edu/~dattorro/EffectDesignPart1.pdf
	static int32 FeedbackDelayNetworkGetDelaySamples(float InSampleRate, float InPresetValue) {
		
		return (int32)((float)InSampleRate / 29761.f * InPresetValue);
	}

	FFDNDelaySettings FFDNDelaySettings::DefaultLeftDelays(float InSampleRate)
	{
		FFDNDelaySettings Settings;
		Settings.APF0DelayNumSamples = FeedbackDelayNetworkGetDelaySamples(InSampleRate, 5 * 142);
		Settings.APF1DelayNumSamples = FeedbackDelayNetworkGetDelaySamples(InSampleRate, 5 * 107);
		Settings.APF2DelayNumSamples = FeedbackDelayNetworkGetDelaySamples(InSampleRate, 5 * 379);
		Settings.APF3DelayNumSamples = FeedbackDelayNetworkGetDelaySamples(InSampleRate, 5 * 277);
		return Settings;
	}

	FFDNDelaySettings FFDNDelaySettings::DefaultRightDelays(float InSampleRate)
	{
		FFDNDelaySettings Settings;
		Settings.APF0DelayNumSamples = FeedbackDelayNetworkGetDelaySamples(InSampleRate, 5 * 279);
		Settings.APF1DelayNumSamples = FeedbackDelayNetworkGetDelaySamples(InSampleRate, 5 * 137);
		Settings.APF2DelayNumSamples = FeedbackDelayNetworkGetDelaySamples(InSampleRate, 5 * 213);
		Settings.APF3DelayNumSamples = FeedbackDelayNetworkGetDelaySamples(InSampleRate, 5 * 327);
		return Settings;
	}

	FFeedbackDelayNetwork::FFeedbackDelayNetwork(int32 InMaxNumInternalBufferSamples, const FFDNDelaySettings& InSettings)
	:	Settings(InSettings)
	{
		// num internal buffer samples must be less than the minimum delay line and abide by memory alignment.
		NumInternalBufferSamples = FMath::Min(
			FMath::Min(Settings.APF0DelayNumSamples, Settings.APF1DelayNumSamples),
			FMath::Min(Settings.APF2DelayNumSamples, Settings.APF3DelayNumSamples));

		NumInternalBufferSamples = FMath::Min(NumInternalBufferSamples, InMaxNumInternalBufferSamples);

		NumInternalBufferSamples -= (NumInternalBufferSamples % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);

		if (NumInternalBufferSamples <= 0)
		{
			NumInternalBufferSamples = 1;
		}

		// Allocate delay lines.
		DelayLine0 = MakeUnique<FAlignedBlockBuffer>((2 * Settings.APF0DelayNumSamples) + NumInternalBufferSamples, NumInternalBufferSamples);
		DelayLine1 = MakeUnique<FAlignedBlockBuffer>((2 * Settings.APF1DelayNumSamples) + NumInternalBufferSamples, NumInternalBufferSamples);
		DelayLine2 = MakeUnique<FAlignedBlockBuffer>((2 * Settings.APF2DelayNumSamples) + NumInternalBufferSamples, NumInternalBufferSamples);
		DelayLine3 = MakeUnique<FAlignedBlockBuffer>((2 * Settings.APF3DelayNumSamples) + NumInternalBufferSamples, NumInternalBufferSamples);

		// Initialize delay lines
		DelayLine0->AddZeros(Settings.APF0DelayNumSamples);
		DelayLine1->AddZeros(Settings.APF1DelayNumSamples);
		DelayLine2->AddZeros(Settings.APF2DelayNumSamples);
		DelayLine3->AddZeros(Settings.APF3DelayNumSamples);

		// Initialize internal buffers
		WorkBuffer0.Reset(NumInternalBufferSamples);
		WorkBuffer1.Reset(NumInternalBufferSamples);
		WorkBuffer2.Reset(NumInternalBufferSamples);
		WorkBuffer3.Reset(NumInternalBufferSamples);
		WorkBuffer0.AddUninitialized(NumInternalBufferSamples);
		WorkBuffer1.AddUninitialized(NumInternalBufferSamples);
		WorkBuffer2.AddUninitialized(NumInternalBufferSamples);
		WorkBuffer3.AddUninitialized(NumInternalBufferSamples);

		// Initialize filter memory
		for (int32 i = 0; i < 4; i++)
		{
			FMO[i] = 0.0f;
			LPFZ[i] = 0.0f;
		}
	}

	FFeedbackDelayNetwork::~FFeedbackDelayNetwork()
	{}

	void FFeedbackDelayNetwork::SetCoefficients(const FFDNCoefficients& InCoefficients)
	{
		Coefficients = InCoefficients;
	}

	void FFeedbackDelayNetwork::ProcessAudio(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples)
	{
		const int32 InNum = InSamples.Num();
		const float* InSampleData = InSamples.GetData();

		// Prepare output buffer
		OutSamples.Reset(InNum);
		OutSamples.AddUninitialized(InNum);

		// Process audio one block at a time
		float* OutSampleData = OutSamples.GetData();
		int32 LeftToProcess = InNum;
		int32 BufferPos = 0;

		while (LeftToProcess > 0)
		{
			int32 NumToProcess = FMath::Min(NumInternalBufferSamples, LeftToProcess);
			ProcessAudioBuffer(&InSampleData[BufferPos], NumToProcess, &OutSampleData[BufferPos]);
			LeftToProcess -= NumToProcess;
			BufferPos += NumToProcess;
		}

	}

	int32 FFeedbackDelayNetwork::GetNumInternalBufferSamples() const
	{
		return NumInternalBufferSamples;
	}

	void FFeedbackDelayNetwork::ProcessAudioBuffer(const float* InSampleData, const int32 InNum, float* OutSampleData)
	{
		const float* DelayLineData0 = DelayLine0->InspectSamples(InNum);
		const float* DelayLineData1 = DelayLine1->InspectSamples(InNum);
		const float* DelayLineData2 = DelayLine2->InspectSamples(InNum);
		const float* DelayLineData3 = DelayLine3->InspectSamples(InNum);
		float* WorkBuffer0Data = WorkBuffer0.GetData();
		float* WorkBuffer1Data = WorkBuffer1.GetData();
		float* WorkBuffer2Data = WorkBuffer2.GetData();
		float* WorkBuffer3Data = WorkBuffer3.GetData();
		
		float AllPassOut[4] = {0.0f};
		float DelayInput[4] = {0.0f};
		for (int32 i = 0; i < InNum; i++)
		{
			
			float DelayOut[4] = {DelayLineData0[i], DelayLineData1[i], DelayLineData2[i], DelayLineData3[i]};

			// Scale input
			float ScaledInput = Coefficients.InputScale * InSampleData[i];
			
			// Calculate input to delay line. 
			for (int32 j = 0; j < 4; j++)
			{
				DelayInput[j] = (ScaledInput + FMO[j]) + (Coefficients.APFG[j] * DelayOut[j]);
			}

			// Store delay line values
			WorkBuffer0Data[i] = DelayInput[0];
			WorkBuffer1Data[i] = DelayInput[1];
			WorkBuffer2Data[i] = DelayInput[2];
			WorkBuffer3Data[i] = DelayInput[3];

			// complete all pass filters and process low pass filters
			OutSampleData[i] = 0.0f;
			for (int32 j = 0; j < 4; j++)
			{
				AllPassOut[j] = (DelayInput[j] * -Coefficients.APFG[j]) + DelayOut[j];
				// Note the reasignment of lpz. 
				LPFZ[j] = (AllPassOut[j] * Coefficients.LPFA[j]) + (LPFZ[j] * Coefficients.LPFB[j]);
				OutSampleData[i] += LPFZ[j];
			}

			// Mix channels and apply feedback.
			FMO[0] = Coefficients.Feedback * (-LPFZ[1] + LPFZ[2]);
			FMO[1] = Coefficients.Feedback * (LPFZ[0] + LPFZ[3]);
			FMO[2] = Coefficients.Feedback * (LPFZ[0] - LPFZ[3]);
			FMO[3] = Coefficients.Feedback * (-LPFZ[1] - LPFZ[2]);

		}

		// Update delay lines.
		DelayLine0->RemoveSamples(InNum);
		DelayLine1->RemoveSamples(InNum);
		DelayLine2->RemoveSamples(InNum);
		DelayLine3->RemoveSamples(InNum);
		DelayLine0->AddSamples(WorkBuffer0Data, InNum);
		DelayLine1->AddSamples(WorkBuffer1Data, InNum);
		DelayLine2->AddSamples(WorkBuffer2Data, InNum);
		DelayLine3->AddSamples(WorkBuffer3Data, InNum);
	}

	void FFeedbackDelayNetwork::FlushAudio()
	{
		for (int32 i = 0; i < 4; i++)
		{
			LPFZ[i] = 0.f;
			FMO[i] = 0.f;
		}

		DelayLine0->ClearSamples();
		DelayLine0->AddZeros(Settings.APF0DelayNumSamples);

		DelayLine1->ClearSamples();
		DelayLine1->AddZeros(Settings.APF1DelayNumSamples);

		DelayLine2->ClearSamples();
		DelayLine2->AddZeros(Settings.APF2DelayNumSamples);

		DelayLine3->ClearSamples();
		DelayLine3->AddZeros(Settings.APF3DelayNumSamples);
	}
}
