// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/FileEncoder.h"
#include "DSP/Encoders/OggVorbisEncoder.h"
#include "DSP/Encoders/OpusEncoder.h"
#include "DSP/Encoders/WavEncoder.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformFile.h"

namespace Audio
{
	FAudioFileWriter::FAudioFileWriter(const FString& InPath, const FSoundQualityInfo& InInfo)
		: QualityInfo(InInfo)
		, Encoder(GetNewEncoderForFile(InPath))
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		FileHandle.Reset(PlatformFile.OpenWrite(*InPath));
	}

	FAudioFileWriter::~FAudioFileWriter()
	{
		check(Encoder.IsValid() && FileHandle.IsValid());

		int32 DataBufferSize = Encoder->Finalize();
		FlushEncoderToFile(DataBufferSize);
	}

	void FAudioFileWriter::GetFileInfo(FSoundQualityInfo& OutInfo)
	{
		OutInfo = QualityInfo;
	}

	bool FAudioFileWriter::PushAudio(const float* InAudio, int32 NumSamples, bool bEncodeIfPossible /*= true*/)
	{
		check(Encoder.IsValid() && FileHandle.IsValid());
		bool bSuccess = Encoder->PushAudio(InAudio, NumSamples, bEncodeIfPossible);

		if (bEncodeIfPossible && bSuccess)
		{
			FlushEncoderToFile();
			return true;
		}
		else
		{
			return bSuccess;
		}
	}

	bool FAudioFileWriter::EncodeIfPossible()
	{
		check(Encoder.IsValid() && FileHandle.IsValid());

		if (Encoder->EncodeIfPossible())
		{
			FlushEncoderToFile();
			return true;
		}
		else
		{
			return false;
		}
	}

	Audio::IAudioEncoder* FAudioFileWriter::GetNewEncoderForFile(const FString& InPath)
	{
		FString Extension = GetExtensionForFile(InPath);

		static const FString OpusExtension = "opus";
		static const FString OggExtension = "ogg";
		static const FString WavExtension = "wav";


		if (Extension.Equals(WavExtension))
		{
			return new FWavEncoder(QualityInfo, 4096);
		}
#if PLATFORM_SUPPORTS_VORBIS_ENCODE
		if (Extension.Equals(OggExtension))
		{
			return new FOggVorbisEncoder(QualityInfo, 4096);
		}
#endif
#if PLATFORM_SUPPORTS_OPUS_CODEC 
		if (Extension.Equals(OpusExtension))
		{
			return new FOpusEncoder(QualityInfo, 4096);
		}
#endif
		UE_LOGF(LogTemp, Error, "Invalid file extension %ls.", *Extension);
		return nullptr;
	}

	FString FAudioFileWriter::GetExtensionForFile(const FString& InPath)
	{
		int32 Index = INDEX_NONE;
		if (InPath.FindLastChar(TCHAR('.'), Index))
		{
			return InPath.RightChop(Index + 1);
		}
		else
		{
			return FString();
		}
	}

	void FAudioFileWriter::FlushEncoderToFile(int32 DataBufferSize /* = 4096*/)
	{
		check(Encoder.IsValid() && FileHandle.IsValid());
		
		if (DataBufferSize == 0)
		{
			return;
		}

		int32 BytesToWrite = -1;
		do
		{
			DataBuffer.Reset();
			DataBuffer.AddUninitialized(DataBufferSize);

			BytesToWrite = Encoder->PopData(DataBuffer.GetData(), DataBuffer.Num());
			bool WriteResult = FileHandle->Write(DataBuffer.GetData(), BytesToWrite);
			check(WriteResult);

		} while (BytesToWrite == DataBufferSize);
	}
}
