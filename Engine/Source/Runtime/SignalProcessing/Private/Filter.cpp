// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Filter.h"

#include <complex>
#include "DSP/Dsp.h"

namespace Audio
{
	// Simple biquad filter structure handling a biquad formulation
	// See: https://en.wikipedia.org/wiki/Digital_biquad_filter
	// Calculations of coefficients are handled outside this class.
	// Filter coefficients are public and are intended to be used externally.
	struct FBiquadFilter::FBiquadCoeff
	{
	public:
		FBiquadCoeff()
			: A0(1.0f)
			, A1(0.0f)
			, A2(0.0f)
			, B1(0.0f)
			, B2(0.0f)
		{
			Reset();
		}

		// Reset the filter (flush delays)
		void Reset()
		{
			X_Z1 = 0.0f;
			X_Z2 = 0.0f;
			Y_Z1 = 0.0f;
			Y_Z2 = 0.0f;
		}

		FORCEINLINE float ProcessAudio(const float InSample)
		{
			// Use the biquad difference eq: y(n) = a0*x(n) + a1*x(n-1) + a2*x(n-2) - b1*y(n-1) - b2*y(n-2) 
			const float Output = A0 * InSample + A1 * X_Z1 + A2 * X_Z2 - B1 * Y_Z1 - B2 * Y_Z2;

			// Apply the z-transforms
			Y_Z2 = Y_Z1;
			Y_Z1 = Output;

			X_Z2 = X_Z1;
			X_Z1 = InSample;

			return Output;
		}

		// Biquad filter coefficients
		float A0;
		float A1;
		float A2;
		float B1;
		float B2;

		float X_Z1; // previous inputs z-transforms
		float X_Z2;
		float Y_Z1; // prvious outputs z-transforms
		float Y_Z2;
	};

    float FBiquadFilter::ClampCutoffFrequency(float InCutoffFrequency)
    {
        return FMath::Clamp(InCutoffFrequency, 5.0f, 0.9f * (SampleRate / 2.0f));
    }
    
	FBiquadFilter::FBiquadFilter()
		: FilterType(EBiquadFilter::Lowpass)
		, Biquad(nullptr)
		, SampleRate(0.0f)
		, NumChannels(0)
		, Frequency(0.0f)
		, Bandwidth(0.0f)
		, GainDB(0.0f)
		, bEnabled(true)
	{
	}

	FBiquadFilter::~FBiquadFilter()
	{
		if (Biquad)
		{
			delete[] Biquad;
		}
	}

	void FBiquadFilter::Init(const float InSampleRate, const int32 InNumChannels, const EBiquadFilter::Type InFilterType, const float InCutoffFrequency, const float InBandwidth, const float InGainDB)
	{
		SampleRate = InSampleRate;
		NumChannels = InNumChannels;
		FilterType = InFilterType;
		Frequency = ClampCutoffFrequency(InCutoffFrequency);
		Bandwidth = InBandwidth;
		GainDB = InGainDB;

		if (Biquad)
		{
			delete[] Biquad;
		}

		Biquad = new FBiquadCoeff[NumChannels];
		Reset();
		CalculateBiquadCoefficients();
	}

	int32 FBiquadFilter::GetNumChannels() const
	{
		return NumChannels;
	}

	void FBiquadFilter::ProcessAudioFrame(const float* InFrame, float* OutFrame)
	{
		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			OutFrame[Channel] = Biquad[Channel].ProcessAudio(InFrame[Channel]);
		}
	}

	void FBiquadFilter::ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer)
	{
		if (bEnabled)
		{
			if (NumChannels == 1)
			{
				// keep these in registers to prevent loading and storing them on every loop iteration
				float A0 = Biquad->A0;
				float A1 = Biquad->A1;
				float A2 = Biquad->A2;
				float B1 = Biquad->B1;
				float B2 = Biquad->B2;
				float X_Z1 = Biquad->X_Z1;
				float X_Z2 = Biquad->X_Z2;
				float Y_Z1 = Biquad->Y_Z1;
				float Y_Z2 = Biquad->Y_Z2;

				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
				{
					const float InSample = InBuffer[SampleIndex];
					// Use the biquad difference eq: y(n) = a0*x(n) + a1*x(n-1) + a2*x(n-2) - b1*y(n-1) - b2*y(n-2) 
					const float Output = A0 * InSample + A1 * X_Z1 + A2 * X_Z2 - B1 * Y_Z1 - B2 * Y_Z2;

					// Apply the z-transforms
					Y_Z2 = Y_Z1;
					Y_Z1 = Output;

					X_Z2 = X_Z1;
					X_Z1 = InSample;

					OutBuffer[SampleIndex] = Output;
				}

				Biquad->X_Z1 = X_Z1;
				Biquad->X_Z2 = X_Z2;
				Biquad->Y_Z1 = Y_Z1;
				Biquad->Y_Z2 = Y_Z2;
			}
			else
			{
				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
				{
					ProcessAudioFrame(&InBuffer[SampleIndex], &OutBuffer[SampleIndex]);
				}
			}
		}
	}

	void FBiquadFilter::ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers)
	{
		if (bEnabled)
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const float* Source = InBuffers[Channel];
				float* Destination = OutBuffers[Channel];
				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
				{
					Destination[SampleIndex] = Biquad[Channel].ProcessAudio(Source[SampleIndex]);
				}
			}
		}
	}

	void FBiquadFilter::Reset()
	{
		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			Biquad[Channel].Reset();
		}
	}

	void FBiquadFilter::SetParams(const EBiquadFilter::Type InFilterType, const float InCutoffFrequency, const float InBandwidth, const float InGainDB)
	{
		const float InCutoffFrequencyClamped = ClampCutoffFrequency(InCutoffFrequency);

		if (FilterType != InFilterType ||
			!FMath::IsNearlyEqual(Frequency, InCutoffFrequencyClamped) ||
			!FMath::IsNearlyEqual(Bandwidth, InBandwidth) ||
			!FMath::IsNearlyEqual(GainDB, InGainDB))
		{
			FilterType = InFilterType;
			Frequency = InCutoffFrequencyClamped;
			Bandwidth = InBandwidth;
			GainDB = InGainDB;
			CalculateBiquadCoefficients();
		}
	}

	void FBiquadFilter::SetType(const EBiquadFilter::Type InFilterType)
	{
		if (FilterType != InFilterType)
		{
			FilterType = InFilterType;
			CalculateBiquadCoefficients();
		}
	}

	void FBiquadFilter::SetFrequency(const float InCutoffFrequency)
	{
        const float InCutoffFrequencyClamped = ClampCutoffFrequency(InCutoffFrequency);
		if (Frequency != InCutoffFrequencyClamped)
		{
			Frequency = InCutoffFrequencyClamped;
			CalculateBiquadCoefficients();
		}
	}

	void FBiquadFilter::SetBandwidth(const float InBandwidth)
	{
		if (Bandwidth != InBandwidth)
		{
			Bandwidth = InBandwidth;
			CalculateBiquadCoefficients();
		}
	}

	void FBiquadFilter::SetGainDB(const float InGainDB)
	{
		if (GainDB != InGainDB)
		{
			GainDB = InGainDB;
			if (FilterType == EBiquadFilter::ParametricEQ
				|| FilterType == EBiquadFilter::HighShelf
				|| FilterType == EBiquadFilter::LowShelf)
			{
				CalculateBiquadCoefficients();
			}
		}
	}

	void FBiquadFilter::SetEnabled(const bool bInEnabled)
	{
		bEnabled = bInEnabled;
	}

	void FBiquadFilter::ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		constexpr int32 ChannelIndex = 0;
		check(ChannelIndex < NumChannels);
		const FBiquadCoeff& Coefficients = Biquad[ChannelIndex];

		const int32 NumFloats = InOutComplexValues.Num();
		check(NumFloats % 2 == 0);
		for (int32 Index = 0; Index < NumFloats; Index += 2)
		{
			const float ZReal = InOutComplexValues[Index + 0];
			const float ZImag = InOutComplexValues[Index + 1];
			const std::complex<float> Z(ZReal, ZImag);

			// H(z) == (a0*z^2 + a1*z + a2) / (z^2 + b1*z + b2)
			const std::complex<float> Numerator = (Coefficients.A0 * Z + Coefficients.A1) * Z + Coefficients.A2;
			const std::complex<float> Denominator = (Z + Coefficients.B1) * Z + Coefficients.B2;
			const std::complex<float> H = Numerator / Denominator;

			InOutComplexValues[Index + 0] = H.real();
			InOutComplexValues[Index + 1] = H.imag();
		}
	}

	void FBiquadFilter::CalculateBiquadCoefficients()
	{
		static const float NaturalLog2 = 0.69314718055994530942f;

		const float Omega = 2.0f * PI * Frequency / SampleRate;
		const float Sn = FMath::Sin(Omega);
		const float Cs = FMath::Cos(Omega);

		const float Alpha = Sn * FMath::Sinh(0.5f * NaturalLog2 * Bandwidth * Omega / Sn);

		float a0;
		float a1;
		float a2;
		float b0;
		float b1;
		float b2;

		switch (FilterType)
		{
			default:
			case EBiquadFilter::Lowpass:
			{
				a0 = (1.0f - Cs) / 2.0f;
				a1 = (1.0f - Cs);
				a2 = (1.0f - Cs) / 2.0f;
				b0 = 1.0f + Alpha;
				b1 = -2.0f * Cs;
				b2 = 1.0f - Alpha;
			}
			break;

			case EBiquadFilter::Highpass:
			{
				a0 = (1.0f + Cs) / 2.0f;
				a1 = -(1.0f + Cs);
				a2 = (1.0f + Cs) / 2.0f;
				b0 = 1.0f + Alpha;
				b1 = -2.0f * Cs;
				b2 = 1.0f - Alpha;
			}
			break;

			case EBiquadFilter::Bandpass:
			{
				a0 = Alpha;
				a1 = 0.0f;
				a2 = -Alpha;
				b0 = 1.0f + Alpha;
				b1 = -2.0f * Cs;
				b2 = 1.0f - Alpha;
			}
			break;

			case EBiquadFilter::Notch:
			{
				a0 = 1.0f;
				a1 = -2.0f * Cs;
				a2 = 1.0f;
				b0 = 1.0f + Alpha;
				b1 = -2.0f * Cs;
				b2 = 1.0f - Alpha;
			}
			break;

			case EBiquadFilter::ParametricEQ:
			{
				const float Amp = FMath::Pow(10.0f, GainDB / 40.0f);

				a0 = 1.0f + (Alpha * Amp);
				a1 = -2.0f * Cs;
				a2 = 1.0f - (Alpha * Amp);
				b0 = 1.0f + (Alpha / Amp);
				b1 = -2.0f * Cs;
				b2 = 1.0f - (Alpha / Amp);
			}
			break;

			case EBiquadFilter::HighShelf:
			{
				const float Amp = FMath::Pow(10.0f, GainDB / 40.0f);
				const float BetaSn = Sn * FMath::Sqrt(2.0f * Amp);

				a0 = Amp * ((Amp + 1) + (Amp - 1) * Cs + BetaSn);
				a1 = -2 * Amp * ((Amp - 1) + (Amp + 1) * Cs);
				a2 = Amp * ((Amp + 1) + (Amp - 1) * Cs - BetaSn);
				b0 = (Amp + 1) - (Amp - 1) * Cs + BetaSn;
				b1 = 2 * ((Amp - 1) - (Amp + 1) * Cs);
				b2 = (Amp + 1) - (Amp - 1) * Cs - BetaSn;
			}
			break;

			case EBiquadFilter::LowShelf:
			{
				const float Amp = FMath::Pow(10.0f, GainDB / 40.0f);
				const float BetaSn = Sn * FMath::Sqrt(2.0f * Amp);

				a0 = Amp * ((Amp + 1) - (Amp - 1) * Cs + BetaSn);
				a1 = 2 * Amp * ((Amp - 1) - (Amp + 1) * Cs);
				a2 = Amp * ((Amp + 1) - (Amp - 1) * Cs - BetaSn);
				b0 = (Amp + 1) + (Amp - 1) * Cs + BetaSn;
				b1 = -2 * ((Amp - 1) + (Amp + 1) * Cs);
				b2 = (Amp + 1) + (Amp - 1) * Cs - BetaSn;
			}
			break;

			case EBiquadFilter::AllPass:
			{
				a0 = 1.0f - Alpha;
				a1 = -2.0f * Cs;
				a2 = 1.0f + Alpha;
				b0 = 1.0f + Alpha;
				b1 = -2.0f * Cs;
				b2 = 1.0f - Alpha;
			}
			break;

			case EBiquadFilter::ButterworthLowPass:
			{
				float Lambda = 1.f / FMath::Tan(PI * Frequency / SampleRate);
				float OneOverQ = 2.f * Alpha / Sn;
				float LambdaScaled = UE_SQRT_2 * Lambda * OneOverQ;
				float LambdaSq = Lambda * Lambda;

				a0 = 1.f / (1.f + LambdaScaled + LambdaSq);
				a1 = 2.f * a0;
				a2 = a0;

				b0 = 1.f;
				b1 = a1 * (1.f - LambdaSq);
				b2 = a0 * (1.f - LambdaScaled + LambdaSq);
			}
			break;

			case EBiquadFilter::ButterworthHighPass:
			{
				float Lambda = FMath::Tan(PI * Frequency / SampleRate);
				float OneOverQ = 2.f * Alpha / Sn;
				float LambdaScaled = UE_SQRT_2 * Lambda * OneOverQ;
				float LambdaSq = Lambda * Lambda;

				a0 = 1.f / (1.f + LambdaScaled + LambdaSq);
				a1 = -2.f * a0;
				a2 = a0;

				b0 = 1.f;
				b1 = -a1 * (LambdaSq - 1.f);
				b2 = a0 * (1.f - LambdaScaled + LambdaSq);
			}
			break;
		}

		// Protect against indefinite filter coefficients. Once a NaN or inf gets
		// into the filter, it cannot return to normal operation unless the history
		// samples are reset. 
		if (FMath::IsFinite(b0))
		{
			a0 /= b0;
			a1 /= b0;
			a2 /= b0;
			b1 /= b0;
			b2 /= b0;
		}
		else
		{
			// Coefficients for silence.
			a0 = 0.f;
			a1 = 0.f;
			a2 = 0.f;
			b0 = 1.f;
			b1 = 0.f;
			b2 = 0.f;
		}

		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			Biquad[Channel].A0 = a0;
			Biquad[Channel].A1 = a1;
			Biquad[Channel].A2 = a2;
			Biquad[Channel].B1 = b1;
			Biquad[Channel].B2 = b2;
		}
	}

	static const float MaxFilterFreq = 18000.0f;
	static const float MinFilterFreq = 80.0f;

	IFilter::IFilter()
		: VoiceId(0)
		, SampleRate(44100.0f)
		, NumChannels(1)
		, Frequency(MaxFilterFreq)
		, BaseFrequency(MaxFilterFreq)
		, ModFrequency(0.0f)
		, ExternalModFrequency(0.0f)
		, Q(1.5f)
		, ModQ(0.0f)
		, BaseQ(1.5f)
		, ExternalModQ(0.0f)
		, FilterType(EFilter::LowPass)
		, ModMatrix(nullptr)
		, bChanged(false)
	{
	}

	IFilter::IFilter(const IFilter&) = default;

	IFilter::~IFilter() = default;

	void IFilter::Init(const float InSampleRate, const int32 InNumChannels, const int32 InVoiceId, FModulationMatrix* InModMatrix)
	{
		VoiceId = InVoiceId;
		SampleRate = InSampleRate;
		NumChannels = FMath::Min(MaxFilterChannels, InNumChannels);

		ModMatrix = InModMatrix;
		if (ModMatrix)
		{
			ModCutoffFrequencyDest = ModMatrix->CreatePatchDestination(VoiceId, 1, 100.0f);
			ModQDest = ModMatrix->CreatePatchDestination(VoiceId, 1, 10.0f);

#if MOD_MATRIX_DEBUG_NAMES
			ModCutoffFrequencyDest.Name = TEXT("ModCutoffFrequencyDest");
			ModQDest.Name = TEXT("ModQDest");
#endif
		}
	}

	void IFilter::Reset()
	{
		ModQ = 0.0f;
		ExternalModQ = 0.0f;
		ExternalModFrequency = 0.0f;
		ModFrequency = 0.0f;
		bChanged = true;
	}

	void IFilter::SetFrequency(const float InCutoffFrequency)
	{
		if (BaseFrequency != InCutoffFrequency)
		{
			BaseFrequency = FMath::Clamp(InCutoffFrequency, 20.f, 0.9f * (SampleRate / 2.f));
			bChanged = true;
		}
	}

	void IFilter::SetFrequencyMod(const float InModFrequency)
	{
		if (ExternalModFrequency != InModFrequency)
		{	
			ExternalModFrequency = InModFrequency;		
			bChanged = true;
		}
	}

	void IFilter::SetQ(const float InQ)
	{
		if (BaseQ != InQ)
		{
			BaseQ = InQ;
			bChanged = true;
		}
	}

	void IFilter::SetQMod(const float InModQ)
	{
		if (ExternalModQ != InModQ)
		{
			ExternalModQ = InModQ;
			bChanged = true;
		}
	}

	void IFilter::SetFilterType(const EFilter::Type InFilterType)
	{
		FilterType = InFilterType;
	}

	void IFilter::Update()
	{
		if (ModMatrix)
		{
			bChanged |= ModMatrix->GetDestinationValue(VoiceId, ModCutoffFrequencyDest, ModFrequency);
			bChanged |= ModMatrix->GetDestinationValue(VoiceId, ModQDest, ModQ);
		}

		if (bChanged)
		{
			bChanged = false;

			Frequency = FMath::Clamp(BaseFrequency * GetFrequencyMultiplier(ModFrequency + ExternalModFrequency), 80.0f, 0.9f * (SampleRate / 2.f));
			Q = BaseQ + ModQ + ExternalModQ;
		}
	}

	// One pole filter

	FOnePoleFilter::FOnePoleFilter()
		: A0(0.0f)
		, Z1(nullptr)
	{

	}

	FOnePoleFilter::~FOnePoleFilter()
	{
		if (Z1)
		{
			delete[] Z1;
		}
	}

	void FOnePoleFilter::Init(const float InSampleRate, const int32 InNumChannels, const int32 InVoiceId, FModulationMatrix* InModMatrix)
	{
		IFilter::Init(InSampleRate, InNumChannels, InVoiceId, InModMatrix);

		if (Z1)
		{
			delete[] Z1;
		}
		Z1 = new float[NumChannels];
		Reset();
	}

	void FOnePoleFilter::Reset()
	{
		IFilter::Reset();

		FMemory::Memzero(Z1, sizeof(float)*NumChannels);
	}

	void FOnePoleFilter::Update()
	{
		// Call base class to get an updated frequency
		IFilter::Update();

		const float G = GetGCoefficient();
		A0 = G / (1.0f + G);
	}

	void FOnePoleFilter::ProcessAudioFrame(const float* InFrame, float* OutFrame)
	{
		if (FilterType == EFilter::HighPass)
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const float Vn = (InFrame[Channel] - Z1[Channel]) * A0;
				const float LPF = Vn + Z1[Channel];
				Z1[Channel] = Vn + LPF;

				OutFrame[Channel] = InFrame[Channel] - LPF;
			}
		}
		else
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const float Vn = (InFrame[Channel] - Z1[Channel]) * A0;
				const float LPF = Vn + Z1[Channel];
				Z1[Channel] = Vn + LPF;

				OutFrame[Channel] = LPF;
			}
		}
	}

	void FOnePoleFilter::ProcessAudio(const float* InSamples, const int32 InNumSamples, float* OutSamples)
	{
		if (FilterType == EFilter::HighPass)
		{
			for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
			{
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					const float InputSample = InSamples[SampleIndex + Channel];
					const float Vn = (InputSample - Z1[Channel]) * A0;
					const float LPF = Vn + Z1[Channel];
					Z1[Channel] = Vn + LPF;

					OutSamples[SampleIndex + Channel] = InputSample - LPF;
				}
			}
		}
		else
		{
			for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
			{
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					const float Vn = (InSamples[SampleIndex + Channel] - Z1[Channel]) * A0;
					const float LPF = Vn + Z1[Channel];
					Z1[Channel] = Vn + LPF;

					OutSamples[SampleIndex + Channel] = LPF;
				}
			}
		}
	}

	void FOnePoleFilter::ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers)
	{
		if (FilterType == EFilter::HighPass)
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const float* Source = InBuffers[Channel];
				float* Destination = OutBuffers[Channel];
				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
				{
					const float InputSample = Source[SampleIndex];
					const float Vn = (InputSample - Z1[Channel]) * A0;
					const float LPF = Vn + Z1[Channel];
					Z1[Channel] = Vn + LPF;

					Destination[SampleIndex] = InputSample - LPF;
				}
			}
		}
		else
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const float* Source = InBuffers[Channel];
				float* Destination = OutBuffers[Channel];
				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
				{
					const float Vn = (Source[SampleIndex] - Z1[Channel]) * A0;
					const float LPF = Vn + Z1[Channel];
					Z1[Channel] = Vn + LPF;

					Destination[SampleIndex] = LPF;
				}
			}
		}
	}

	void FOnePoleFilter::ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		// Get the coefficients for the filter transfer function:
		const float A1 = A0;
		const float B1 = 2.0f * A0 - 1.0f;

		const int32 NumFloats = InOutComplexValues.Num();
		check(NumFloats % 2 == 0);
		if (FilterType == EFilter::HighPass)
		{
			for (int32 Index = 0; Index < NumFloats; Index += 2)
			{
				const float ZReal = InOutComplexValues[Index + 0];
				const float ZImag = InOutComplexValues[Index + 1];
				const std::complex<float> Z(ZReal, ZImag);

				const std::complex<float> LPF = (A0 * Z + A1) / (B1 + Z);
				const std::complex<float> HPF = 1.0f - LPF;

				InOutComplexValues[Index + 0] = HPF.real();
				InOutComplexValues[Index + 1] = HPF.imag();
			}
		}
		else
		{
			for (int32 Index = 0; Index < NumFloats; Index += 2)
			{
				const float ZReal = InOutComplexValues[Index + 0];
				const float ZImag = InOutComplexValues[Index + 1];
				const std::complex<float> Z(ZReal, ZImag);

				const std::complex<float> LPF = (A0 * Z + A1) / (B1 + Z);

				InOutComplexValues[Index + 0] = LPF.real();
				InOutComplexValues[Index + 1] = LPF.imag();
			}
		}
	}

	FStateVariableFilter::FStateVariableFilter()
		: InputScale(1.0f)
		, A0(1.0f)
		, Feedback(1.0f)
		, BandStopParam(0.5f)
	{
	}

	FStateVariableFilter::~FStateVariableFilter()
	{
	}

	void FStateVariableFilter::Init(const float InSampleRate, const int32 InNumChannels, const int32 InVoiceId, FModulationMatrix* InModMatrix)
	{
		IFilter::Init(InSampleRate, InNumChannels, InVoiceId, InModMatrix);

		FilterState.Reset();
		FilterState.AddDefaulted(NumChannels);
		Reset();
	}

	void FStateVariableFilter::SetBandStopControl(const float InBandStop)
	{
		BandStopParam = FMath::Clamp(InBandStop, 0.0f, 1.0f);
	}

	void FStateVariableFilter::Reset()
	{
		IFilter::Reset();

		FMemory::Memzero(FilterState.GetData(), sizeof(FFilterState) * NumChannels);
	}

	void FStateVariableFilter::Update()
	{
		IFilter::Update();

		float FinalQ = FMath::Clamp(Q, 1.0f, 10.0f);
		FinalQ = FMath::Lerp(0.5f, 25.0f, (FinalQ - 1.0f) / 9.0f);

		const float G = GetGCoefficient();
		const float Dampening = 0.5f / FinalQ;

		InputScale = 1.0f / (1.0f + 2.0f*Dampening*G + G*G);
		A0 = G;
		Feedback = 2.0f * Dampening + G;
	}

	void FStateVariableFilter::ProcessAudio(const float* InSamples, const int32 InNumSamples, float* OutSamples)
	{
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const float HPF = InputScale * (InSamples[SampleIndex + Channel] - Feedback * FilterState[Channel].Z1_1 - FilterState[Channel].Z1_2);
				float BPF = Audio::FastTanh(A0 * HPF + FilterState[Channel].Z1_1);

				const float LPF = A0 * BPF + FilterState[Channel].Z1_2;
				const float Dampening = 0.5f / Q;
				const float BSF = BandStopParam * HPF + (1.0f - BandStopParam) * LPF;

				FilterState[Channel].Z1_1 = A0 * HPF + BPF;
				FilterState[Channel].Z1_2 = A0 * BPF + LPF;

				switch (FilterType)
				{
				default:
				case EFilter::LowPass:
					OutSamples[SampleIndex + Channel] = LPF;
					break;

				case EFilter::HighPass:
					OutSamples[SampleIndex + Channel] = HPF;
					break;

				case EFilter::BandPass:
					OutSamples[SampleIndex + Channel] = BPF;
					break;

				case EFilter::BandStop:
					OutSamples[SampleIndex + Channel] = BSF;
					break;
				}
			}
		}
	}

	void FStateVariableFilter::ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers)
	{
		float FilterResults[4];
		enum ResultSourceType { kLPF = 0, kHPF = 1, kBPF = 2, kBSF = 3 };
		int32 OutputSource = 0;
		switch (FilterType)
		{
		default:
		case EFilter::LowPass:  OutputSource = ResultSourceType::kLPF; break;
		case EFilter::HighPass: OutputSource = ResultSourceType::kHPF; break;
		case EFilter::BandPass: OutputSource = ResultSourceType::kBPF; break;
		case EFilter::BandStop: OutputSource = ResultSourceType::kBSF; break;
		}

		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			const float* Source = InBuffers[Channel];
			float* Destination = OutBuffers[Channel];
			FFilterState& ChannelFilterState = FilterState[Channel];
			for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
			{
				FilterResults[ResultSourceType::kHPF] = InputScale * (Source[SampleIndex] - Feedback * ChannelFilterState.Z1_1 - ChannelFilterState.Z1_2);
				FilterResults[ResultSourceType::kBPF] = Audio::FastTanh(A0 * FilterResults[ResultSourceType::kHPF] + ChannelFilterState.Z1_1);

				FilterResults[kLPF] = A0 * FilterResults[kBPF] + ChannelFilterState.Z1_2;
				const float Dampening = 0.5f / Q;
				FilterResults[kBSF] = BandStopParam * FilterResults[kHPF] + (1.0f - BandStopParam) * FilterResults[kLPF];

				ChannelFilterState.Z1_1 = A0 * FilterResults[kHPF] + FilterResults[kBPF];
				ChannelFilterState.Z1_2 = A0 * FilterResults[kBPF] + FilterResults[kLPF];

				Destination[SampleIndex] = FilterResults[OutputSource];
			}
		}

	}

	void FStateVariableFilter::ProcessAudio(const float* InSamples, const int32 InNumSamples
		, float* LpfOutput, float* HpfOutput, float* BpfOutput, float* BsfOutput)
	{
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const float HPF = InputScale * (InSamples[SampleIndex + Channel] - Feedback * FilterState[Channel].Z1_1 - FilterState[Channel].Z1_2);
				float BPF = Audio::FastTanh(A0 * HPF + FilterState[Channel].Z1_1);

				const float LPF = A0 * BPF + FilterState[Channel].Z1_2;
				const float Dampening = 0.5f / Q;
				const float BSF = BandStopParam * HPF + (1.0f - BandStopParam) * LPF;

				FilterState[Channel].Z1_1 = A0 * HPF + BPF;
				FilterState[Channel].Z1_2 = A0 * BPF + LPF;

				LpfOutput[SampleIndex + Channel] = LPF;
				HpfOutput[SampleIndex + Channel] = HPF;
				BpfOutput[SampleIndex + Channel] = BPF;
				BsfOutput[SampleIndex + Channel] = BSF;
			}
		}
	}

	void FStateVariableFilter::ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, 
		float* const* LpfOutBuffers, float* const* HpfOutBuffers, float* const* BpfOutBuffers, float* const* BsfOutBuffers)
	{
		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			const float* Source = InBuffers[Channel];
			float* LpfDestination = LpfOutBuffers[Channel];
			float* HpfDestination = HpfOutBuffers[Channel];
			float* BpfDestination = BpfOutBuffers[Channel];
			float* BsfDestination = BsfOutBuffers[Channel];
			FFilterState& ChannelFilterState = FilterState[Channel];
			for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
			{
				const float HPF = InputScale * (Source[SampleIndex] - Feedback * ChannelFilterState.Z1_1 - ChannelFilterState.Z1_2);
				float BPF = Audio::FastTanh(A0 * HPF + ChannelFilterState.Z1_1);

				const float LPF = A0 * BPF + ChannelFilterState.Z1_2;
				const float Dampening = 0.5f / Q;
				const float BSF = BandStopParam * HPF + (1.0f - BandStopParam) * LPF;

				ChannelFilterState.Z1_1 = A0 * HPF + BPF;
				ChannelFilterState.Z1_2 = A0 * BPF + LPF;

				LpfDestination[SampleIndex] = LPF;
				HpfDestination[SampleIndex] = HPF;
				BpfDestination[SampleIndex] = BPF;
				BsfDestination[SampleIndex] = BSF;
			}
		}
	}

	void FStateVariableFilter::ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		// Get the params:
		const float G = A0;
		const float R = 0.5f * (Feedback - G); // Dampening

		// Evaluate common subexpressions:
		const float GSqr = G * G;
		const float TwoRG = 2.0f * R * G;
		const float TwoGSqr = 2.0f * GSqr;

		const int32 NumFloats = InOutComplexValues.Num();
		check(NumFloats % 2 == 0);
		for (int32 Index = 0; Index < NumFloats; Index += 2)
		{
			const float ZReal = InOutComplexValues[Index + 0];
			const float ZImag = InOutComplexValues[Index + 1];
			const std::complex<float> Z(ZReal, ZImag);

			// Evaluate the polynomials that form the rational fractions:
			const std::complex<float> NumeratorLPF = (GSqr * Z + TwoGSqr) * Z + GSqr;
			const std::complex<float> NumeratorHPF = (Z - 2.0f) * Z + 1.0f;
			const std::complex<float> NumeratorBPF = G * Z * Z - G;
			const std::complex<float> Denominator = ((GSqr + TwoRG + 1.0f) * Z + (TwoGSqr - 2.0f)) * Z + (GSqr - TwoRG + 1.0f);

			// Evaluate the rational functions:
			const std::complex<float> Reciprocal = 1.0f / Denominator;
			const std::complex<float> LPF = NumeratorLPF * Reciprocal;
			const std::complex<float> HPF = NumeratorHPF * Reciprocal;
			const std::complex<float> BPF = NumeratorBPF * Reciprocal;
			const std::complex<float> BSF = BandStopParam * HPF + (1.0f - BandStopParam) * LPF;

			switch (FilterType)
			{
			default:
			case EFilter::LowPass:
				InOutComplexValues[Index + 0] = LPF.real();
				InOutComplexValues[Index + 1] = LPF.imag();
				break;
			case EFilter::HighPass:
				InOutComplexValues[Index + 0] = HPF.real();
				InOutComplexValues[Index + 1] = HPF.imag();
				break;
			case EFilter::BandPass:
				InOutComplexValues[Index + 0] = BPF.real();
				InOutComplexValues[Index + 1] = BPF.imag();
				break;
			case EFilter::BandStop:
				InOutComplexValues[Index + 0] = BSF.real();
				InOutComplexValues[Index + 1] = BSF.imag();
				break;
			}
		}
	}

	FLadderFilter::FLadderFilter()
		: K(0.0f)
		, Gamma(0.0f)
		, Alpha(0.0f)
		, PassBandGainCompensation(0.0f)
	{
		FMemory::Memzero(Factors, sizeof(float) * UE_ARRAY_COUNT(Beta));
		FMemory::Memzero(Beta, sizeof(float) * UE_ARRAY_COUNT(Beta));
	}

	FLadderFilter::~FLadderFilter()
	{

	}

	void FLadderFilter::Init(const float InSampleRate, const int32 InNumChannels, const int32 InVoiceId, FModulationMatrix* InModMatrix)
	{
		IFilter::Init(InSampleRate, InNumChannels, InVoiceId, InModMatrix);

		for (int32 i = 0; i < 4; ++i)
		{
			OnePoleFilters[i].Init(InSampleRate, InNumChannels);
			OnePoleFilters[i].SetFilterType(EFilter::LowPass);
		}
	}

	void FLadderFilter::Reset()
	{
		IFilter::Reset();

		for (int32 i = 0; i < 4; ++i)
		{
			OnePoleFilters[i].Reset();
			OnePoleFilters[i].Reset();
		}
	}

	void FLadderFilter::Update()
	{
		IFilter::Update();

		// Compute feedforward coefficient to use on all LPFs
		const float G = GetGCoefficient();
		const float FeedforwardCoeff = G / (1.0f + G);

		Gamma = FeedforwardCoeff * FeedforwardCoeff * FeedforwardCoeff * FeedforwardCoeff;
		Alpha = 1.0f / (1.0f + K * Gamma);

		Beta[0] = FeedforwardCoeff * FeedforwardCoeff * FeedforwardCoeff;
		Beta[1] = FeedforwardCoeff * FeedforwardCoeff;
		Beta[2] = FeedforwardCoeff;
		Beta[3] = 1.0f;

		const float Div = 1.0f + FeedforwardCoeff;
		for (int32 i = 0; i < 4; ++i)
		{
			OnePoleFilters[i].SetCoefficient(FeedforwardCoeff);
			Beta[i] /= Div;
		}

		// Setup LPF factors based on filter type
		switch (FilterType)
		{
			default:
			case EFilter::LowPass:
				Factors[0] = 0.0f;
				Factors[1] = 0.0f;
				Factors[2] = 0.0f;
				Factors[3] = 0.0f;
				Factors[4] = 1.0f;
				break;

			case EFilter::BandPass:
				Factors[0] = 0.0f;
				Factors[1] = 0.0f;
				Factors[2] = 4.0f;
				Factors[3] = -8.0f;
				Factors[4] = 4.0f;
				break;

			case EFilter::HighPass:
				Factors[0] = 1.0f;
				Factors[1] = -4.0f;
				Factors[2] = 6.0f;
				Factors[3] = -4.0f;
				Factors[4] = 1.0f;
				break;
		}
	}

	void FLadderFilter::SetQ(const float InQ)
	{
		Q = FMath::Clamp(InQ, 1.0f, 10.0f);
		K = 3.88f * (Q - 1.0f) / 9.0f + 0.1f;
	}

	void FLadderFilter::SetPassBandGainCompensation(const float InPassBandGainCompensation)
	{
		PassBandGainCompensation = InPassBandGainCompensation;
	}

	void FLadderFilter::ProcessAudio(const float* InSamples, const int32 InNumSamples, float* OutSamples)
	{
		float U[MaxFilterChannels];

		float OutputFilter0[MaxFilterChannels];
		float OutputFilter1[MaxFilterChannels];
		float OutputFilter2[MaxFilterChannels];
		float OutputFilter3[MaxFilterChannels];

		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				float Sigma = 0.0f;
				for (int32 i = 0; i < 4; ++i)
				{
					Sigma += OnePoleFilters[i].GetState(Channel) * Beta[i];
				}

				float InSample = InSamples[SampleIndex + Channel];
				InSample *= 1.0f + PassBandGainCompensation * K;

				// Compute input into first LPF
				U[Channel] = FMath::Min(Audio::FastTanh((InSample - K * Sigma)* Alpha), 1.0f);
			}

			OnePoleFilters[0].ProcessAudioFrame(U, OutputFilter0);
			OnePoleFilters[1].ProcessAudioFrame(OutputFilter0, OutputFilter1);
			OnePoleFilters[2].ProcessAudioFrame(OutputFilter1, OutputFilter2);
			OnePoleFilters[3].ProcessAudioFrame(OutputFilter2, OutputFilter3);

			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				const int32 OutputSampleIndex = SampleIndex + Channel;

				// Feed U into first filter, then cascade down
				OutSamples[OutputSampleIndex] = Factors[0] * U[Channel];
				OutSamples[OutputSampleIndex] += Factors[1] * OutputFilter0[Channel];
				OutSamples[OutputSampleIndex] += Factors[2] * OutputFilter1[Channel];
				OutSamples[OutputSampleIndex] += Factors[3] * OutputFilter2[Channel];
				OutSamples[OutputSampleIndex] += Factors[4] * OutputFilter3[Channel];
			}
		}
	}

	void FLadderFilter::ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers)
	{
		float U[MaxFilterChannels];

		float OutputFilter0[MaxFilterChannels];
		float OutputFilter1[MaxFilterChannels];
		float OutputFilter2[MaxFilterChannels];
		float OutputFilter3[MaxFilterChannels];

		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
		{
			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				float Sigma = 0.0f;
				for (int32 i = 0; i < 4; ++i)
				{
					Sigma += OnePoleFilters[i].GetState(Channel) * Beta[i];
				}

				float InSample = InBuffers[Channel][SampleIndex];
				InSample *= 1.0f + PassBandGainCompensation * K;

				// Compute input into first LPF
				U[Channel] = FMath::Min(Audio::FastTanh((InSample - K * Sigma) * Alpha), 1.0f);
			}

			OnePoleFilters[0].ProcessAudioFrame(U, OutputFilter0);
			OnePoleFilters[1].ProcessAudioFrame(OutputFilter0, OutputFilter1);
			OnePoleFilters[2].ProcessAudioFrame(OutputFilter1, OutputFilter2);
			OnePoleFilters[3].ProcessAudioFrame(OutputFilter2, OutputFilter3);

			for (int32 Channel = 0; Channel < NumChannels; ++Channel)
			{
				// Feed U into first filter, then cascade down
				OutBuffers[Channel][SampleIndex] = Factors[0] * U[Channel];
				OutBuffers[Channel][SampleIndex] += Factors[1] * OutputFilter0[Channel];
				OutBuffers[Channel][SampleIndex] += Factors[2] * OutputFilter1[Channel];
				OutBuffers[Channel][SampleIndex] += Factors[3] * OutputFilter2[Channel];
				OutBuffers[Channel][SampleIndex] += Factors[4] * OutputFilter3[Channel];
			}
		}
	}

	void FLadderFilter::ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		check(FilterType == EFilter::LowPass);

		// Get the one-pole LPF coefficients:
		const float A0 = OnePoleFilters[0].GetCoefficient();
		const float A1 = A0;
		const float B1 = 2.0f * A0 - 1.0f;

		const int32 NumFloats = InOutComplexValues.Num();
		check(NumFloats % 2 == 0);
		for (int32 Index = 0; Index < NumFloats; Index += 2)
		{
			const float ZReal = InOutComplexValues[Index + 0];
			const float ZImag = InOutComplexValues[Index + 1];
			const std::complex<float> Z(ZReal, ZImag);

			// Calculate the response for the 4 one-pole filters in series:
			const std::complex<float> OnePoleResponse = (A0 * Z + A1) / (B1 + Z);
			const std::complex<float> Series = OnePoleResponse * OnePoleResponse * OnePoleResponse * OnePoleResponse;

			// Apply pass-band gain-compensation and feedback:
			const std::complex<float> H = (1.0f + PassBandGainCompensation * K) * Series / (K * Series + 1.0f);

			InOutComplexValues[Index + 0] = H.real();
			InOutComplexValues[Index + 1] = H.imag();
		}
	}
}
