// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Flanger.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	const float FFlanger::MaxDelaySec = 5.0f;
	const float FFlanger::MaxModulationRate = 20.0f;
	const float FFlanger::MaxCenterDelay = 20.0f;

	FFlanger::FFlanger()
	{
	}

	FFlanger::~FFlanger()
	{
	}

	void FFlanger::Init(const float InSampleRate)
	{
		SampleRate = InSampleRate;
		DelayBuffer.Init(InSampleRate, MaxDelaySec);
		DelayedSignalLevel = FMath::Clamp(MixLevel, 0.0f, 1.0f);
		NonDelayedSignalLevel = 1.0f - DelayedSignalLevel;
	}

	void FFlanger::SetModulationRate(const float InModulationRate)
	{
		float ClampedInModulationRate = FMath::Clamp(InModulationRate, 0.0f, MaxModulationRate);
		if (!FMath::IsNearlyEqual(ClampedInModulationRate, ModulationRate))
		{
			ModulationRate = ClampedInModulationRate;
		}
	}

	// since this will be clamped based on CenterDelayMsec to avoid clipping, 
	// make sure SetCenterDelay is called before this function if necessary
	void FFlanger::SetModulationDepth(const float InModulationDepth)
	{
		float ClampedInModulationDepth = FMath::Clamp(InModulationDepth, 0.0f, CenterDelayMsec);
		if (!FMath::IsNearlyEqual(ClampedInModulationDepth, ModulationDepth))
		{
			ModulationDepth = ClampedInModulationDepth;
		}
	}

	void FFlanger::SetCenterDelay(const float InCenterDelay)
	{
		float ClampedInCenterDelay = FMath::Clamp(InCenterDelay, 0.0f, MaxCenterDelay);
		if (!FMath::IsNearlyEqual(ClampedInCenterDelay, CenterDelayMsec))
		{
			CenterDelayMsec = ClampedInCenterDelay;
		}
	}

	void FFlanger::SetMixLevel(const float InMixLevel)
	{
		float ClampedInMixLevel = FMath::Clamp(InMixLevel, 0.0f, 1.0f);
		if (!FMath::IsNearlyEqual(ClampedInMixLevel, MixLevel))
		{
			MixLevel = ClampedInMixLevel;
			DelayedSignalLevel = MixLevel;
			NonDelayedSignalLevel = 1.0f - DelayedSignalLevel;
		}
	}

	void FFlanger::ProcessAudio(const FAlignedFloatBuffer& InBuffer, const int32 InNumSamples, FAlignedFloatBuffer& OutBuffer)
	{
		// LFO sample for delay is generated at buffer, not sample, rate 
		LFO.GenerateBuffer(SampleRate / InNumSamples, ModulationRate, &DelaySample, 1);
		DelayBuffer.SetEasedDelayMsec(ModulationDepth * DelaySample + CenterDelayMsec, true);

		// initialize scratch buffer if necessary and get delay samples (non SIMD)
		if (ScratchBuffer.Num() != InNumSamples)
		{
			ScratchBuffer.Reset();
			ScratchBuffer.AddUninitialized(InBuffer.Num());
		}
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
		{
			ScratchBuffer[SampleIndex] = DelayBuffer.ProcessAudioSample(InBuffer[SampleIndex]);
		}

		ArrayWeightedSum(ScratchBuffer, DelayedSignalLevel, InBuffer, NonDelayedSignalLevel, OutBuffer);
	}
}
