// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/FloatArrayMath.h"
#include "CoreMinimal.h"
#include "SignalProcessingModule.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "DSP/Dsp.h"
#include "DSP/InterpolatedOnePole.h"

#if INTEL_ISPC && !UE_BUILD_SHIPPING
#include "HAL/IConsoleManager.h"
#endif

#if INTEL_ISPC
#include "FloatArrayMath.ispc.generated.h"
#endif

#if !defined(AUDIO_FLOAT_ARRAY_MATH_ISPC_ENABLED_DEFAULT)
#define AUDIO_FLOAT_ARRAY_MATH_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bAudio_FloatArrayMath_ISPC_Enabled = INTEL_ISPC && AUDIO_FLOAT_ARRAY_MATH_ISPC_ENABLED_DEFAULT;
#else
static bool bAudio_FloatArrayMath_ISPC_Enabled = AUDIO_FLOAT_ARRAY_MATH_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarAudioFloatArrayMathISPCEnabled(TEXT("au.FloatArrayMath.ISPC"), bAudio_FloatArrayMath_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in audio float array math operations"));
#endif

CSV_DEFINE_CATEGORY(Audio_Dsp, false);

namespace Audio
{
	namespace MathIntrinsics
	{
		const float Loge10 = FMath::Loge(10.f);
		const int32 SimdMask = 0xFFFFFFFC;
		const int32 NotSimdMask = 0x00000003;

		const int32 Simd8Mask = 0xFFFFFFF8;
		const int32 NotSimd8Mask = 0x00000007;

		const int32 Simd16Mask = 0xFFFFFFF0;
		const int32 NotSimd16Mask = 0x0000000F;
	}

	void ArraySum(TArrayView<const float> InValues, float& OutSum)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySum);
		OutSum = 0.f;

		int32 Num = InValues.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySum(InValues.GetData(), OutSum, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				VectorRegister4Float Total = VectorSetFloat1(0.f);

				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InValues[i]);
					Total = VectorAdd(Total, VectorData);
				}

				float Val[4];
				VectorStore(Total, Val);
				OutSum += Val[0] + Val[1] + Val[2] + Val[3];
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					OutSum += InValues[i];
				}
			}

		}
	}

	void ArraySum(TArrayView<const float> InFloatBuffer1, TArrayView<const float> InFloatBuffer2, TArrayView<float> OutputBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySum);
		checkf(InFloatBuffer1.Num() == InFloatBuffer2.Num(), TEXT("Input buffers must be equal length"));

		const int32 Num = InFloatBuffer1.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySum2(InFloatBuffer1.GetData(), InFloatBuffer2.GetData(), OutputBuffer.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input1 = VectorLoad(&InFloatBuffer1[i]);
					VectorRegister4Float Input2 = VectorLoad(&InFloatBuffer2[i]);

					VectorRegister4Float Output = VectorAdd(Input1, Input2);
					VectorStore(Output, &OutputBuffer[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					OutputBuffer[i] = InFloatBuffer1[i] + InFloatBuffer2[i];
				}
			}
		}
	}

	void ArrayCumulativeSum(TArrayView<const float> InView, TArray<float>& OutData)
	{
		// Initialize output data
		int32 Num = InView.Num();
		OutData.Reset();
		OutData.AddUninitialized(Num);

		if (Num < 1)
		{
			return;
		}
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayCumulativeSum);

		float* OutDataPtr = OutData.GetData();
		const float* InViewPtr = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayCumulativeSum(InViewPtr, OutDataPtr, Num);
#endif
		}
		else
		{
			// Start summing
			*OutDataPtr = *InViewPtr++;
	
			for (int32 i = 1; i < Num; i++)
			{
				float Temp = *OutDataPtr++ + *InViewPtr++;
				*OutDataPtr = Temp;
			}
		}
	}

	void ArrayMean(TArrayView<const float> InView, float& OutMean)
	{
		OutMean = 0.f;

		const int32 Num = InView.Num();

		if (Num < 1)
		{
			return;
		}
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMean);


		const float* DataPtr = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMean(DataPtr, OutMean, Num);
#endif
		}
		else
		{
			for (int32 i = 0; i < Num; i++)
			{
				OutMean += DataPtr[i];
			}
	
			OutMean /= static_cast<float>(Num);
		}
	}

	void ArrayMeanSquared(TArrayView<const float> InView, float& OutMean)
	{
		OutMean = 0.0f;

		const int32 Num = InView.Num();

		if (Num < 1)
		{
			return;
		}
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMeanSquared);

		const float* DataPtr = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMeanSquared(DataPtr, OutMean, Num);
#endif
		}
		else
		{
			for (int32 i = 0; i < Num; i++)
			{
				OutMean += DataPtr[i] * DataPtr[i];
			}
	
			OutMean /= static_cast<float>(Num);
		}
	}

	float ArrayGetMagnitude(TArrayView<const float> Buffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayGetMagnitude);
		const int32 Num = Buffer.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			return ispc::ArrayGetMagnitude(Buffer.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			float Sum = 0.0f;

			if (NumToSimd)
			{
				VectorRegister4Float VectorSum = VectorZero();

				const float Exponent = 2.0f;
				VectorRegister4Float ExponentVector = VectorLoadFloat1(&Exponent);

				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input = VectorPow(VectorLoad(&Buffer[i]), ExponentVector);
					VectorSum = VectorAdd(VectorSum, Input);
				}

				float PartionedSums[4];
				VectorStore(VectorSum, PartionedSums);

				Sum += PartionedSums[0] + PartionedSums[1] + PartionedSums[2] + PartionedSums[3];
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					Sum += Buffer[i] * Buffer[i];
				}
			}

			return FMath::Sqrt(Sum);
		}
	}

	float ArrayGetAverageValue(TArrayView<const float> Buffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayGetAverageValue);
		const int32 Num = Buffer.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			return ispc::ArrayGetAverageValue(Buffer.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			float Sum = 0.0f;

			if (NumToSimd)
			{
				VectorRegister4Float VectorSum = VectorZero();

				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input = VectorLoad(&Buffer[i]);
					VectorSum = VectorAdd(VectorSum, Input);
				}

				float PartionedSums[4];
				VectorStore(VectorSum, PartionedSums);

				Sum += PartionedSums[0] + PartionedSums[1] + PartionedSums[2] + PartionedSums[3];
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					Sum += Buffer[i];
				}
			}

			return Sum / Num;
		}
	}

	float ArrayGetAverageAbsValue(TArrayView<const float> Buffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayGetAverageAbsValue);
		const int32 Num = Buffer.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			return ispc::ArrayGetAverageAbsValue(Buffer.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			float Sum = 0.0f;

			if (NumToSimd)
			{
				VectorRegister4Float VectorSum = VectorZero();

				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input = VectorAbs(VectorLoad(&Buffer[i]));
					VectorSum = VectorAdd(VectorSum, Input);
				}

				float PartionedSums[4];
				VectorStore(VectorSum, PartionedSums);

				Sum += PartionedSums[0] + PartionedSums[1] + PartionedSums[2] + PartionedSums[3];
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					Sum += FMath::Abs(Buffer[i]);
				}
			}

			return Sum / Num;
		}
	}

	void ArrayMeanFilter(TArrayView<const float> InView, int32 WindowSize, int32 WindowOrigin, TArray<float>& OutData)
	{
		// a quick but sinful implementation of a mean filter. encourages floating point rounding errors. 
		check(WindowOrigin < WindowSize);
		check(WindowOrigin >= 0);
		check(WindowSize > 0);

		// Initialize output data
		const int32 Num = InView.Num();
		OutData.Reset();
		OutData.AddUninitialized(Num);

		if (Num < 1)
		{
			return;
		}
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMeanFilter);

		// Use cumulative sum to avoid multiple summations 
		// Instead of summing over InView[StartIndex:EndIndex], avoid all that
		// calculation by taking difference of cumulative sum at those two points:
		//  cumsum(X[0:b]) - cumsum(X[0:a]) = sum(X[a:b])
		TArray<float> SummedData;
		ArrayCumulativeSum(InView, SummedData);
		const float LastSummedData = SummedData.Last();

		float* OutDataPtr = OutData.GetData();
		const float* SummedDataPtr = SummedData.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMeanFilter(SummedDataPtr, WindowSize, WindowOrigin, OutDataPtr, LastSummedData, Num);
#endif
		}
		else
		{
			const int32 LastIndexBeforeEndBoundaryCondition = FMath::Max(WindowOrigin + 1, Num - WindowSize + WindowOrigin + 1);
			const int32 StartOffset = -WindowOrigin - 1;
			const int32 EndOffset = WindowSize - WindowOrigin - 1;
			const int32 WindowTail = WindowSize - WindowOrigin;
	
			if ((WindowSize - WindowOrigin) < Num)
			{
				// Handle boundary condition where analysis window precedes beginning of array.
				for (int32 i = 0; i < (WindowOrigin + 1); i++)
				{
					OutDataPtr[i] = SummedDataPtr[i + EndOffset] / FMath::Max(1.f, static_cast<float>(WindowTail + i));
				}
	
				// No boundary conditions to handle here.	
				const float MeanDivisor = static_cast<float>(WindowSize);
				for (int32 i = WindowOrigin + 1; i < LastIndexBeforeEndBoundaryCondition; i++)
				{
					OutDataPtr[i] = (SummedDataPtr[i + EndOffset] - SummedDataPtr[i + StartOffset]) / MeanDivisor;
				}
			}
			else
			{
				// Handle boundary condition where window precedes beginning and goes past end of array
				const float ArrayMean = LastSummedData / static_cast<float>(Num);
				for (int32 i = 0; i < LastIndexBeforeEndBoundaryCondition; i++)
				{
					OutDataPtr[i] = ArrayMean;
				}
			}
	
			// Handle boundary condition where analysis window goes past end of array.
			for (int32 i = LastIndexBeforeEndBoundaryCondition; i < Num; i++)
			{
				OutDataPtr[i] = (LastSummedData - SummedDataPtr[i + StartOffset]) / static_cast<float>(Num - i + WindowOrigin);
			}
		}
	}

	void ArrayMaxFilter(TArrayView<const float> InView, int32 WindowSize, int32 WindowOrigin, TArray<float>& OutData)
	{
		// A reasonable implementation of a max filter for the data we're interested in, though surely not the fastest.
		check(WindowOrigin < WindowSize);
		check(WindowOrigin >= 0);
		check(WindowSize > 0);
		
		int32 StartIndex = -WindowOrigin;
		int32 EndIndex = StartIndex + WindowSize;

		// Initialize output
		int32 Num = InView.Num();
		OutData.Reset();
		OutData.AddUninitialized(Num);

		if (Num < 1)
		{
			return;
		}
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMaxFilter);

		// Get max in first window
		int32 ActualStartIndex = 0;
		int32 ActualEndIndex = FMath::Min(EndIndex, Num);

		const float* InViewPtr = InView.GetData();
		float* OutDataPtr = OutData.GetData();
		int32 MaxIndex = 0;
		float MaxValue = InView[0];

		for (int32 i = ActualStartIndex; i < ActualEndIndex; i++)
		{
			if (InViewPtr[i] > MaxValue)
			{
				MaxValue = InViewPtr[i];
				MaxIndex = i;
			}		
		}
		OutDataPtr[0] = MaxValue;

		StartIndex++;
		EndIndex++;

		// Get max in remaining windows
		for (int32 i = 1; i < Num; i++)
		{
			ActualStartIndex = FMath::Max(StartIndex, 0);
			ActualEndIndex = FMath::Min(EndIndex, Num);

			if (MaxIndex < StartIndex)
			{
				// We need to evaluate the entire window because the previous maximum value was not in this window.
				MaxIndex = ActualStartIndex;
				MaxValue = InViewPtr[MaxIndex];
				for (int32 j = ActualStartIndex + 1; j < ActualEndIndex; j++)
				{
					if (InViewPtr[j] > MaxValue)
					{
						MaxIndex = j;
						MaxValue = InViewPtr[MaxIndex];
					}
				}
			}
			else
			{
				// We only need to inspect the newest sample because the previous maximum value was in this window.
				if (InViewPtr[ActualEndIndex - 1] > MaxValue)
				{
					MaxIndex = ActualEndIndex - 1;
					MaxValue = InViewPtr[MaxIndex];
				}
			}

			OutDataPtr[i] = MaxValue;

			StartIndex++;
			EndIndex++;
		}
	}

	void ArrayGetEuclideanNorm(TArrayView<const float> InView, float& OutEuclideanNorm)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayGetEuclideanNorm);

		// Initialize output.
		OutEuclideanNorm = 0.0f;
		const int32 Num = InView.Num();
		const float* InViewData = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayGetEuclideanNorm(InViewData, OutEuclideanNorm, Num);
#endif
		}
		else
		{
			// Sum it up.
			for (int32 i = 0; i < Num; i++)
			{
				OutEuclideanNorm += InViewData[i] * InViewData[i];
			}
	
			OutEuclideanNorm = FMath::Sqrt(OutEuclideanNorm);
		}
	}

	void ArrayAbs(TArrayView<const float> InBuffer, TArrayView<float> OutBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayAbs);
		const int32 Num = InBuffer.Num();
		check(OutBuffer.Num() == Num);

		const float* InData = InBuffer.GetData();
		float* OutData = OutBuffer.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayAbs(InData, OutData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input = VectorLoad(&InData[i]);
					VectorStore(VectorAbs(Input), &OutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					OutData[i] = FMath::Abs(InData[i]);
				}
			}
		}
	}


	void ArrayAbsInPlace(TArrayView<float> InView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayAbsInPlace);
		const int32 Num = InView.Num();
		float* Data = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayAbsInPlace(Data, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input = VectorLoad(&Data[i]);
					VectorStore(VectorAbs(Input), &Data[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					Data[i] = FMath::Abs(Data[i]);
				}
			}
		}
	}

	void ArrayClampMinInPlace(TArrayView<float> InView, float InMin)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayClampMinInPlace);
		const int32 Num = InView.Num();
		float* Data = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayClampMinInPlace(Data, InMin, Num);
#endif
		}
		else
		{
			for (int32 i = 0; i < Num; i++)
			{
				Data[i] = FMath::Max(InMin, Data[i]);
			}
		}
	}

	void ArrayClampMaxInPlace(TArrayView<float> InView, float InMax)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayClampMaxInPlace);
		const int32 Num = InView.Num();
		float* Data = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayClampMaxInPlace(Data, InMax, Num);
#endif
		}
		else
		{
			for (int32 i = 0; i < Num; i++)
			{
				Data[i] = FMath::Min(InMax, Data[i]);
			}
		}
	}

	void ArrayClampInPlace(TArrayView<float> InView, float InMin, float InMax)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayClampInPlace);

		const int32 Num = InView.Num();
		float* Data = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayClampInPlace(Data, InMin, InMax, Num);
#endif
		}
		else
		{
			for (int32 i = 0; i < Num; i++)
			{
				Data[i] = FMath::Clamp(Data[i], InMin, InMax);
			}
		}
	}

	void ArrayMinMaxNormalize(TArrayView<const float> InView, TArray<float>& OutArray)
	{
		const int32 Num = InView.Num();
		OutArray.Reset(Num);

		if (Num < 1)
		{
			return;
		}
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMinMaxNormalize);

		OutArray.AddUninitialized(Num);

		const float* InDataPtr = InView.GetData();
		float* OutDataPtr = OutArray.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMinMaxNormalize(InDataPtr, OutDataPtr, Num);
#endif
		}
		else
		{
			float MaxValue = InDataPtr[0];
			float MinValue = InDataPtr[0];
	
			// determine min and max
			for (int32 i = 1; i < Num; i++)
			{
				if (InDataPtr[i] < MinValue)
				{
					MinValue = InDataPtr[i];
				}
				else if (InDataPtr[i] > MaxValue)
				{
					MaxValue = InDataPtr[i];
				}
			}
	
			// Normalize data by subtracting minimum value and dividing by range
			float Scale = 1.f / FMath::Max(SMALL_NUMBER, MaxValue - MinValue);
			for (int32 i = 0; i < Num; i++)
			{
				OutDataPtr[i] = (InDataPtr[i] - MinValue) * Scale;
			}
		}
	}
	
	void ArrayMax(const TArrayView<const float>& InView1, const TArrayView<const float>& InView2, const TArrayView<float>& OutView)
	{
		check(InView1.Num() == InView2.Num());
		check(InView1.Num() == OutView.Num());

		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMax);
		const int32 Num = InView1.Num();
		
		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMax(InView1.GetData(), InView2.GetData(), OutView.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input1 = VectorLoad(&InView1[i]);
					VectorRegister4Float Input2 = VectorLoad(&InView2[i]);
					VectorStore(VectorMax(Input1, Input2), &OutView[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					OutView[i] = FMath::Max(InView1[i], InView2[i]);
				}
			}
		}
	}

	float ArrayMaxAbsValue(const TArrayView<const float> InView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMaxAbsValue);

		const int32 Num = InView.Num();
		const float* Data = InView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			return ispc::ArrayMaxAbsValue(Data, Num);
#endif
		}
		else
		{
			float Max = 0.f;
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			VectorRegister4Float MaxVector = VectorSetFloat1(0.f);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input1 = VectorLoad(&Data[i]);

					MaxVector = VectorMax(MaxVector, VectorAbs(Input1));
				}

				AlignedFloat4 OutArray(MaxVector);

				Max = FMath::Max(FMath::Max(OutArray[0], OutArray[1]), FMath::Max(OutArray[2], OutArray[3]));
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					Max = FMath::Max(FMath::Abs(Data[i]), Max);
				}
			}

			return Max;
		}
	}

	void ArrayMultiply(TArrayView<const float> InFloatBufferA, TArrayView<const float> InFloatBufferB, TArrayView<float> OutBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMultiply);
		checkf((InFloatBufferA.Num() == InFloatBufferB.Num()) && (InFloatBufferA.Num() == OutBuffer.Num()), TEXT("Input buffers must be equal length"));

		const int32 Num = InFloatBufferA.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMultiply(InFloatBufferA.GetData(), InFloatBufferB.GetData(), OutBuffer.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;

			for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				VectorRegister4Float Input1 = VectorLoad(&InFloatBufferA[i]);
				VectorRegister4Float Input2 = VectorLoad(&InFloatBufferB[i]);

				VectorRegister4Float Output = VectorMultiply(Input1, Input2);
				VectorStore(Output, &OutBuffer[i]);
			}

			for (int32 i = NumToSimd; i < Num; ++i)
			{
				OutBuffer[i] = InFloatBufferA[i] * InFloatBufferB[i];
			}
		}
	}

	void ArrayMultiplyInPlace(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToMultiply)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMultiplyInPlace);
		checkf(InFloatBuffer.Num() == BufferToMultiply.Num(), TEXT("Input buffers must be equal length"));

		const int32 Num = BufferToMultiply.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMultiplyInPlace(InFloatBuffer.GetData(), BufferToMultiply.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input1 = VectorLoad(&InFloatBuffer[i]);
					VectorRegister4Float Output = VectorLoad(&BufferToMultiply[i]);

					Output = VectorMultiply(Input1, Output);
					VectorStore(Output, &BufferToMultiply[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					BufferToMultiply[i] = InFloatBuffer[i] * BufferToMultiply[i];
				}
			}
		}
	}

	void ArrayComplexMultiplyInPlace(TArrayView<const float> InValues1, TArrayView<float> InValues2)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayComplexMultiplyInPlace);
		check(InValues1.Num() == InValues2.Num());

		const int32 Num = InValues1.Num();

		// Needs to be in interleaved format.
		check((Num % 2) == 0);

		const float* InData1 = InValues1.GetData();
		float* InData2 = InValues2.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayComplexMultiplyInPlace(InData1, InData2, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				const VectorRegister4Float RealSignFlip = MakeVectorRegister(-1.f, 1.f, -1.f, 1.f);

				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData1 = VectorLoad(&InData1[i]);
					VectorRegister4Float VectorData2 = VectorLoad(&InData2[i]);

					VectorRegister4Float VectorData1Real = VectorSwizzle(VectorData1, 0, 0, 2, 2);
					VectorRegister4Float VectorData1Imag = VectorSwizzle(VectorData1, 1, 1, 3, 3);
					VectorRegister4Float VectorData2Swizzle = VectorSwizzle(VectorData2, 1, 0, 3, 2);

					VectorRegister4Float Result = VectorMultiply(VectorData1Imag, VectorData2Swizzle);
					Result = VectorMultiply(Result, RealSignFlip);
					Result = VectorMultiplyAdd(VectorData1Real, VectorData2, Result);

					VectorStore(Result, &InData2[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i += 2)
				{
					float Real = (InData1[i] * InData2[i]) - (InData1[i + 1] * InData2[i + 1]);
					float Imag = (InData1[i] * InData2[i + 1]) + (InData1[i + 1] * InData2[i]);
					InData2[i] = Real;
					InData2[i + 1] = Imag;
				}
			}
		}
	}

	void ArrayComplexMultiplyAdd(TArrayView<const float> InValues1, TArrayView<const float> InValues2, TArrayView<float> OutArray)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayComplexMultiplyAdd);

		check(InValues1.Num() == InValues2.Num());
		check(OutArray.Num() == InValues1.Num());

		const int32 Num = InValues1.Num();

		const float* InAData = InValues1.GetData();
		const float* InBData = InValues2.GetData();
		float* OutData = OutArray.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayComplexMultiplyAdd(InAData, InBData, OutData, Num);
#endif
		}
		else
		{
			const int32 NumSimd = Num & MathIntrinsics::SimdMask;

			// Complex numbers are stored as [real_0, complex_0, real_1, complex_1, ... real_N, complex_N]
			// So we final amount must be evenly divisble by 2.
			check(NumSimd % 2 == 0);

			const VectorRegister4Float SignFlip = MakeVectorRegisterFloat(-1.f, 1.f, -1.f, 1.f);

			for (int32 i = 0; i < NumSimd; i += 4)
			{
				// Complex multiply add
				// Nr = real component of Nth number
				// Ni = imaginary component of Nth number
				//
				//
				// The input is then 
				// A1r A1i A2r A2i
				// B1r B1i B2r B2i

				// VectorA = A1r A1i A2r A2i
				VectorRegister4Float VectorInA = VectorLoad(&InAData[i]);
				// Temp12 = A1i A1r A2i A2r
				VectorRegister4Float Temp1 = VectorSwizzle(VectorInA, 1, 0, 3, 2);

				// VectorB = B1r B1i B2r B2i
				VectorRegister4Float VectorInB = VectorLoad(&InBData[i]);
				// Temp2 = B1r B1r B2r B2r
				VectorRegister4Float Temp2 = VectorSwizzle(VectorInB, 0, 0, 2, 2);
				// Temp3 = B1i B1i B2i B2i
				VectorRegister4Float Temp3 = VectorSwizzle(VectorInB, 1, 1, 3, 3);


				// VectorA = A1rB1r, A1iB1r, A2rB2r, A2iB2r
				VectorInA = VectorMultiply(VectorInA, Temp2);

				// Temp1 = A1iB1i, A1rB1i, A2iB2i, A2rb2i
				Temp1 = VectorMultiply(Temp1, Temp3);

				// Temp1 = -A1iB1i, A1rB1i, -A2iB2i, A2rb2i
				// Temp1 = A1rB1r - A1iB1i, A1iB1r + A1rB1i, A2rB2r - A2iB2i, A2iB2r + A2rB2i
				Temp1 = VectorMultiplyAdd(Temp1, SignFlip, VectorInA);

				// VectorOut = O1r + A1rB1r - A1iB1i, O1i + A1iB1r + A1rB1i, O2r + A2rB2r - A2iB2i, O2i + A2iB2r + A2rB2i
				VectorRegister4Float VectorOut = VectorLoad(&OutData[i]);
				VectorOut = VectorAdd(Temp1, VectorOut);

				VectorStore(VectorOut, &OutData[i]);
			}

			for (int32 i = NumSimd; i < Num; i += 2)
			{
				// Real output
				OutData[i] += (InAData[i] * InBData[i]) - (InAData[i + 1] * InBData[i + 1]);
				// Imaginary output
				OutData[i + 1] += (InAData[i + 1] * InBData[i]) + (InAData[i] * InBData[i + 1]);
			}
		}
	}

	void ArrayMultiplyByConstant(TArrayView<const float> InFloatBuffer, float InValue, TArrayView<float> OutFloatBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMultiplyByConstant);
		check(InFloatBuffer.Num() == OutFloatBuffer.Num());

		const int32 Num = InFloatBuffer.Num();

		// Get ptrs to audio buffers to avoid bounds check in non-shipping builds
		const float* InBufferPtr = InFloatBuffer.GetData();
		float* OutBufferPtr = OutFloatBuffer.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMultiplyByConstant(InBufferPtr, InValue, OutBufferPtr, Num);
#endif
		}
		else
		{
			// Can only SIMD on multiple of 4 buffers, we'll do normal multiples on last bit
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				// Load the single value we want to multiply all values by into a vector register
				const VectorRegister4Float MultiplyValue = VectorLoadFloat1(&InValue);
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					// Load the next 4 samples of the input buffer into a register
					VectorRegister4Float InputBufferRegister = VectorLoad(&InBufferPtr[i]);

					// Perform the multiply
					VectorRegister4Float Temp = VectorMultiply(InputBufferRegister, MultiplyValue);

					// Store results into the output buffer
					VectorStore(Temp, &OutBufferPtr[i]);
				}
			}

			if (NumNotToSimd)
			{
				// Perform remaining non-simd values left over
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					OutBufferPtr[i] = InValue * InBufferPtr[i];
				}
			}
		}
	}

	void ArrayMultiplyByConstantInPlace(TArrayView<float> InOutBuffer, float InGain)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMultiplyByConstantInPlace);
		int32 Num = InOutBuffer.Num();
		float* InOutData = InOutBuffer.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMultiplyByConstantInPlace(InOutData, Num, InGain);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const VectorRegister4Float Gain = VectorLoadFloat1(&InGain);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Output = VectorLoad(&InOutData[i]);
					Output = VectorMultiply(Output, Gain);
					VectorStore(Output, &InOutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InOutData[i] *= InGain;
				}
			}
		}
	}

	void ArrayAddInPlace(TArrayView<const float> InValues, TArrayView<float> InAccumulateValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayAddInPlace);

		check(InValues.Num() == InAccumulateValues.Num());

		const int32 Num = InValues.Num();

		const float* InData = InValues.GetData();
		float* InAccumulateData = InAccumulateValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayAddInPlace(InData, InAccumulateData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InData[i]);
					VectorRegister4Float VectorAccumData = VectorLoad(&InAccumulateData[i]);

					VectorRegister4Float VectorOut = VectorAdd(VectorData, VectorAccumData);
					VectorStore(VectorOut, &InAccumulateData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InAccumulateData[i] += InData[i];
				}
			}
		}
	}

	void ArrayAddConstantInplace(TArrayView<float> InOutBuffer, float InConstant)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayAddConstantInplace);
		int32 Num = InOutBuffer.Num();
		float* InOutData = InOutBuffer.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayAddConstantInplace(InOutData, Num, InConstant);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const VectorRegister4Float Constant = VectorLoadFloat1(&InConstant);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Output = VectorLoad(&InOutData[i]);
					Output = VectorAdd(Output, Constant);
					VectorStore(Output, &InOutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InOutData[i] += InConstant;
				}
			}
		}
	}

	void ArrayMultiplyAddInPlace(TArrayView<const float> InValues, float InMultiplier, TArrayView<float> InAccumulateValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMultiplyAddInPlace);
		check(InValues.Num() == InAccumulateValues.Num());
		
		const int32 Num = InValues.Num();

		const float* InData = InValues.GetData();
		float* InAccumulateData = InAccumulateValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMultiplyAddInPlace(InData, InMultiplier, InAccumulateData, Num);
#endif
		}
		else
		{
			for (int32 i = 0; i < Num; i++)
			{
				InAccumulateData[i] += InData[i] * InMultiplier;
			}
		}
	}

	void ArrayLerpAddInPlace(TArrayView<const float> InValues, float InStartMultiplier, float InEndMultiplier, TArrayView<float> InAccumulateValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayLerpAddInPlace);
		check(InValues.Num() == InAccumulateValues.Num());

		const int32 Num = InValues.Num();

		const float* InData = InValues.GetData();
		float* InAccumulateData = InAccumulateValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayLerpAddInPlace(InData, InStartMultiplier, InEndMultiplier, InAccumulateData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const float Delta = (InEndMultiplier - InStartMultiplier) / FMath::Max(1.f, static_cast<float>(Num - 1));

			const float FourByDelta = 4.f * Delta;
			VectorRegister4Float VectorDelta = MakeVectorRegister(FourByDelta, FourByDelta, FourByDelta, FourByDelta);
			VectorRegister4Float VectorMultiplier = MakeVectorRegister(InStartMultiplier, InStartMultiplier + Delta, InStartMultiplier + 2.f * Delta, InStartMultiplier + 3.f * Delta);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InData[i]);
					VectorRegister4Float VectorAccumData = VectorLoad(&InAccumulateData[i]);

					VectorRegister4Float VectorOut = VectorMultiplyAdd(VectorData, VectorMultiplier, VectorAccumData);
					VectorMultiplier = VectorAdd(VectorMultiplier, VectorDelta);

					VectorStore(VectorOut, &InAccumulateData[i]);
				}
			}

			if (NumNotToSimd)
			{
				float Multiplier = InStartMultiplier + NumToSimd * Delta;

				for (int32 i = NumToSimd; i < Num; i++)
				{
					InAccumulateData[i] += InData[i] * Multiplier;
					Multiplier += Delta;
				}
			}
		}
	}

	/* Subtracts two buffers together element-wise. */
	void ArraySubtract(TArrayView<const float> InMinuend, TArrayView<const float> InSubtrahend, TArrayView<float> OutBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySubtract);
		const int32 Num = InMinuend.Num();

		checkf(Num == InSubtrahend.Num() && Num == OutBuffer.Num(), TEXT("InMinuend, InSubtrahend, and OutBuffer must have equal Num elements (%d vs %d vs %d)"), Num, InSubtrahend.Num(), OutBuffer.Num());

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySubtract(InMinuend.GetData(), InSubtrahend.GetData(), OutBuffer.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input1 = VectorLoad(&InMinuend[i]);
					VectorRegister4Float Input2 = VectorLoad(&InSubtrahend[i]);
					VectorRegister4Float Output = VectorSubtract(Input1, Input2);
					VectorStore(Output, &OutBuffer[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					OutBuffer[i] = InMinuend[i] - InSubtrahend[i];
				}
			}
		}
	}

	/* Performs element-wise in-place subtraction placing the result in the subtrahend. InOutSubtrahend = InMinuend - InOutSubtrahend */
	void ArraySubtractInPlace1(TArrayView<const float> InMinuend, TArrayView<float> InOutSubtrahend)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySubtractInPlace1);
		checkf(InMinuend.Num() == InOutSubtrahend.Num(), TEXT("Input buffers must be equal length"));

		const int32 Num = InMinuend.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySubtractInPlace1(InMinuend.GetData(), InOutSubtrahend.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input1 = VectorLoad(&InMinuend[i]);
					VectorRegister4Float Input2 = VectorLoad(&InOutSubtrahend[i]);

					VectorRegister4Float Output = VectorSubtract(Input1, Input2);
					VectorStore(Output, &InOutSubtrahend[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					InOutSubtrahend[i] = InMinuend[i] - InOutSubtrahend[i];
				}
			}
		}
	}

	/* Performs element-wise in-place subtraction placing the result in the minuend. InOutMinuend = InOutMinuend - InSubtrahend */
	void ArraySubtractInPlace2(TArrayView<float> InOutMinuend, TArrayView<const float> InSubtrahend)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySubtractInPlace2);
		checkf(InOutMinuend.Num() == InSubtrahend.Num(), TEXT("Input buffers must be equal length"));

		const int32 Num = InOutMinuend.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySubtractInPlace2(InOutMinuend.GetData(), InSubtrahend.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float Input1 = VectorLoad(&InOutMinuend[i]);
					VectorRegister4Float Input2 = VectorLoad(&InSubtrahend[i]);

					VectorRegister4Float Output = VectorSubtract(Input1, Input2);
					VectorStore(Output, &InOutMinuend[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					InOutMinuend[i] = InOutMinuend[i] - InSubtrahend[i];
				}
			}
		}
	}

	void ArraySubtractByConstantInPlace(TArrayView<float> InValues, float InSubtrahend)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySubtractByConstantInPlace);
		const int32 Num = InValues.Num();
		float* InData = InValues.GetData();
		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySubtractByConstantInPlace(InData, InSubtrahend, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const VectorRegister4Float VectorSubtrahend = VectorSetFloat1(InSubtrahend);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InData[i]);
					VectorData = VectorSubtract(VectorData, VectorSubtrahend);
					VectorStore(VectorData, &InData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InData[i] -= InSubtrahend;
				}
			}
		}
	}

	void ArraySquare(TArrayView<const float> InValues, TArrayView<float> OutValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySquare);
		check(InValues.Num() == OutValues.Num());

		const int32 Num = InValues.Num();

		const float* InData = InValues.GetData();
		float* OutData = OutValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySquare(InData, OutData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InData[i]);
					VectorData = VectorMultiply(VectorData, VectorData);
					VectorStore(VectorData, &OutData[i]);
				}
			}
	
			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					OutData[i] = InData[i] * InData[i];
				}
			}
		}
	}

	void ArraySquareInPlace(TArrayView<float> InValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySquareInPlace);
		const int32 Num = InValues.Num();

		float* InData = InValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySquareInPlace(InData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InData[i]);
					VectorData = VectorMultiply(VectorData, VectorData);
					VectorStore(VectorData, &InData[i]);
				}
			}
	
			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InData[i] = InData[i] * InData[i];
				}
			}
		}
	}

	void ArraySqrtInPlace(TArrayView<float> InValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySqrtInPlace);

		const int32 Num = InValues.Num();
		float* InValuesData = InValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySqrtInPlace(InValuesData, Num);
#endif
		}
		else
		{
			for (int32 i = 0; i < Num; i++)
			{
				InValues[i] = FMath::Sqrt(InValues[i]);
			}
		}
	}

	void ArrayComplexConjugate(TArrayView<const float> InValues, TArrayView<float> OutValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayComplexConjugate);

		check(OutValues.Num() == InValues.Num());
		check((InValues.Num() % 2) == 0);

		int32 Num = InValues.Num();

		const float* InData = InValues.GetData();
		float* OutData = OutValues.GetData();
		
		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayComplexConjugate(InData, OutData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const VectorRegister4Float ConjugateMult = MakeVectorRegister(1.f, -1.f, 1.f, -1.f);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InData[i]);

					VectorData = VectorMultiply(VectorData, ConjugateMult);

					VectorStore(VectorData, &OutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i += 2)
				{
					OutData[i] = InData[i];
					OutData[i + 1] = -InData[i + 1];
				}
			}
		}
	}

	void ArrayComplexConjugateInPlace(TArrayView<float> InValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayComplexConjugateInPlace);

		check((InValues.Num() % 2) == 0);

		int32 Num = InValues.Num();

		float* InData = InValues.GetData();
		
		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayComplexConjugateInPlace(InData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const VectorRegister4Float ConjugateMult = MakeVectorRegister(1.f, -1.f, 1.f, -1.f);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InData[i]);

					VectorData = VectorMultiply(VectorData, ConjugateMult);

					VectorStore(VectorData, &InData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					if ((i % 2) == 1)
					{
						InData[i] *= -1.f;
					}
				}
			}
		}
	}

	void ArrayMagnitudeToDecibelInPlace(TArrayView<float> InValues, float InMinimumDb)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMagnitudeToDecibelInPlace);

		const int32 Num = InValues.Num();
		float* InValuesData = InValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMagnitudeToDecibelInPlace(InValuesData, InMinimumDb, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const float Scale = 20.f / MathIntrinsics::Loge10;
			const float Minimum = FMath::Exp(InMinimumDb * MathIntrinsics::Loge10 / 20.f);

			const VectorRegister4Float VectorScale = VectorSetFloat1(Scale);
			const VectorRegister4Float VectorMinimum = VectorSetFloat1(Minimum);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InValuesData[i]);

					VectorData = VectorMax(VectorData, VectorMinimum);
					VectorData = VectorLog(VectorData);
					VectorData = VectorMultiply(VectorData, VectorScale);

					VectorStore(VectorData, &InValuesData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InValuesData[i] = FMath::Max(InValuesData[i], Minimum);
					InValuesData[i] = 20.f * FMath::Loge(InValuesData[i]) / MathIntrinsics::Loge10;
				}
			}
		}
	}

	void ArrayPowerToDecibelInPlace(TArrayView<float> InValues, float InMinimumDb)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayPowerToDecibelInPlace);

		const int32 Num = InValues.Num();
		float* InValuesData = InValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayPowerToDecibelInPlace(InValuesData, InMinimumDb, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const float Scale = 10.f / MathIntrinsics::Loge10;
			const float Minimum = FMath::Exp(InMinimumDb * MathIntrinsics::Loge10 / 10.f);

			const VectorRegister4Float VectorMinimum = VectorSetFloat1(Minimum);
			const VectorRegister4Float VectorScale = VectorSetFloat1(Scale);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorData = VectorLoad(&InValuesData[i]);

					VectorData = VectorMax(VectorData, VectorMinimum);
					VectorData = VectorLog(VectorData);
					VectorData = VectorMultiply(VectorData, VectorScale);

					VectorStore(VectorData, &InValuesData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InValuesData[i] = FMath::Max(InValuesData[i], Minimum);
					InValuesData[i] = 10.f * FMath::Loge(InValuesData[i]) / MathIntrinsics::Loge10;
				}
			}
		}
	}

	void ArrayComplexToPower(TArrayView<const float> InComplexValues, TArrayView<float> OutPowerValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayComplexToPower);

		check((InComplexValues.Num() % 2) == 0);
		check(InComplexValues.Num() == (OutPowerValues.Num() * 2));

		const int32 NumOut = OutPowerValues.Num();

		const float* InComplexData = InComplexValues.GetData();
		float* OutPowerData = OutPowerValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayComplexToPowerInterleaved(InComplexData, OutPowerData, NumOut);
#endif
		}
		else
		{
			const int32 NumToSimd = NumOut & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = NumOut & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VectorComplex1 = VectorLoad(&InComplexData[2 * i]);
					VectorRegister4Float VectorSquared1 = VectorMultiply(VectorComplex1, VectorComplex1);

					VectorRegister4Float VectorComplex2 = VectorLoad(&InComplexData[(2 * i) + 4]);
					VectorRegister4Float VectorSquared2 = VectorMultiply(VectorComplex2, VectorComplex2);

					VectorRegister4Float VectorSquareReal = VectorShuffle(VectorSquared1, VectorSquared2, 0, 2, 0, 2);
					VectorRegister4Float VectorSquareImag = VectorShuffle(VectorSquared1, VectorSquared2, 1, 3, 1, 3);

					VectorRegister4Float VectorOut = VectorAdd(VectorSquareReal, VectorSquareImag);

					VectorStore(VectorOut, &OutPowerData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < NumOut; i++)
				{
					int32 ComplexPos = 2 * i;

					float RealValue = InComplexData[ComplexPos];
					float ImagValue = InComplexData[ComplexPos + 1];

					OutPowerData[i] = (RealValue * RealValue) + (ImagValue * ImagValue);
				}
			}
		}
	}

	void ArrayComplexToPower(TArrayView<const float> InRealSamples, TArrayView<const float> InImaginarySamples, TArrayView<float> OutPowerSamples)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayComplexToPower);

		checkf(InRealSamples.Num() == InImaginarySamples.Num(), TEXT("Input buffers must have equal number of elements"));

		const int32 Num = InRealSamples.Num();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayComplexToPower(InRealSamples.GetData(), InImaginarySamples.GetData(), OutPowerSamples.GetData(), Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VInReal = VectorLoad(&InRealSamples[i]);
					VectorRegister4Float VInRealSquared = VectorMultiply(VInReal, VInReal);

					VectorRegister4Float VInImag = VectorLoad(&InImaginarySamples[i]);
					VectorRegister4Float VOut = VectorMultiplyAdd(VInImag, VInImag, VInRealSquared);

					VectorStore(VOut, &OutPowerSamples[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; ++i)
				{
					const float InRealSquared = InRealSamples[i] * InRealSamples[i];
					const float InImagSquared = InImaginarySamples[i] * InImaginarySamples[i];

					OutPowerSamples[i] = InRealSquared + InImagSquared;
				}
			}
		}
	}

	/* Sets a values to zero if value is denormal. Denormal numbers significantly slow down floating point operations. */
	void ArrayUnderflowClamp(TArrayView<float> InOutValues)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayUnderflowClamp);

		int32 Num = InOutValues.Num();
		float* InOutData = InOutValues.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayUnderflowClamp(InOutData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const VectorRegister4Float VFMIN = MakeVectorRegister(FLT_MIN, FLT_MIN, FLT_MIN, FLT_MIN);
			const VectorRegister4Float VNFMIN = MakeVectorRegister(-FLT_MIN, -FLT_MIN, -FLT_MIN, -FLT_MIN);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VInOut = VectorLoad(&InOutData[i]);

					// Create mask of denormal numbers.
					VectorRegister4Float Mask = VectorBitwiseAnd(VectorCompareGT(VInOut, VNFMIN), VectorCompareLT(VInOut, VFMIN));

					// Choose between zero or original number based upon mask.
					VInOut = VectorSelect(Mask, GlobalVectorConstants::FloatZero, VInOut);
					VectorStore(VInOut, &InOutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					float InOut = InOutData[i];

					// Create mask of denormal numbers.
					const bool Mask = (InOut > -FLT_MIN) && (InOut < FLT_MIN);

					// Choose between zero or original number based upon mask.
					InOut = Mask ? 0.0f : InOut;
					InOutData[i] = InOut;
				}
			}
		}
	}

	/* Clamps values in the buffer to be between InMinValue and InMaxValue */
	void ArrayRangeClamp(TArrayView<float> InOutBuffer, float InMinValue, float InMaxValue)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayRangeClamp);

		int32 Num = InOutBuffer.Num();
		float* InOutData = InOutBuffer.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayRangeClamp(InOutData, Num, InMinValue, InMaxValue);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const VectorRegister4Float VMinVal = MakeVectorRegister(InMinValue, InMinValue, InMinValue, InMinValue);
			const VectorRegister4Float VMaxVal = MakeVectorRegister(InMaxValue, InMaxValue, InMaxValue, InMaxValue);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorRegister4Float VInOut = VectorLoad(&InOutData[i]);

					// Create masks to flag elements outside of range.
					VectorRegister4Float MinMask = VectorCompareLT(VInOut, VMinVal);
					VectorRegister4Float MaxMask = VectorCompareGT(VInOut, VMaxVal);

					// Choose between range extremes or original number based on masks.
					VInOut = VectorSelect(MinMask, VMinVal, VInOut);
					VInOut = VectorSelect(MaxMask, VMaxVal, VInOut);

					VectorStore(VInOut, &InOutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InOutData[i] = FMath::Clamp(InOutData[i], InMinValue, InMaxValue);
				}
			}
		}
	}

	void ArraySetToConstantInplace(TArrayView<float> InOutBuffer, float InConstant)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArraySetToConstantInplace);

		int32 Num = InOutBuffer.Num();
		float* InOutData = InOutBuffer.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArraySetToConstantInplace(InOutData, Num, InConstant);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			const VectorRegister4Float Constant = VectorLoadFloat1(&InConstant);

			if (NumToSimd)
			{
				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					VectorStore(Constant, &InOutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					InOutData[i] = InConstant;
				}
			}
		}
	}

	/* Performs an element-wise weighted sum OutputBuffer = (InBuffer1 x InGain1) + (InBuffer2 x InGain2) */
	void ArrayWeightedSum(TArrayView<const float> InBuffer1, float InGain1, TArrayView<const float> InBuffer2, float InGain2, TArrayView<float> OutBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayWeightedSum);

		checkf(InBuffer1.Num() == InBuffer2.Num(), TEXT("Buffers must be equal length"));

		int32 Num = InBuffer1.Num();
		const float* InData1 = InBuffer1.GetData();
		const float* InData2 = InBuffer2.GetData();
		float* OutData = OutBuffer.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayWeightedSumTwoGain(InData1, InGain1, InData2, InGain2, OutData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				VectorRegister4Float Gain1Vector = VectorLoadFloat1(&InGain1);
				VectorRegister4Float Gain2Vector = VectorLoadFloat1(&InGain2);

				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					// InBuffer1 x InGain1
					VectorRegister4Float Input1 = VectorLoad(&InData1[i]);

					// InBuffer2 x InGain2
					VectorRegister4Float Input2 = VectorLoad(&InData2[i]);
					VectorRegister4Float Weighted2 = VectorMultiply(Input2, Gain2Vector);

					VectorRegister4Float Output = VectorMultiplyAdd(Input1, Gain1Vector, Weighted2);
					VectorStore(Output, &OutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					OutData[i] = (InData1[i] * InGain1) + (InData2[i] * InGain2);
				}
			}
		}
	}

	/* Performs an element-wise weighted sum OutputBuffer = (InBuffer1 x InGain1) + InBuffer2 */
	void ArrayWeightedSum(TArrayView<const float> InBuffer1, float InGain1, TArrayView<const float> InBuffer2, TArrayView<float> OutBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayWeightedSum);
		checkf(InBuffer1.Num() == InBuffer2.Num() && InBuffer1.Num() == OutBuffer.Num(), TEXT("Buffers must be equal length"));

		int32 Num = InBuffer1.Num();
		const float* InData1 = InBuffer1.GetData();
		const float* InData2 = InBuffer2.GetData();
		float* OutData = OutBuffer.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayWeightedSumOneGain(InData1, InGain1, InData2, OutData, Num);
#endif
		}
		else
		{
			const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
			const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

			if (NumToSimd)
			{
				VectorRegister4Float Gain1Vector = VectorLoadFloat1(&InGain1);

				for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
				{
					// InBuffer1 x InGain1
					VectorRegister4Float Input1 = VectorLoad(&InData1[i]);
					VectorRegister4Float Input2 = VectorLoad(&InData2[i]);
					VectorRegister4Float Output = VectorMultiplyAdd(Input1, Gain1Vector, Input2);
					VectorStore(Output, &OutData[i]);
				}
			}

			if (NumNotToSimd)
			{
				for (int32 i = NumToSimd; i < Num; i++)
				{
					OutData[i] = (InData1[i] * InGain1) + InData2[i];
				}
			}
		}
	}

	void ArrayFade(TArrayView<float> InOutBuffer, const float StartValue, const float EndValue)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayFade);

		int32 Num = InOutBuffer.Num();
		float* OutFloatBuffer = InOutBuffer.GetData();

		if (FMath::IsNearlyEqual(StartValue, EndValue))
		{
			// No need to do anything if start and end values are both 0.0
			if (StartValue == 0.0f)
			{
				FMemory::Memset(OutFloatBuffer, 0, sizeof(float) * Num);
			}
			else
			{
				ArrayMultiplyByConstantInPlace(InOutBuffer, StartValue);
			}
		}
		else
		{
			if (bAudio_FloatArrayMath_ISPC_Enabled)
			{
#if INTEL_ISPC
				ispc::ArrayFade(OutFloatBuffer, Num, StartValue, EndValue);
#endif
			}
			else
			{
				const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
				const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

				const float DeltaValue = ((EndValue - StartValue) / Num);

				if (NumToSimd)
				{
					constexpr VectorRegister4Float VectorFour = MakeVectorRegisterFloatConstant(4.f, 4.f, 4.f, 4.f);
					VectorRegister4Float Accumulator = MakeVectorRegisterFloat(0.f, 1.f, 2.f, 3.f);
					VectorRegister4Float Delta = VectorLoadFloat1(&DeltaValue);
					VectorRegister4Float Start = VectorLoadFloat1(&StartValue);

					for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
					{
						VectorRegister4Float Output = VectorLoad(&OutFloatBuffer[i]);
						VectorRegister4Float Gain = VectorMultiplyAdd(Accumulator, Delta, Start);
						Output = VectorMultiply(Output, Gain);
						Accumulator = VectorAdd(Accumulator, VectorFour);
						VectorStore(Output, &OutFloatBuffer[i]);
					}
				}

				if (NumNotToSimd)
				{
					float Gain = (NumToSimd * DeltaValue) + StartValue;

					// Do a fade from start to end
					for (int32 i = NumToSimd; i < Num; ++i)
					{
						OutFloatBuffer[i] = OutFloatBuffer[i] * Gain;
						Gain += DeltaValue;
					}
				}
			}
		}
	}

	void ArrayFade(TArrayView<const float> InBuffer, const float InStartValue, const float InEndValue, TArrayView<float> OutBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayFade);
		
		const int32 Num = InBuffer.Num();
		check(Num <= OutBuffer.Num());

		const float* InFloatBuffer = InBuffer.GetData();
		float* OutFloatBuffer = OutBuffer.GetData();

		// case 1: no fade
		if (FMath::IsNearlyEqual(InStartValue, InEndValue))
		{
			if (InStartValue == 0.0f)
			{
				// No need to do anything if start and end values are both 0.0
				FMemory::Memset(OutFloatBuffer, 0, sizeof(float) * Num);
			}
			else
			{
				// no fade, just scale the output
				ArrayMultiplyByConstant(InBuffer, InStartValue, OutBuffer);
			}

			return;
		}

		// case 2: fade w/ ISPC
#if INTEL_ISPC
		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
			ispc::ArrayFade2(InFloatBuffer, Num, InStartValue, InEndValue, OutFloatBuffer);
			return;
		}
#endif


		// case 3: fade w/ our vectorization abstraction
		const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
		const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

		const float DeltaValue = ((InEndValue - InStartValue) / Num);

		if (NumToSimd)
		{
			constexpr VectorRegister4Float VectorFour = MakeVectorRegisterFloatConstant(4.f, 4.f, 4.f, 4.f);
			VectorRegister4Float Accumulator = MakeVectorRegisterFloat(0.f, 1.f, 2.f, 3.f);
			VectorRegister4Float Delta = VectorLoadFloat1(&DeltaValue);
			VectorRegister4Float Start = VectorLoadFloat1(&InStartValue);

			for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				VectorRegister4Float Gain = VectorMultiplyAdd(Accumulator, Delta, Start);
				VectorRegister4Float Input = VectorLoad(&InFloatBuffer[i]);
				VectorRegister4Float Output = VectorMultiply(Input, Gain);

				Accumulator = VectorAdd(Accumulator, VectorFour);
				VectorStore(Output, &OutFloatBuffer[i]);
			}
		}

		if (NumNotToSimd)
		{
			float Gain = (NumToSimd * DeltaValue) + InStartValue;

			// Do a fade from start to end
			for (int32 i = NumToSimd; i < Num; ++i)
			{
				OutFloatBuffer[i] = InFloatBuffer[i] * Gain;
				Gain += DeltaValue;
			}
		}
	}

	void ArrayMixIn(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToSumTo, const float Gain)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMixIn);

		checkf(InFloatBuffer.Num() == BufferToSumTo.Num(), TEXT("Buffers must be equal size"));

		int32 Num = InFloatBuffer.Num();
		const float* InData = InFloatBuffer.GetData();
		float* InOutData = BufferToSumTo.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMixInWithGain(InData, InOutData, Num, Gain);
#endif
		}
		else
		{
			VectorRegister4Float GainVector = VectorLoadFloat1(&Gain);
			int32 i = 0;
			const int32 SimdNum = Num & MathIntrinsics::Simd16Mask;
			for (; i < SimdNum; i += 16)
			{
				// manually unrolling the loop produces a bit faster code
				VectorRegister4x4Float Input = VectorLoad16(&InData[i]);
				VectorRegister4x4Float Output = VectorLoad16(&InOutData[i]);
				Output.val[0] = VectorMultiplyAdd(Input.val[0], GainVector, Output.val[0]);
				Output.val[1] = VectorMultiplyAdd(Input.val[1], GainVector, Output.val[1]);
				Output.val[2] = VectorMultiplyAdd(Input.val[2], GainVector, Output.val[2]);
				Output.val[3] = VectorMultiplyAdd(Input.val[3], GainVector, Output.val[3]);
				VectorStore16(Output, &InOutData[i]);
			}

			for (; i < Num; ++i)
			{
				InOutData[i] += InData[i] * Gain;
			}
		}
	}

	void ArrayMixIn(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToSumTo)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMixIn);

		checkf(InFloatBuffer.Num() == BufferToSumTo.Num(), TEXT("Buffers must be equal size"));

		int32 Num = InFloatBuffer.Num();
		const float* InData = InFloatBuffer.GetData();
		float* InOutData = BufferToSumTo.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayMixIn(InData, InOutData, Num);
#endif
		}
		else
		{
			int32 i = 0;
			const int32 SimdNum = Num & MathIntrinsics::Simd16Mask;
			for (; i < SimdNum; i += 16)
			{
				// manually unrolling the loop produces a bit faster code
				VectorRegister4x4Float Input = VectorLoad16(&InData[i]);
				VectorRegister4x4Float Output = VectorLoad16(&InOutData[i]);
				Output.val[0] = VectorAdd(Input.val[0], Output.val[0]);
				Output.val[1] = VectorAdd(Input.val[1], Output.val[1]);
				Output.val[2] = VectorAdd(Input.val[2], Output.val[2]);
				Output.val[3] = VectorAdd(Input.val[3], Output.val[3]);
				VectorStore16(Output, &InOutData[i]);
			}

			for (; i < Num; ++i)
			{
				InOutData[i] += InData[i];
			}
		}
	}

	void ArrayMixIn(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToSumTo, const float StartGain, const float EndGain)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMixIn);

		checkf(InFloatBuffer.Num() == BufferToSumTo.Num(), TEXT("Buffers must be equal size"));

		int32 Num = InFloatBuffer.Num();

		if (FMath::IsNearlyEqual(StartGain, EndGain))
		{
			// No need to do anything if start and end values are both 0.0
			if (StartGain == 0.0f)
			{
				return;
			}
			else
			{
				ArrayMixIn(InFloatBuffer, BufferToSumTo, StartGain);
			}
		}
		else
		{
			if (bAudio_FloatArrayMath_ISPC_Enabled)
			{
#if INTEL_ISPC
				ispc::ArrayMixInWithDelta(InFloatBuffer.GetData(), BufferToSumTo.GetData(), Num, StartGain, EndGain);
#endif
			}
			else
			{
				const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
				const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

				const float DeltaValue = ((EndGain - StartGain) / Num);

				if (NumToSimd)
				{
					constexpr VectorRegister4Float VectorFour = MakeVectorRegisterFloatConstant(4.f, 4.f, 4.f, 4.f);
					VectorRegister4Float Accumulator = MakeVectorRegisterFloat(0.f, 1.f, 2.f, 3.f);

					VectorRegister4Float Start = VectorLoadFloat1(&StartGain);
					VectorRegister4Float Delta = VectorLoadFloat1(&DeltaValue);

					for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
					{
						VectorRegister4Float Input = VectorLoad(&InFloatBuffer[i]);
						VectorRegister4Float Output = VectorLoad(&BufferToSumTo[i]);
						VectorRegister4Float Gain = VectorMultiplyAdd(Accumulator, Delta, Start);
						Output = VectorMultiplyAdd(Input, Gain, Output);
						Accumulator = VectorAdd(Accumulator, VectorFour);
						VectorStore(Output, &BufferToSumTo[i]);
					}
				}

				if (NumNotToSimd)
				{
					float Gain = (NumToSimd * DeltaValue) + StartGain;

					for (int32 i = NumToSimd; i < Num; ++i)
					{
						BufferToSumTo[i] += InFloatBuffer[i] * Gain;
						Gain += DeltaValue;
					}
				}
			}
		}
	}

	void ArrayMixIn(TArrayView<const int16> InPcm16Buffer, TArrayView<float> BufferToSumTo, const float Gain)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayMixIn);

		checkf(InPcm16Buffer.Num() == BufferToSumTo.Num(), TEXT("Buffers must be equal size"));

		const int32 Num = InPcm16Buffer.Num();

		const int32 NumToSimd = Num & MathIntrinsics::SimdMask;
		const int32 NumNotToSimd = Num & MathIntrinsics::NotSimdMask;

		const int16* InputPtr = InPcm16Buffer.GetData();
		float* OutPtr = BufferToSumTo.GetData();

		const float ConversionValue = Gain / static_cast<float>(TNumericLimits<int16>::Max());

		if (NumToSimd)
		{
			const VectorRegister4Float ConversionVector = VectorSetFloat1(ConversionValue);
			AlignedFloat4 FloatArray(GlobalVectorConstants::FloatZero);

			for (int32 i = 0; i < NumToSimd; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				FloatArray[0] = (float)InputPtr[i];
				FloatArray[1] = (float)InputPtr[i + 1];
				FloatArray[2] = (float)InputPtr[i + 2];
				FloatArray[3] = (float)InputPtr[i + 3];

				const VectorRegister4Float InVector = FloatArray.ToVectorRegister();
				const VectorRegister4Float OutData = VectorLoad(&OutPtr[i]);
				const VectorRegister4Float ScaledVector = VectorMultiplyAdd(InVector, ConversionVector, OutData);
				VectorStore(ScaledVector, &OutPtr[i]);
			}
		}

		if (NumNotToSimd)
		{
			for (int32 i = NumToSimd; i < Num; i++)
			{
				OutPtr[i] += (float)InputPtr[i] * ConversionValue;
			}
		}
	}

	void ArrayFilterAndMixIn(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToSumTo, const float Gain, const int32 InNumChannels, bool bEnableLPF, float InLpfCutoff, bool bEnableHPF, float InHpfCutoff, FInterpolatedLPF& InLpf, FInterpolatedHPF& InHpf, FAlignedFloatBuffer& ScratchBuffer)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayFilterAndMixIn);

		if (!ensureMsgf(InFloatBuffer.Num() == BufferToSumTo.Num(), TEXT("ArrayFilterAndMixIn: input (%d) and output (%d) buffers must be equal size"), InFloatBuffer.Num(), BufferToSumTo.Num()))
		{
			return;
		}
		
		if (!ensureMsgf(InNumChannels > 0, TEXT("ArrayFilterAndMixIn: InNumChannels must be > 0")))
		{
			return;
		}
		checkf(InNumChannels <= 1 || InFloatBuffer.Num() % InNumChannels == 0, TEXT("NumSamples must be frame-aligned (multiple of InNumChannels)"));

		const int32 NumSamples = InFloatBuffer.Num();

		const bool bBypassLPF = !bEnableLPF;
		const bool bBypassHPF = !bEnableHPF;

		if (bBypassLPF && bBypassHPF)
		{
			ArrayMixIn(InFloatBuffer, BufferToSumTo, Gain);
			return;
		}

		// Copy source into scratch and filter in place.
		ScratchBuffer.SetNumUninitialized(NumSamples);
		FMemory::Memcpy(ScratchBuffer.GetData(), InFloatBuffer.GetData(), NumSamples * sizeof(float));

		// Coefficient steps once per frame across all SIMD paths, so InterpLength must be NumFrames not NumSamples.
		// StartFrequencyInterpolation + Process + StopFrequencyInterpolation must always be called as a matched set -
		// failing to Stop after exactly InterpLength frames causes the coefficient to drift and the filter to become unstable.
		const int32 NumFrames = NumSamples / FMath::Max(InNumChannels, 1);

		if (!bBypassLPF)
		{
			InLpf.StartFrequencyInterpolation(InLpfCutoff, NumFrames);
			InLpf.ProcessBufferInPlace(ScratchBuffer.GetData(), NumSamples);
			InLpf.StopFrequencyInterpolation();
		}

		if (!bBypassHPF)
		{
			InHpf.StartFrequencyInterpolation(InHpfCutoff, NumFrames);
			InHpf.ProcessBufferInPlace(ScratchBuffer.GetData(), NumSamples);
			InHpf.StopFrequencyInterpolation();
		}

		ArrayMixIn(ScratchBuffer, BufferToSumTo, Gain);
	}

	void ArrayFloatToPcm16(TArrayView<const float> InView, TArrayView<int16> OutView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayFloatToPcm16);

		check(OutView.Num() >= InView.Num());

		const int32 Num = InView.Num();

		const float* InputPtr = InView.GetData();
		int16* OutPtr = OutView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayFloatToPcm16(InputPtr, OutPtr, Num);
#endif
		}
		else
		{
			constexpr float ConversionValue = static_cast<float>(TNumericLimits<int16>::Max());
			const VectorRegister4Float Multiplier = VectorSetFloat1(ConversionValue);
			int32 i = 0;
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
			const int32 SimdNum = Num & MathIntrinsics::Simd8Mask;
			for (; i < SimdNum; i += 8)
			{
				const float32x4x2_t InVector = vld1q_f32_x2(&InputPtr[i]);
				const VectorRegister4Float ScaledVector1 = VectorMultiply(InVector.val[0], Multiplier);
				const VectorRegister4Float ScaledVector2 = VectorMultiply(InVector.val[1], Multiplier);
				const VectorRegister4Int IntVector1 = VectorFloatToInt(ScaledVector1);
				const VectorRegister4Int IntVector2 = VectorFloatToInt(ScaledVector2);
				const int16x8_t Result = vmovn_high_s32(vmovn_u32(IntVector1), IntVector2);
				vst1q_s16(&OutPtr[i], Result);
			}
#else
			const int32 SimdNum = Num & MathIntrinsics::SimdMask;
			for (; i < SimdNum; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				const VectorRegister4Float InVector = VectorLoad(&InputPtr[i]);
				const VectorRegister4Float ScaledVector = VectorMultiply(InVector, Multiplier);
				const VectorRegister4Int IntVector = VectorFloatToInt(ScaledVector);

				const AlignedFloat4 ScaledFloatArray(ScaledVector);

				OutPtr[i + 0] = (int16)ScaledFloatArray[0];
				OutPtr[i + 1] = (int16)ScaledFloatArray[1];
				OutPtr[i + 2] = (int16)ScaledFloatArray[2];
				OutPtr[i + 3] = (int16)ScaledFloatArray[3];
			}
#endif //~PLATFORM_ENABLE_VECTORINTRINSICS_NEON

			for (; i < Num; i++)
			{
				OutPtr[i] = (int16)(InputPtr[i] * ConversionValue);
			}
		}
	}
	
	void ArrayPcm16ToFloat(TArrayView<const int16> InView, TArrayView<float> OutView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayPcm16ToFloat);

		check(OutView.Num() >= InView.Num());

		const int32 Num = InView.Num();

		const int16* InputPtr = InView.GetData();
		float* OutPtr = OutView.GetData();

		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayPcm16ToFloat(InputPtr, OutPtr, Num);
#endif
		}
		else
		{
			constexpr float ConversionValue = 1.f / static_cast<float>(TNumericLimits<int16>::Max());
			const VectorRegister4Float Multiplier = VectorSetFloat1(ConversionValue);

			int32 i = 0;
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
			const int32 SimdNum = Num & MathIntrinsics::Simd8Mask;
			for (; i < SimdNum; i += 8)
			{
				int16x8_t Data = vld1q_s16(&InputPtr[i]);
				int32x4_t VecA = vmovl_s16(vget_low_s16(Data));
				int32x4_t VecB = vmovl_high_s16(Data);
				float32x4x2_t FloatVec;
				FloatVec.val[0] = VectorMultiply(vcvtq_f32_s32(VecA), Multiplier);
				FloatVec.val[1] = VectorMultiply(vcvtq_f32_s32(VecB), Multiplier);
				vst1q_f32_x2(&OutPtr[i], FloatVec);
			}
#else
			AlignedFloat4 FloatArray(GlobalVectorConstants::FloatZero);
			const int32 SimdNum = Num & MathIntrinsics::SimdMask;
			for (; i < SimdNum; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				FloatArray[0] = (float)InputPtr[i];
				FloatArray[1] = (float)InputPtr[i + 1];
				FloatArray[2] = (float)InputPtr[i + 2];
				FloatArray[3] = (float)InputPtr[i + 3];

				const VectorRegister4Float InVector = FloatArray.ToVectorRegister();
				const VectorRegister4Float ScaledVector = VectorMultiply(InVector, Multiplier);

				VectorStore(ScaledVector, &OutPtr[i]);
			}
#endif //~PLATFORM_ENABLE_VECTORINTRINSICS_NEON

			for (; i < Num; i++)
			{
				OutPtr[i] = (float)InputPtr[i] * ConversionValue;
			}
		}
	}
	
	constexpr int CreateByteMask(uint32 A, uint32 B, uint32 C, uint32 D)
	{
		return A | (B << 8) | (C << 16) | (D << 24);
	}

	void ArrayFloatToPcm24(TArrayView<const float> InView, TArrayView<int8> OutView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayFloatToPcm24);

        const int32 Num = InView.Num();
		const float* InputPtr = InView.GetData();
		int8* OutPtr = OutView.GetData();

        int32 InIndex = 0;
        int32 OutIndex = 0;
        
        constexpr int32 SizeofPCM24 = 3;
        constexpr float ConversionValue = static_cast<float>(0x7fffff);

        constexpr int32 SimdIndexStride = (4 * SizeofPCM24);
        check((OutView.Num() / SizeofPCM24) >= InView.Num());

		const VectorRegister4Float Multiplier = VectorSetFloat1(ConversionValue);

		// In the SIMD loop below, we overwrite an extra 4 bytes of zeros. The ZeroFillMargin
		// is used to ensure we don't write off the end of the output array.
		const int32 ZeroFillMargin = 2;
		const int32 SimdNum = (Num - ZeroFillMargin) & MathIntrinsics::SimdMask;

		// 0x80 designates zero fill for the _mm_shuffle_epi8 intrinsic for SSE
		// Neon uses >= number of source bytes, which is 0x10 in this case so 0x80 works for both platforms
		constexpr uint32 ZeroFill = 0x80;
		// Here we convert from 4 32-bit ints to 4 24-bit ints using the mask register below.
		// Notice that every 4th byte is skipped. This is the uneeded high byte of the 32-bit int.
		VectorRegister4Int Mask = MakeVectorRegisterInt(
			CreateByteMask(0, 1, 2, 4),
			CreateByteMask(5, 6, 8, 9),
			CreateByteMask(10, 12, 13, 14),
			CreateByteMask(ZeroFill, ZeroFill, ZeroFill, ZeroFill));

		for (; InIndex < SimdNum; InIndex += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			const VectorRegister4Float InVector = VectorLoad(&InputPtr[InIndex]);
			const VectorRegister4Float ScaledVector = VectorMultiply(InVector, Multiplier);
			const VectorRegister4Int IntVector = VectorFloatToInt(ScaledVector);

			const VectorRegister4Int OutVector = VectorShuffleByte4(IntVector, Mask);
			VectorIntStore(OutVector, &OutPtr[OutIndex]);
			OutIndex += SimdIndexStride;
		}
        
		for (; InIndex < Num; InIndex++)
		{
			// Cast to signed integer first because casting a negative float directly to an 
			// unsigned int is undefined behavior. Some compilers will assign zero. Others
			// will implicily cast to signed int first and then to unsigned.
			// https://en.cppreference.com/w/c/language/conversion
			const int32 ConvertedValue = InputPtr[InIndex] * ConversionValue;
			const uint32 UnsignedValue = uint32(ConvertedValue);

			uint8* UnsignedOutPtr = (uint8*)&OutPtr[OutIndex];

			UnsignedOutPtr[0] = UnsignedValue & 0xFF;
			UnsignedOutPtr[1] = UnsignedValue >> 8 & 0xFF;
			UnsignedOutPtr[2] = UnsignedValue >> 16 & 0xFF;

			OutIndex += SizeofPCM24;
		}
	}

	void ArrayFloatToPcm32(TArrayView<const float> InView, TArrayView<int32> OutView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayFloatToPcm32);

		check(OutView.Num() >= InView.Num());

		const int32 Num = InView.Num();
		const float* InputPtr = InView.GetData();
		int32* OutPtr = OutView.GetData();

        int32 Index = 0;
        // Use double precision due to the limitations of
        // single precision floats (e.g. values >= 2^24 get rounded)
        constexpr double ConversionValue = static_cast<double>(TNumericLimits<int32>::Max());

		const VectorRegister4Double Multiplier = VectorSetFloat1(ConversionValue);

		const int32 SimdNum = Num & MathIntrinsics::SimdMask;
		for (; Index < SimdNum; Index += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			const VectorRegister4Float InputVectorFloat = VectorLoad(&InputPtr[Index]);
			const VectorRegister4Double InputVector = MakeVectorRegisterDouble(InputVectorFloat);
			const VectorRegister4Double ScaledVector = VectorMultiply(InputVector, Multiplier);
			
			const VectorRegister4Int OutVector = VectorDoubleToInt(ScaledVector);
			VectorIntStore(OutVector, &OutPtr[Index]);
		}
        
		for (; Index < Num; Index++)
		{
			const double Value = InputPtr[Index];
			OutPtr[Index] = static_cast<int32>(Value * ConversionValue);
		}
	}

	void ArrayFloatToPcmDouble(TArrayView<const float> InView, TArrayView<double> OutView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayFloatToPcmDouble);

		check(OutView.Num() >= InView.Num());

		const int32 Num = InView.Num();

		const float* InputPtr = InView.GetData();
		double* OutPtr = OutView.GetData();

		int32 Index = 0;
		const int32 SimdNum = Num & MathIntrinsics::SimdMask;
		for (; Index < SimdNum; Index += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
		{
			const VectorRegister4Float InputVectorFloat = VectorLoad(&InputPtr[Index]);
			const VectorRegister4Double OutVector = MakeVectorRegisterDouble(InputVectorFloat);

			VectorStore(OutVector, &OutPtr[Index]);
		}

		for (; Index < Num; ++Index)
		{
			OutPtr[Index] = (double)InputPtr[Index];
		}
	}

	void ArrayInterleave(const TArray<FAlignedFloatBuffer>& InBuffers, FAlignedFloatBuffer& OutBuffer)
	{
		if(InBuffers.Num() == 0)
		{
			return;
		}
		const int32 NumChannels = InBuffers.Num();
		const int32 NumFrames = InBuffers[0].Num();

		OutBuffer.SetNumUninitialized(NumChannels * NumFrames);
		
		TArray<const float*> BufferPtrArray;
		BufferPtrArray.Reset(NumChannels);
		
		for(const FAlignedFloatBuffer& Buffer : InBuffers)
		{
			const float* BufferPtr = Buffer.GetData();
			BufferPtrArray.Add(BufferPtr);
		}
		
		const float** InBufferPtr = BufferPtrArray.GetData();
		
		ArrayInterleave(InBufferPtr, OutBuffer.GetData(), NumFrames, NumChannels);
	}

	void ArrayInterleave(const float* const* RESTRICT InBuffers, float* RESTRICT OutBuffer, const int32 InFrames, const int32 InChannels)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayInterleave);
		for(int32 ChannelIdx = 0; ChannelIdx < InChannels; ChannelIdx++)
		{
			const float* InPtr = InBuffers[ChannelIdx];
			float* OutPtr = &OutBuffer[ChannelIdx];
			
			for(int32 SampleIdx = 0; SampleIdx < InFrames; SampleIdx++)
			{
				*OutPtr = *InPtr++;
				OutPtr += InChannels;
			}
		}
	}

	void ArrayDeinterleave(const FAlignedFloatBuffer& InBuffer, TArray<FAlignedFloatBuffer>& OutBuffers, const int32 InChannels)
	{
		check(InChannels > 0);
		
		const int32 NumFrames = InBuffer.Num() / InChannels;

		ArrayDeinterleave(TArrayView<const float>(InBuffer.GetData(), NumFrames), OutBuffers, InChannels);
	}

	void ArrayDeinterleave(const TArrayView<const float> InView, TArray<FAlignedFloatBuffer>& OutBuffers, const int32 InChannels)
	{
		check(InChannels > 0);
		
		const int32 NumFrames = InView.Num();

		TArray<float*> BufferPtrArray;
		BufferPtrArray.Reset(InChannels);

		OutBuffers.SetNum(InChannels);
		
		for(FAlignedFloatBuffer& Buffer : OutBuffers)
		{
			Buffer.SetNumUninitialized(NumFrames);
			
			float* BufferPtr = Buffer.GetData();
			BufferPtrArray.Add(BufferPtr);
		}
		
		float** OutBufferPtr = BufferPtrArray.GetData();
		
		ArrayDeinterleave(InView.GetData(), OutBufferPtr, NumFrames, InChannels);
	}

	void ArrayDeinterleave(const float* RESTRICT InBuffer, float* const* RESTRICT OutBuffers, const int32 InFrames, const int32 InChannels)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayDeinterleave);

		for (int32 ChannelIdx = 0; ChannelIdx < InChannels; ChannelIdx++)
		{
			const float* InPtr = &InBuffer[ChannelIdx];
			float* OutPtr = OutBuffers[ChannelIdx];

			for (int32 SampleIdx = 0; SampleIdx < InFrames; SampleIdx++)
			{
				*OutPtr++ = *InPtr;
				InPtr += InChannels;
			}
		}
	}

	void ArrayInterpolate(const float* InBuffer, float* OutBuffer, const int32 NumInSamples, const int32 NumOutSamples)
	{
		if (NumOutSamples <= 0 || NumInSamples <= 0)
		{
			return;
		}

		const float SampleStride = (float)NumInSamples / (float)NumOutSamples;

		const int32 NumToSimd = NumOutSamples & MathIntrinsics::SimdMask;
		const int32 NumNotToSimd = NumOutSamples & MathIntrinsics::NotSimdMask;

		if (NumToSimd)
		{
			VectorRegister4Float Strides = VectorSet(
				4.f * SampleStride,
				4.f * SampleStride,
				4.f * SampleStride,
				4.f * SampleStride
			);

			VectorRegister4Float Indeces = VectorSet(
				0.f * SampleStride,
				1.f * SampleStride,
				2.f * SampleStride,
				3.f * SampleStride
			);

			for (int32 OutputIndex = 0; OutputIndex < NumToSimd; OutputIndex += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				alignas(16) int32 LeftIndecesRaw[4];
				alignas(16) int32 RightIndecesRaw[4];

				VectorRegister4Float LeftIndeces = VectorFloor(Indeces);
				VectorRegister4Float Fractions = VectorSubtract(Indeces, LeftIndeces);
				VectorRegister4Float InvFractions = VectorSubtract(GlobalVectorConstants::FloatOne, Fractions);

				VectorRegister4Int LeftIndecesInt = VectorFloatToInt(LeftIndeces);

				// Lookup samples for interpolation
				VectorIntStoreAligned(LeftIndecesInt, LeftIndecesRaw);
				VectorIntStoreAligned(VectorIntAdd(LeftIndecesInt, GlobalVectorConstants::IntOne), RightIndecesRaw);

				VectorRegister4Float LowerSamples = VectorSet(
					InBuffer[LeftIndecesRaw[0]],
					InBuffer[LeftIndecesRaw[1]],
					InBuffer[LeftIndecesRaw[2]],
					InBuffer[LeftIndecesRaw[3]]
				);
				VectorRegister4Float UpperSamples = VectorSet(
					InBuffer[RightIndecesRaw[0]],
					InBuffer[RightIndecesRaw[1]],
					InBuffer[RightIndecesRaw[2]],
					InBuffer[RightIndecesRaw[3]]
				);

				VectorRegister4Float VOut = VectorMultiplyAdd(
					LowerSamples,
					Fractions,
					VectorMultiply(UpperSamples, InvFractions));
				VectorStore(VOut, &OutBuffer[OutputIndex]);

				Indeces = VectorAdd(Indeces, Strides);
			}
		}

		if (NumNotToSimd)
		{
			float SampleIndex = (float)(NumToSimd)*SampleStride;

			for (int32 OutputIndex = NumToSimd; OutputIndex < NumOutSamples; OutputIndex++)
			{
				const int32 LeftSample = FMath::FloorToInt32(SampleIndex);
				int32 RightSample = FMath::CeilToInt32(SampleIndex);

				const float Frac = SampleIndex - LeftSample;
				OutBuffer[OutputIndex] = (Frac * InBuffer[LeftSample]) + ((1.f - Frac) * InBuffer[RightSample]);

				SampleIndex += SampleStride;
			}
		}
	}

	void ArrayInt16SwapBytes(TArrayView<int16> InView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayInt16SwapBytes);

		const int32 Num = InView.Num();
		int16* InputPtr = InView.GetData();

		int32 Index = 0;
		const int32 SimdNum = Num & MathIntrinsics::Simd8Mask;
		constexpr int32 NumInt16PerVectorRegister = sizeof(VectorRegister4Int) / sizeof(int16);

		const VectorRegister4Int LeftMask = VectorIntSet1(0x00ff00ff);
		const VectorRegister4Int RightMask = VectorIntSet1(0xff00ff00);

		for (; Index < SimdNum; Index += NumInt16PerVectorRegister)
		{
			VectorRegister4Int InputVector = VectorIntLoad(&InputPtr[Index]);

			const VectorRegister4Int LeftVector = VectorShiftLeftImm(VectorIntAnd(InputVector, LeftMask), 8);
			const VectorRegister4Int RightVector = VectorShiftRightImmLogical(VectorIntAnd(InputVector, RightMask), 8);

			InputVector = VectorIntOr(LeftVector, RightVector);

			VectorIntStore(InputVector, &InputPtr[Index]);
		}

		for (; Index < Num; Index++)
		{
			InputPtr[Index] = BYTESWAP_ORDER16(InputPtr[Index]);
		}
	}

	void ArrayInt24SwapBytes(TArrayView<int8> InView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayInt24SwapBytes);

        const int32 NumBytes = InView.Num();
        int8* InputPtr = InView.GetData();

        int32 Index = 0;
        constexpr int32 SizeofPCM24 = 3;

		// We can fit 5 PCM24 samples into a vector register with one byte left over
		constexpr int32 NumPCM24PerVectorRegister = sizeof(VectorRegister4Int) / SizeofPCM24;
		constexpr int32 SimdIndexStride = (NumPCM24PerVectorRegister * SizeofPCM24);

		const int32 MaxSimdBytes = NumBytes - sizeof(VectorRegister4Int);

		VectorRegister4Int Mask = MakeVectorRegisterInt(
			CreateByteMask(2, 1, 0, 5),
			CreateByteMask(4, 3, 8, 7),
			CreateByteMask(6, 11, 10, 9),
			CreateByteMask(14, 13, 12, 15));

		for (; Index < MaxSimdBytes; Index += SimdIndexStride)
		{
			const VectorRegister4Int InputVector = VectorIntLoad(&InputPtr[Index]);
			const VectorRegister4Int OutVector = VectorShuffleByte4(InputVector, Mask);

			VectorIntStore(OutVector, &InputPtr[Index]);
		}
        
		for (; Index < NumBytes; Index += SizeofPCM24)
		{
			int8 TempValue = InputPtr[Index];

			InputPtr[Index] = InputPtr[Index + 2];
			InputPtr[Index + 2] = TempValue;
		}
	}

	void ArrayInt32SwapBytes(TArrayView<int32> InView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayInt32SwapBytes);

		const int32 Num = InView.Num();
		int32* InputPtr = InView.GetData();

        int32 Index = 0;

		const int32 SimdNum = Num & MathIntrinsics::SimdMask;
		constexpr int32 NumInt32PerVectorRegister = sizeof(VectorRegister4Int) / sizeof(int32);

		VectorRegister4Int Mask = MakeVectorRegisterInt(
			CreateByteMask(3, 2, 1, 0),
			CreateByteMask(7, 6, 5, 4),
			CreateByteMask(11, 10, 9, 8),
			CreateByteMask(15, 14, 13, 12));

		for (; Index < SimdNum; Index += NumInt32PerVectorRegister)
		{
			const VectorRegister4Int InputVector = VectorIntLoad(&InputPtr[Index]);

			// Byte shuffle is approximately 2x faster than mask and shift method in this case
			const VectorRegister4Int OutVector = VectorShuffleByte4(InputVector, Mask);

			VectorIntStore(OutVector, &InputPtr[Index]);
		}
        
		for (; Index < Num; Index++)
		{
			InputPtr[Index] = BYTESWAP_ORDER32(InputPtr[Index]);
		}
	}

	void ArrayFloatSwapBytes(TArrayView<float> InView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayFloatSwapBytes);

		ArrayInt32SwapBytes(TArrayView<int32>((int32*)InView.GetData(), InView.Num()));
	}

	void ArrayDoubleSwapBytes(TArrayView<double> InView)
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, ArrayDoubleSwapBytes);

		const int32 Num = InView.Num();
		double* InputPtr = InView.GetData();

        int32 Index = 0;

		const int32 SimdNum = Num & MathIntrinsics::SimdMask;
		constexpr int32 NumDoublePerVectorRegister = sizeof(VectorRegister4Double) / sizeof(double);

		VectorRegister4Int Mask = MakeVectorRegisterInt(
			CreateByteMask(7, 6, 5, 4),
			CreateByteMask(3, 2, 1, 0),
			CreateByteMask(15, 14, 13, 12),
			CreateByteMask(11, 10, 9, 8));

		for (; Index < SimdNum; Index += NumDoublePerVectorRegister)
		{
			const VectorRegister4Double InputVector = VectorLoad(&InputPtr[Index]);

			const VectorRegister4Int OutVectorXY = VectorShuffleByte4(VectorCastDoubleToInt(InputVector.XY), Mask);
			const VectorRegister4Int OutVectorZW = VectorShuffleByte4(VectorCastDoubleToInt(InputVector.ZW), Mask);

			const VectorRegister4Double OutVector(VectorCastIntToDouble(OutVectorXY), VectorCastIntToDouble(OutVectorZW));

			VectorStore(OutVector, &InputPtr[Index]);
		}
        
		for (; Index < Num; Index++)
		{
			InputPtr[Index] = BYTESWAP_ORDERD(InputPtr[Index]);
		}
	}

	void ArrayAPFLongDelayProcess(const float* InSamples, const float* InDelaySamples, const int32 InNum, float* OutSamples, float* OutDelaySamples, const float Gain)
	{
		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayAPFLongDelayProcess(InSamples,
				InDelaySamples,
				InNum,
				OutSamples,
				OutDelaySamples,
				Gain);
#endif
		}
		else
		{
			// Calculate new delay line samples. "w[n] = x[n] + gw[n - d]"
			int32 NumToSIMD = InNum - (InNum % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);

			VectorRegister4Float VG = MakeVectorRegisterFloat(Gain, Gain, Gain, Gain);
			VectorRegister4Float VNG = MakeVectorRegisterFloat(-Gain, -Gain, -Gain, -Gain);

			for (int32 i = 0; i < InNum; i += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				VectorRegister4Float VInDelay = VectorLoadAligned(&InDelaySamples[i]);
				VectorRegister4Float VInSamples = VectorLoadAligned(&InSamples[i]);
				// w[n] = x[n] + G * w[n - D]
				VectorRegister4Float VOutDelay = VectorMultiplyAdd(VInDelay, VG, VInSamples);
				VectorStoreAligned(VOutDelay, &OutDelaySamples[i]);

				// y[n] = -G * w[n] + w[n - D]
				VectorRegister4Float VOut = VectorMultiplyAdd(VOutDelay, VNG, VInDelay);
				VectorStoreAligned(VOut, &OutSamples[i]);
			}

			// Calculate allpass for remaining samples that we couldn't SIMD
			for (int32 i = NumToSIMD; i < InNum; i++)
			{
				OutDelaySamples[i] = InDelaySamples[i] * Gain + InSamples[i];
				OutSamples[i] = OutDelaySamples[i] * -Gain + InDelaySamples[i];
			}
		}
	}

	void ArrayLerpFractionalDelay(const float* InSamples, const float* InDelays, const float* DelayData, const int* IntegerDelays, int* UpperDelayPos, int* LowerDelayPos, const int32 InNum, float* OutSamples, const float MaxDelay)
	{
		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayLerpFractionalDelay(InSamples,
				InDelays,
				DelayData,
				IntegerDelays,
				InNum,
				OutSamples,
				MaxDelay);
#endif
		}
		else
		{
			const VectorRegister4Float VMaxDelay = MakeVectorRegister(MaxDelay, MaxDelay, MaxDelay, MaxDelay);
			for (int32 i = 0; i < InNum; i += 4)
			{
				VectorRegister4Float VFractionalDelays = VectorLoad(&InDelays[i]);
				// Ensure fractional delays are positive
				VFractionalDelays = VectorMax(VFractionalDelays, GlobalVectorConstants::FloatZero);
				VFractionalDelays = VectorMin(VFractionalDelays, VMaxDelay);

				// Separate integer from fraction
				VectorRegister4Float VFloorDelays = VectorFloor(VFractionalDelays);

				// Determine linear weights
				VectorRegister4Float VUpperCoefficients = VectorSubtract(VFractionalDelays, VFloorDelays);
				VectorRegister4Float VLowerCoefficients = VectorSubtract(GlobalVectorConstants::FloatOne, VUpperCoefficients);


				// Make integer locations relative to block
				VectorRegister4Int VIntegerDelays = VectorFloatToInt(VFloorDelays);
				VectorRegister4Int VIntegerDelayOffset = VectorIntLoadAligned(&IntegerDelays[i]);
				VIntegerDelays = VectorIntSubtract(VIntegerDelayOffset, VIntegerDelays);

				// Lookup samples for interpolation
				VectorIntStoreAligned(VIntegerDelays, UpperDelayPos);
				VectorIntStoreAligned(VectorIntAdd(VIntegerDelays, GlobalVectorConstants::IntOne), LowerDelayPos);

				VectorRegister4Float VLowerSamples = MakeVectorRegister(
					DelayData[LowerDelayPos[0]],
					DelayData[LowerDelayPos[1]],
					DelayData[LowerDelayPos[2]],
					DelayData[LowerDelayPos[3]]
				);
				VectorRegister4Float VUpperSamples = MakeVectorRegister(
					DelayData[UpperDelayPos[0]],
					DelayData[UpperDelayPos[1]],
					DelayData[UpperDelayPos[2]],
					DelayData[UpperDelayPos[3]]
				);

				// Interpolate samples
				VectorRegister4Float VOut = VectorMultiplyAdd(
					VLowerSamples,
					VLowerCoefficients,
					VectorMultiply(VUpperSamples, VUpperCoefficients));
				VectorStore(VOut, &OutSamples[i]);
			}
		}
	}

	void ArrayScaledComplexConjugate(const float* RESTRICT InValues, const int32 Num, float* RESTRICT OutValues, const float Scale)
	{
		if (bAudio_FloatArrayMath_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ArrayScaledComplexConjugate(InValues,
				Num,
				OutValues,
				Scale);
#endif
		}
		else
		{
			// Use mask to quickly find out number of values that can be SIMD'd
			const int32 SIMD_MASK = 0xFFFFFFFC;
			const int32 NumToSimd = SIMD_MASK & Num;

			// Complex values in a vector are [real_1, imag_1, real_2, imag_2].  
			// By multipling this value, we flip the sign of the imaginary components, which
			// is the equivalent of a complex conjugate.
			const VectorRegister4Float SignFlipImag = MakeVectorRegisterFloat(Scale, -Scale, Scale, -Scale);

			// Perform operation using SIMD
			for (int32 i = 0; i < NumToSimd; i += 4)
			{
				VectorRegister4Float Value = VectorLoad(&InValues[i]);
				Value = VectorMultiply(SignFlipImag, Value);
				VectorStore(Value, &OutValues[i]);
			}

			// Perform operation where SIMD not possible.
			for (int32 i = NumToSimd; i < Num; i += 2)
			{
				OutValues[i] = Scale * InValues[i];
				OutValues[i + 1] = -Scale * InValues[i + 1];
			}
		}
	}

	FContiguousSparse2DKernelTransform::FContiguousSparse2DKernelTransform(const int32 NumInElements, const int32 NumOutElements)
	:	NumIn(NumInElements)
	,	NumOut(NumOutElements)
	{
		check(NumIn >= 0);
		check(NumOut >= 0)
		FRow EmptyRow;
		EmptyRow.StartIndex = 0;

		// Fill up the kernel with empty rows
		Kernel.Init(EmptyRow, NumOut);
	}

	FContiguousSparse2DKernelTransform::~FContiguousSparse2DKernelTransform()
	{
	}

	int32 FContiguousSparse2DKernelTransform::GetNumInElements() const
	{
		return NumIn;
	}


	int32 FContiguousSparse2DKernelTransform::GetNumOutElements() const
	{
		return NumOut;
	}

	void FContiguousSparse2DKernelTransform::SetRow(const int32 RowIndex, const int32 StartIndex, TArrayView<const float> OffsetValues)
	{
		check((StartIndex + OffsetValues.Num()) <= NumIn);

		// Copy row data internally
		Kernel[RowIndex].StartIndex = StartIndex;
		Kernel[RowIndex].OffsetValues = TArray<float>(OffsetValues.GetData(), OffsetValues.Num());
	}

	void FContiguousSparse2DKernelTransform::TransformArray(TArrayView<const float> InView, TArray<float>& OutArray) const
	{
		check(InView.Num() == NumIn);

		// Resize output
		OutArray.Reset(NumOut);
		if (NumOut > 0)
		{
			OutArray.AddUninitialized(NumOut);
		}

		TransformArray(InView.GetData(), OutArray.GetData());
	}

	void FContiguousSparse2DKernelTransform::TransformArray(TArrayView<const float> InView, FAlignedFloatBuffer& OutArray) const
	{	
		check(InView.Num() == NumIn);

		// Resize output
		OutArray.Reset(NumOut);
		if (NumOut > 0)
		{
			OutArray.AddUninitialized(NumOut);
		}

		TransformArray(InView.GetData(), OutArray.GetData());
	}

	void FContiguousSparse2DKernelTransform::TransformArray(const float* InArray, float* OutArray) const
	{
		CSV_SCOPED_TIMING_STAT(Audio_Dsp, TransformArray);

		check(nullptr != InArray);
		check(nullptr != OutArray);

		// Initialize output
		FMemory::Memset(OutArray, 0, sizeof(float) * NumOut);

		// Apply kernel one row at a time
		const FRow* KernelData = Kernel.GetData();
		for (int32 RowIndex = 0; RowIndex < Kernel.Num(); RowIndex++)
		{
			const FRow& Row = KernelData[RowIndex];

			// Get offset pointer into input array.
			const float* OffsetInData = &InArray[Row.StartIndex];
			// Get offset pointer of row.
			const float* RowValuePtr = Row.OffsetValues.GetData();

			// dot prod 'em. 
			int32 NumToMult = Row.OffsetValues.Num();

			if (bAudio_FloatArrayMath_ISPC_Enabled)
			{
#if INTEL_ISPC
				ispc::TransformArrayRow(OffsetInData, RowValuePtr, OutArray, RowIndex, NumToMult);
#endif
			}
			else
			{
				for (int32 i = 0; i < NumToMult; i++)
				{
					OutArray[RowIndex] += OffsetInData[i] * RowValuePtr[i];
				}
			}
		}
	}
}
