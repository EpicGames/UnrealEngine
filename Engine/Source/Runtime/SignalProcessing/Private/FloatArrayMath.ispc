// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Aossoa.isph"
#include "Math/Scalar.isph"

export void ArraySum(const uniform float InData[],
					uniform float &OutSum,
					const uniform int Num)
{
	varying float Sum = 0.0f;

	foreach(i = 0 ... Num)
	{
		Sum += InData[i];
	}

	OutSum = reduce_add(Sum);
}

export void ArraySum2(const uniform float InFloatBuffer1[],
						const uniform float InFloatBuffer2[],
						uniform float OutputBuffer[],
						const uniform int NumSamples)
{
	foreach(i = 0 ... NumSamples)
	{
		const varying float Input1 = InFloatBuffer1[i];
		const varying float Input2 = InFloatBuffer2[i];

		OutputBuffer[i] = Input1 + Input2;
	}
}

export void ArrayCumulativeSum(const uniform float InViewPtr[],
								uniform float OutDataPtr[],
								const uniform int Num)
{
	uniform float Sum = 0;

	for(uniform int32 i = 0; i < Num; i++)
	{
		Sum += InViewPtr[i];
		OutDataPtr[i] = Sum;
	}
}

export void ArrayMean(const uniform float DataPtr[],
					uniform float &OutMean,
					const uniform int Num)
{
	varying float Sum = 0.0f;

	foreach(i = 0 ... Num)
	{
		Sum += DataPtr[i];
	}

	OutMean = reduce_add(Sum) / (uniform float)Num;
}

export void ArrayMeanSquared(const uniform float DataPtr[],
							uniform float &OutMean,
							const uniform int Num)
{
	varying float Sum = 0.0f;

	foreach(i = 0 ... Num)
	{
		Sum += DataPtr[i] * DataPtr[i];
	}

	OutMean = reduce_add(Sum) / (uniform float)Num;
}

export void ArrayMax(const uniform float InFloatBuffer1[],
	const uniform float InFloatBuffer2[],
	uniform float OutputBuffer[], 
	const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		OutputBuffer[i] = max(InFloatBuffer1[i], InFloatBuffer2[i]);
	}
}

export uniform float ArrayMaxAbsValue(const uniform float DataPtr[], const uniform int Num)
{
	float Max = 0.f;

	foreach(i = 0 ... Num)
	{
		Max = max(abs(DataPtr[i]), Max);
	}

	return reduce_max(Max);
}

export uniform float ArrayGetMagnitude(const uniform float Buffer[], const uniform int NumSamples)
{
	varying float Sum = 0.0f;
	foreach(i = 0 ... NumSamples)
	{
		const float Input = Buffer[i];
		Sum += Input * Input;
	}
	return sqrt(reduce_add(Sum));
}

export uniform float ArrayGetAverageValue(const uniform float Buffer[], const uniform int NumSamples)
{
	varying float Sum = 0.0f;
	foreach(i = 0 ... NumSamples)
	{
		const float Input = Buffer[i];
		Sum += Input;
	}
	return reduce_add(Sum) / NumSamples;
}

export uniform float ArrayGetAverageAbsValue(const uniform float Buffer[], const uniform int NumSamples)
{
	varying float Sum = 0.0f;
	foreach(i = 0 ... NumSamples)
	{
		const float Input = Buffer[i];
		Sum += abs(Input);
	}
	return reduce_add(Sum) / NumSamples;
}

export void ArrayMeanFilter(const uniform float SummedDataPtr[],
							const uniform int WindowSize,
							const uniform int WindowOrigin,
							uniform float OutDataPtr[],
							const uniform float LastSummedData,
							const uniform int Num)
{
	const uniform int32 LastIndexBeforeEndBoundaryCondition = max(WindowOrigin + 1, Num - WindowSize + WindowOrigin + 1);
	const uniform int32 StartOffset = -WindowOrigin - 1;
	const uniform int32 EndOffset = WindowSize - WindowOrigin - 1;
	const uniform int32 WindowTail = WindowSize - WindowOrigin;

	if ((WindowSize - WindowOrigin) < Num)
	{
		// Handle boundary condition where analysis window precedes beginning of array.
		foreach (i = 0 ... (WindowOrigin + 1))
		{
			OutDataPtr[i] = SummedDataPtr[i + EndOffset] / max(1.f, (varying float)(WindowTail + i));
		}

		// No boundary conditions to handle here.
		const uniform float MeanDivisor = (uniform float)(WindowSize);
		foreach (i = WindowOrigin + 1 ... LastIndexBeforeEndBoundaryCondition)
		{
			OutDataPtr[i] = (SummedDataPtr[i + EndOffset] - SummedDataPtr[i + StartOffset]) / MeanDivisor;
		}
	}
	else
	{
		// Handle boundary condition where window precedes beginning and goes past end of array
		const uniform float ArrayMean = LastSummedData / (uniform float)(Num);
		foreach (i = 0 ... LastIndexBeforeEndBoundaryCondition)
		{
			OutDataPtr[i] = ArrayMean;
		}
	}

	// Handle boundary condition where analysis window goes past end of array.
	foreach (i = LastIndexBeforeEndBoundaryCondition ... Num)
	{
		OutDataPtr[i] = (LastSummedData - SummedDataPtr[i + StartOffset]) / (varying float)(Num - i + WindowOrigin);
	}
}

export void ArrayGetEuclideanNorm(const uniform float InViewData[],
								uniform float &OutEuclideanNorm,
								const uniform int Num)
{
	varying float SquareSum = 0.0f;

	foreach(i = 0 ... Num)
	{
		SquareSum += InViewData[i] * InViewData[i];
	}

	OutEuclideanNorm = sqrt(reduce_add(SquareSum));
}

export void ArrayAbs(const uniform float InData[],
					uniform float OutData[],
					const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		OutData[i] = abs(InData[i]);
	}
}

export void ArrayAbsInPlace(uniform float Data[],
							const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		Data[i] = abs(Data[i]);
	}
}

export void ArrayClampMinInPlace(uniform float Data[],
								const uniform float InMin,
								const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		Data[i] = max(InMin, Data[i]);
	}
}

export void ArrayClampMaxInPlace(uniform float Data[],
								const uniform float InMax,
								const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		Data[i] = min(InMax, Data[i]);
	}
}

export void ArrayClampInPlace(uniform float Data[],
								const uniform float InMin,
								const uniform float InMax,
								const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		Data[i] = clamp(Data[i], InMin, InMax);
	}
}

export void ArrayMinMaxNormalize(const uniform float InDataPtr[],
								uniform float OutDataPtr[],
								const uniform int Num)
{
	varying float VMaxValue = InDataPtr[0];
	varying float VMinValue = InDataPtr[0];

	// Determine min and max
	foreach(i = 1 ... Num)
	{
		const varying float Data = InDataPtr[i];
		VMaxValue = max(VMaxValue, Data);
		VMinValue = min(VMinValue, Data);
	}

	const uniform float MaxValue = reduce_max(VMaxValue);
	const uniform float MinValue = reduce_min(VMinValue);

	// Normalize data by subtracting minimum value and dividing by range
	const uniform float Scale = 1.f / max(FLOAT_SMALL_NUMBER, MaxValue - MinValue);
	foreach(i = 0 ... Num)
	{
		OutDataPtr[i] = (InDataPtr[i] - MinValue) * Scale;
	}
}

export void ArrayMultiply(const uniform float InFloatBuffer[],
						  const uniform float BufferToMultiply[],
						  uniform float OutDataPtr[],
						  const uniform int NumSamples)
{
	foreach(i = 0 ... NumSamples)
	{
		const varying float Input = InFloatBuffer[i];
		const varying float Output = BufferToMultiply[i];

		OutDataPtr[i] = Input * Output;
	}
}

export void ArrayMultiplyInPlace(const uniform float InFloatBuffer[],
									uniform float BufferToMultiply[],
									const uniform int NumSamples)
{
	foreach(i = 0 ... NumSamples)
	{
		const varying float Input = InFloatBuffer[i];
		const varying float Output = BufferToMultiply[i];

		BufferToMultiply[i] = Input * Output;
	}
}

export void ArrayComplexMultiplyInPlace(const uniform float InData1[],
										uniform float InData2[],
										const uniform int Num)
{
#if TARGET_WIDTH == 4
	static const varying float RealSignFlip = { -1.f, 1.f, -1.f, 1.f };
#elif TARGET_WIDTH == 8
	static const varying float RealSignFlip = { -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f };
#elif TARGET_WIDTH == 16
	static const varying float RealSignFlip = { -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f };
#endif

	uniform int NumToSimd = Num & ~(programCount-1);

	for(uniform int i = 0; i < NumToSimd; i+=programCount)
	{
		const varying float VectorData1 = InData1[i+programIndex];
		const varying float VectorData2 = InData2[i+programIndex];

		const varying float VectorData1Real = VectorSwizzle(VectorData1, 0, 0, 2, 2);
		const varying float VectorData1Imag = VectorSwizzle(VectorData1, 1, 1, 3, 3);
		const varying float VectorData2Swizzle = VectorSwizzle(VectorData2, 1, 0, 3, 2);

		varying float Result = VectorData1Imag * VectorData2Swizzle;
		Result = Result * RealSignFlip;
		Result = (VectorData1Real * VectorData2) + Result;

		InData2[i+programIndex] = Result;
	}

	for(uniform int i = NumToSimd; i < Num; i+=2)
	{
		uniform float Real = (InData1[i] * InData2[i]) - (InData1[i + 1] * InData2[i + 1]);
		uniform float Imag = (InData1[i] * InData2[i + 1]) + (InData1[i + 1] * InData2[i]);
		InData2[i] = Real;
		InData2[i + 1] = Imag;
	}
}

export void ArrayComplexMultiplyAdd(const uniform float InA[],
	const uniform float InB[],
	uniform float Out[],
	const uniform int Num)
{
#if TARGET_WIDTH == 4
	static const varying float RealSignFlip = { -1.f, 1.f, -1.f, 1.f };
#elif TARGET_WIDTH == 8
	static const varying float RealSignFlip = { -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f };
#elif TARGET_WIDTH == 16
	static const varying float RealSignFlip = { -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f, -1.f, 1.f };
#endif

	foreach(i = 0 ... Num)
	{
		const varying float VectorData1 = InA[i];
		const varying float VectorData2 = InB[i];

		const varying float VectorData1Real = VectorSwizzle(VectorData1, 0, 0, 2, 2);
		const varying float VectorData1Imag = VectorSwizzle(VectorData1, 1, 1, 3, 3);
		const varying float VectorData2Swizzle = VectorSwizzle(VectorData2, 1, 0, 3, 2);

		varying float Result = VectorData1Imag * VectorData2Swizzle;
		Result = Result * RealSignFlip;
		Result = (VectorData1Real * VectorData2) + Result;

		Out[i] += Result;
	}
}

export void ArrayMultiplyByConstant(const uniform float InFloatBuffer[],
									const uniform float InValue,
									uniform float OutFloatBuffer[],
									const uniform int InNumSamples)
{
	foreach(i = 0 ... InNumSamples)
	{
		OutFloatBuffer[i] = InValue * InFloatBuffer[i];
	}
}

export void ArrayMultiplyByConstantInPlace(uniform float OutFloatBuffer[],
										const uniform int NumSamples,
										const uniform float InGain)
{
	foreach(i = 0 ... NumSamples)
	{
		const varying float Output = OutFloatBuffer[i];
		OutFloatBuffer[i] = Output * InGain;
	}
}

export void ArrayAddInPlace(const uniform float InData[],
							uniform float InAccumulateData[],
							const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		InAccumulateData[i] += InData[i];
	}
}

export void ArrayAddConstantInplace(uniform float OutFloatBuffer[],
									const uniform int NumSamples,
									const uniform float InConstant)
{
	foreach(i = 0 ... NumSamples)
	{
		const varying float Output = OutFloatBuffer[i];
		OutFloatBuffer[i] = Output + InConstant;
	}
}

export void ArrayMultiplyAddInPlace(const uniform float InData[],
									const uniform float InMultiplier,
									uniform float InAccumulateData[],
									const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		InAccumulateData[i] += InData[i] * InMultiplier;
	}
}

export void ArrayLerpAddInPlace(const uniform float InData[],
								const uniform float InStartMultiplier,
								const uniform float InEndMultiplier,
								uniform float InAccumulateData[],
								const uniform int Num)
{
	const uniform float Delta = (InEndMultiplier - InStartMultiplier) / max(1.f, (uniform float)(Num - 1));

	foreach(i = 0 ... Num)
	{
		const varying float Multiplier = InStartMultiplier + (i * Delta);
		InAccumulateData[i] += InData[i] * Multiplier;
	}
}

export void ArraySubtract(const uniform float InMinuend[],
								const uniform float InSubtrahend[],
								uniform float OutBuffer[],
								const uniform int InNum)
{
	foreach(i = 0 ... InNum)
	{
		const varying float Input1 = InMinuend[i];
		const varying float Input2 = InSubtrahend[i];

		const varying float Output = Input1 - Input2;

		OutBuffer[i] = Output;
	}
}

export void ArraySubtractInPlace1(const uniform float InMinuend[],
									uniform float InOutSubtrahend[],
									const uniform int InNum)
{
	foreach(i = 0 ... InNum)
	{
		const varying float Input1 = InMinuend[i];
		const varying float Input2 = InOutSubtrahend[i];

		const varying float Output = Input1 - Input2;

		InOutSubtrahend[i] = Output;
	}
}

export void ArraySubtractInPlace2(uniform float InOutMinuend[],
									const uniform float InSubtrahend[],
									const uniform int InNum)
{
	foreach(i = 0 ... InNum)
	{
		const varying float Input1 = InOutMinuend[i];
		const varying float Input2 = InSubtrahend[i];

		const varying float Output = Input1 - Input2;

		InOutMinuend[i] = Output;
	}
}

export void ArraySubtractByConstantInPlace(uniform float InData[],
											const uniform float InSubtrahend,
											const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		InData[i] -= InSubtrahend;
	}
}

export void ArraySquare(const uniform float InData[],
						uniform float OutData[],
						const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		OutData[i] = InData[i] * InData[i];
	}
}

export void ArraySquareInPlace(uniform float InData[],
								const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		InData[i] = InData[i] * InData[i];
	}
}

export void ArraySqrtInPlace(uniform float InData[],
								const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		InData[i] = sqrt(InData[i]);
	}
}

export void ArrayComplexConjugate(const uniform float InData[],
									uniform float OutData[],
									const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		OutData[i] = select(i & 1, -InData[i], InData[i]);
	}
}

export void ArrayComplexConjugateInPlace(uniform float InData[],
											const uniform int Num)
{
	foreach(i = 0 ... Num)
	{
		InData[i] = select(i & 1, -InData[i], InData[i]);
	}
}

export void ArrayMagnitudeToDecibelInPlace(uniform float InData[],
											const uniform float InMinimumDb,
											const uniform int Num)
{
	const uniform float Loge10 = log(10.0f);
	const uniform float Scale = 20.f / Loge10;
	const uniform float Minimum = exp(InMinimumDb * Loge10 / 20.f);

	foreach(i = 0 ... Num)
	{
		varying float VectorData = InData[i];
		VectorData = max(VectorData, Minimum);
		InData[i] = log(VectorData) * Scale;
	}
}

export void ArrayPowerToDecibelInPlace(uniform float InData[],
										const uniform float InMinimumDb,
										const uniform int Num)
{
	const uniform float Loge10 = log(10.0f);
	const uniform float Scale = 10.f / Loge10;
	const uniform float Minimum = exp(InMinimumDb * Loge10 / 10.f);

	foreach(i = 0 ... Num)
	{
		varying float VectorData = InData[i];
		VectorData = max(VectorData, Minimum);
		InData[i] = log(VectorData) * Scale;
	}
}

export void ArrayComplexToPowerInterleaved(const uniform float InComplexData[],
										uniform float OutPowerData[],
										const uniform int Num)
{
	uniform int NumToSimd = Num & ~(programCount-1);

	unmasked
	{
		for(uniform int i = 0; i < NumToSimd; i+=programCount)
		{
			const uniform int ComplexPos = 2 * i;
			varying float RealValue, ImagValue;
			aos_to_soa2_ispc((uniform float *uniform)&InComplexData[ComplexPos], &RealValue, &ImagValue);

			OutPowerData[i + programIndex] = (RealValue * RealValue) + (ImagValue * ImagValue);
		}
	}

	for (uniform int32 i = NumToSimd; i < Num; i++)
	{
		uniform int ComplexPos = 2 * i;

		uniform float RealValue = InComplexData[ComplexPos];
		uniform float ImagValue = InComplexData[ComplexPos + 1];

		OutPowerData[i] = (RealValue * RealValue) + (ImagValue * ImagValue);
	}
}

export void ArrayComplexToPower(const uniform float InRealSamples[],
									const uniform float InImaginarySamples[],
									uniform float OutPowerSamples[],
									const uniform int NumSamples)
{
	foreach(i = 0 ... NumSamples)
	{
		const varying float Real = InRealSamples[i];
		const varying float Imag = InImaginarySamples[i];

		OutPowerSamples[i] = (Real * Real) + (Imag * Imag);
	}
}

export void ArrayUnderflowClamp(uniform float InOutBuffer[],
									const uniform int InNum)
{
	foreach(i = 0 ... InNum)
	{
		const varying float VInOut = InOutBuffer[i];

		// Create mask of denormal numbers.
		const varying float Mask = VectorBitwiseAnd(VectorCompareGT(VInOut, -FLT_MIN), VectorCompareLT(VInOut, FLT_MIN));

		// Choose between zero or original number based upon mask.
		InOutBuffer[i] = VectorSelect(Mask, 0.0f, VInOut);
	}
}

export void ArrayRangeClamp(uniform float InOutBuffer[],
								const uniform int InNum,
								const uniform float InMinValue,
								const uniform float InMaxValue)
{
	foreach(i = 0 ... InNum)
	{
		InOutBuffer[i] = clamp(InOutBuffer[i], InMinValue, InMaxValue);
	}
}

export void ArraySetToConstantInplace(uniform float InBuffer[],
									const uniform int NumSamples,
									const uniform float InConstant)
{
	foreach(i = 0 ... NumSamples)
	{
		InBuffer[i] = InConstant;
	}
}

export void ArrayWeightedSumTwoGain(const uniform float InBuffer1[],
										const uniform float InGain1,
										const uniform float InBuffer2[],
										const uniform float InGain2,
										uniform float OutBuffer[],
										const uniform int InNum)
{
	foreach(i = 0 ... InNum)
	{
		OutBuffer[i] = (InBuffer1[i] * InGain1) + (InBuffer2[i] * InGain2);
	}
}

export void ArrayWeightedSumOneGain(const uniform float InBuffer1[],
										const uniform float InGain1,
										const uniform float InBuffer2[],
										uniform float OutBuffer[],
										const uniform int InNum)
{
	foreach(i = 0 ... InNum)
	{
		OutBuffer[i] = (InBuffer1[i] * InGain1) + InBuffer2[i];
	}
}

export void ArrayFade(uniform float OutFloatBuffer[],
							const uniform int NumSamples,
							const uniform float StartValue,
							const uniform float EndValue)
{
	const uniform float DeltaValue = ((EndValue - StartValue) / NumSamples);

	foreach(i = 0 ... NumSamples)
	{
		const varying float Gain = (i * DeltaValue) + StartValue;
		OutFloatBuffer[i] = OutFloatBuffer[i] * Gain;
	}
}

export void ArrayFade2(const uniform float InBuffer[],
							const uniform int NumSamples,
							const uniform float StartValue,
							const uniform float EndValue,
							uniform float OutBuffer[])
{
	const uniform float DeltaValue = ((EndValue - StartValue) / NumSamples);

	foreach(i = 0 ... NumSamples)
	{
		const varying float Gain = (i * DeltaValue) + StartValue;
		OutBuffer[i] = InBuffer[i] * Gain;
	}
}

export void ArrayMixInWithGain(const uniform float InFloatBuffer[],
								uniform float BufferToSumTo[],
								const uniform int NumSamples,
								const uniform float Gain)
{
	foreach(i = 0 ... NumSamples)
	{
		BufferToSumTo[i] += InFloatBuffer[i] * Gain;
	}
}

export void ArrayMixIn(const uniform float InFloatBuffer[],
							uniform float BufferToSumTo[],
							const uniform int NumSamples)
{
	foreach(i = 0 ... NumSamples)
	{
		BufferToSumTo[i] += InFloatBuffer[i];
	}
}

export void ArrayMixInWithDelta(const uniform float InFloatBuffer[],
									uniform float BufferToSumTo[],
									const uniform int NumSamples,
									const uniform float StartGain,
									const uniform float EndGain)
{
	const uniform float DeltaValue = ((EndGain - StartGain) / NumSamples);

	foreach(i = 0 ... NumSamples)
	{
		const varying float Gain = (i * DeltaValue) + StartGain;
		BufferToSumTo[i] += InFloatBuffer[i] * Gain;
	}
}

export void TransformArrayRow(const uniform float OffsetInData[],
								const uniform float *uniform RowValuePtr,
								uniform float OutArray[],
								const uniform int RowIndex,
								const uniform int NumToMult)
{
	varying float RowSum = 0.0f;

	foreach(i = 0 ... NumToMult)
	{
		RowSum += OffsetInData[i] * RowValuePtr[i];
	}

	OutArray[RowIndex] += reduce_add(RowSum);
}

export void ArrayFloatToPcm16(const uniform float InBuffer[],
	uniform int16 OutBuffer[],
	const uniform int Num)
{
	const float MaxSignedShortFloat = (float)0x7FFF;

	foreach(i = 0 ... Num)
	{
		OutBuffer[i] = (int16)(InBuffer[i] * MaxSignedShortFloat);
	}
}

export void ArrayPcm16ToFloat(const uniform int16 InBuffer[],
	uniform float OutBuffer[],
	const uniform int Num)
{
	const float InvMaxSignedShortFloat = 1.f / ((float)0x7FFF);

	foreach(i = 0 ... Num)
	{
		OutBuffer[i] = ((float)InBuffer[i]) * InvMaxSignedShortFloat;
	}
}

export void ArrayAPFLongDelayProcess(const uniform float InSamples[],
	const uniform float InDelaySamples[],
	const uniform int InNum,
	uniform float OutSamples[],
	uniform float OutDelaySamples[],
	const uniform float Gain)
{
	foreach(i = 0 ... InNum)
	{
		OutDelaySamples[i] = (InDelaySamples[i] * Gain) + InSamples[i];
		OutSamples[i] = (OutDelaySamples[i] * (-Gain)) + InDelaySamples[i];
	}
}

export void ArrayLerpFractionalDelay(const uniform float InSamples[],
	const uniform float InDelays[],
	const uniform float DelayData[],
	const uniform int IntegerDelays[],
	const uniform int InNum,
	uniform float OutSamples[],
	const uniform float MaxDelay)
{
	foreach(i = 0 ... InNum)
	{
		float VFractionalDelays = InDelays[i];
		VFractionalDelays = max(VFractionalDelays, 0.f);
		VFractionalDelays = min(VFractionalDelays, MaxDelay);

		float VFloorDelays = floor(VFractionalDelays);

		float VUpperCoefficients = VFractionalDelays - VFloorDelays;
		float VLowerCoefficients = 1.f - VUpperCoefficients;

		int VIntegerDelays = (int)VFloorDelays;
		const int VIntegerDelayOffset = IntegerDelays[i];
		VIntegerDelays = VIntegerDelayOffset - VIntegerDelays;

		#pragma ignore warning(perf)
		float VLowerSamples = DelayData[VIntegerDelays + 1];
		#pragma ignore warning(perf)
		float VUpperSamples = DelayData[VIntegerDelays];

		OutSamples[i] = (VLowerSamples * VLowerCoefficients) + (VUpperSamples * VUpperCoefficients);
	}
}

export void ArrayScaledComplexConjugate(const uniform float InValues[],
	const uniform int Num,
	uniform float OutValues[],
	const uniform float Scale)
{
	float SignFlip = -Scale;

#if TARGET_WIDTH == 4
	bool Mask = { true, false, true, false };
#elif TARGET_WIDTH == 8
	bool Mask = { true, false, true, false, true, false, true, false };
#elif TARGET_WIDTH == 16
	bool Mask = { true, false, true, false, true, false, true, false, true, false, true, false, true, false, true, false };
#endif

	SignFlip = select(Mask, Scale, SignFlip);

	foreach(i = 0 ... Num)
	{
		OutValues[i] = InValues[i] * SignFlip;
	}
}
