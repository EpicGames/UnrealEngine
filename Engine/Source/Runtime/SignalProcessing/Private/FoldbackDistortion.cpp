// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/FoldbackDistortion.h"
#include "DSP/Dsp.h"

namespace Audio
{
	FFoldbackDistortion::FFoldbackDistortion()
		: Threshold(0.5f)
		, Threshold2(2.0f * Threshold)
		, Threshold4(4.0f * Threshold)
		, OutputGain(1.0f)
		, NumChannels(0)
	{
	}

	FFoldbackDistortion::~FFoldbackDistortion()
	{
	}

	void FFoldbackDistortion::Init(const float InSampleRate, const int32 InNumChannels)
	{
		NumChannels = InNumChannels;
	}

	void FFoldbackDistortion::SetThresholdDb(const float InThresholdDb)
	{
		Threshold = Audio::ConvertToLinear(InThresholdDb);
		Threshold2 = 2.0f * Threshold;
		Threshold4 = 4.0f * Threshold;
	}

	void FFoldbackDistortion::SetInputGainDb(const float InInputGainDb)
	{
		InputGain = Audio::ConvertToLinear(InInputGainDb);
	}

	void FFoldbackDistortion::SetOutputGainDb(const float InOutputGainDb)
	{
		OutputGain = Audio::ConvertToLinear(InOutputGainDb);
	}

	float FFoldbackDistortion::ProcessAudioSample(const float InSample)
	{
		const float Sample = InputGain * InSample;
		float OutSample = 0.0f;

		if (Sample > Threshold || Sample < -Threshold)
		{
			OutSample = FMath::Abs(FMath::Abs(FMath::Fmod(Sample - Threshold, Threshold4)) - Threshold2) - Threshold;
		}
		else
		{
			OutSample = Sample;
		}

		return OutSample * OutputGain;
	}

	void FFoldbackDistortion::ProcessAudioFrame(const float* InFrame, float* OutFrame)
	{
		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			OutFrame[Channel] = ProcessAudioSample(InFrame[Channel]);
		}
	}

	void FFoldbackDistortion::ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer)
	{
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
		{
			ProcessAudioFrame(&InBuffer[SampleIndex], &OutBuffer[SampleIndex]);
		}
	}

}
