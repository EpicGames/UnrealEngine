// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/GrainDelay.h"

namespace Audio
{
	namespace GrainDelay
	{
		// TODO: consider making this a construction pin or cvar
		static constexpr int32 GrainEnvelopeSampleCount = 1024;
		static constexpr float MaxAbsPitchShiftInOctaves = 6.0f;

		FGrainDelay::FGrainDelay(const float InSampleRate, const float InMaxDelaySeconds)
			: SampleRate(InSampleRate)
			, MaxDelaySeconds(FMath::Max(InMaxDelaySeconds, 0.0f))
		{
			// DelayLine must accommodate the delay of the grain and the pitch shifter.
			// The pitch shifter has a maximum delay and requires an extra millisecond 
			// for phase offsets. An additional sample is added to account for floating 
			// point rounding.

			const float RoundingErrorBufferInSeconds = 1.f / InSampleRate;
			const float MaxTapDelayInSeconds = (FTapDelayPitchShifter::MaxDelayLength + 1.f) / 1000.f;
			const float DelayLineSizeInSeconds = MaxDelaySeconds + MaxTapDelayInSeconds + RoundingErrorBufferInSeconds;
			GrainDelayLine.Init(SampleRate, DelayLineSizeInSeconds);

			// Generate the grain data
			GenerateEnvelopeData(GrainEnvelope, GrainEnvelopeSampleCount, GrainEnvelopeType);
			
			InitDynamicsProcessor();
		}

		FGrainDelay::~FGrainDelay()
		{
		}

		void FGrainDelay::Reset()
		{
			GrainDelayLine.Reset();

			InitDynamicsProcessor();

			// Move any active grains into free grains
			FreeGrains.Append(ActiveGrains);
			ActiveGrains.Reset();
		}

		float FGrainDelay::GetGrainDelayClamped(const float InDelay) const
		{
			return FMath::Clamp(InDelay, 0.0f, 1000.0f * MaxDelaySeconds);
		}

		float FGrainDelay::GetGrainDurationClamped(const float InDuration) const
		{
			return FMath::Clamp(InDuration, 0.0f, 1000.0f * MaxDelaySeconds);
		}
	
		float FGrainDelay::GetGrainDelayRatioClamped(const float InGrainDelayRatio) const
		{
			return FMath::Clamp(InGrainDelayRatio, -1.0f, 1.0f);
		}

		float FGrainDelay::GetGrainPitchShiftClamped(const float InPitchShift) const
		{
			return FMath::Clamp(InPitchShift, -12.0f * MaxAbsPitchShiftInOctaves, 12.0f * MaxAbsPitchShiftInOctaves);
		}

		float FGrainDelay::GetGrainPitchShiftFrameRatio(const float InPitchShift) const
		{
			return FMath::Pow(2.0f, InPitchShift / 12.0f);
		}

		void FGrainDelay::SetMaxGrains(const int32 InMaxGrains)
		{
			MaxGrains = FMath::Clamp(InMaxGrains, 1, 100);
		}
		
		void FGrainDelay::SetGrainEnvelope(const Audio::Grain::EEnvelope InGrainEnvelope)
		{
			if (GrainEnvelopeType != InGrainEnvelope)
			{
				GrainEnvelopeType = InGrainEnvelope;
				GenerateEnvelopeData(GrainEnvelope, 1024, GrainEnvelopeType);
			}
		}

		void FGrainDelay::SetFeedbackAmount(float InFeedbackAmount)
		{
			FeedbackAmount = FMath::Clamp(InFeedbackAmount, 0.0f, 0.95f);
		}
					
		void FGrainDelay::SetGrainBasePitchShiftRatio(const float InPitchRatioBase)
		{
			if (!FMath::IsNearlyEqual(InPitchRatioBase, PitchShiftRatioBase))
			{
				PitchShiftRatioBase = InPitchRatioBase;
				for (const int32 ActiveGrainId : ActiveGrains)
				{
					FGrain& Grain = GrainPool[ActiveGrainId];

					// Set the pitch shift ratio of the grain.
					// This function will take into account the initial random pitch offset
					Grain.SetGrainPitchShiftRatio(InPitchRatioBase, SampleRate);
				}
			}
		}
		
		void FGrainDelay::SpawnGrain(const float InDelay, const float InDuration, const float InPitchShiftRatioOffset)
		{
			if (ActiveGrains.Num() >= FMath::Max(MaxGrains, 1))
			{
				// Hit the max grain cap, no grain for you!
				return;
			}

			// Find or create a new grain index
			int32 NewGrainId = INDEX_NONE;
			if (FreeGrains.Num() == 0)
			{
				NewGrainId = GrainPool.Num();
				GrainPool.Add(FGrain());
			}
			else
			{
				NewGrainId = FreeGrains.Pop();
			}

			// Spawn a new grain and setup it's initial state and data
			const float DurationSeconds = 0.001f * InDuration;
			
			FGrain& NewGrain = GrainPool[NewGrainId];
			NewGrain.NumFramesRendered = 0.0f;
			NewGrain.DelayTapPositionMilliseconds = InDelay;
			NewGrain.DurationFrames = FMath::Max(DurationSeconds * SampleRate, 1.0f);
			NewGrain.PitchShiftRatioOffset = InPitchShiftRatioOffset;
			NewGrain.PitchShifter.Init(SampleRate, 0.0f, 1000.0f * DurationSeconds);
			NewGrain.SetGrainPitchShiftRatio(PitchShiftRatioBase, SampleRate);

			// Add to the active grain list
			ActiveGrains.Add(NewGrainId);
		}

		float FGrainDelay::SynthesizeFrame(const Audio::FDelay& InDelayLine)
		{
			float OutFrame = 0.0f;
			
			// Loop through active grains from end so we can clean it up as we go when a grain is done
			for (int32 ActiveGrainIndex = ActiveGrains.Num() - 1; ActiveGrainIndex >= 0; --ActiveGrainIndex)
			{
				// Get the grain data
				const int32 GrainId = ActiveGrains[ActiveGrainIndex];
				FGrain& Grain = GrainPool[GrainId];
				
				// Calculate the grain envelope
				// should have never been added to the pool
				check(!FMath::IsNearlyZero(Grain.DurationFrames));
				const float Fraction = FMath::Min(Grain.NumFramesRendered / Grain.DurationFrames, 1.0f);
				const float GrainVolume = Grain::GetValue(GrainEnvelope, Fraction);

				const float PitchShiftedSample = Grain.PitchShifter.ReadDopplerShiftedTapFromDelay(InDelayLine, Grain.DelayTapPositionMilliseconds);
				OutFrame += GrainVolume * PitchShiftedSample;

				// Note we are tracking real frames rendered not frames consumed from the delay for grain duration 
				Grain.NumFramesRendered += 1.0f;

				// Clean up the grain once it's done
				if (Grain.NumFramesRendered >= Grain.DurationFrames)
				{
					// Pop back on this id so we can reuse it next time a grain is spawned
					FreeGrains.Add(GrainId);

					// Remove at swap allows us to clean up the active grain list while we loop through it
					ActiveGrains.RemoveAtSwap(ActiveGrainIndex, EAllowShrinking::No);
				}
			}

			// Feed audio through the dynamics processor to prevent clipping.
			DynamicsProcessor.ProcessAudio(&OutFrame,1, &OutFrame);

			return OutFrame;
		}

		void FGrainDelay::SynthesizeAudio(const int32 StartFrame, const int32 EndFrame, const float* InAudioBuffer, float* OutAudioBuffer)
		{
			int32 NumFramesProcessed = 0;
			for (int32 FrameIndex = StartFrame; FrameIndex < EndFrame + 1; ++FrameIndex)
			{
				++NumFramesProcessed;
				GrainDelayLine.WriteDelayAndInc(InAudioBuffer[FrameIndex] + FeedbackAmount * LastFrame);
				LastFrame = SynthesizeFrame(GrainDelayLine);
			
				OutAudioBuffer[FrameIndex] = LastFrame;
			}
		}

		void FGrainDelay::InitDynamicsProcessor()
		{
			// Setup the dynamics processor
			DynamicsProcessor.Init(SampleRate, 1);
			DynamicsProcessor.SetLookaheadMsec(3.0f);
			DynamicsProcessor.SetAttackTime(0.0f);
			DynamicsProcessor.SetReleaseTime(10.0f);
			DynamicsProcessor.SetThreshold(-6.0f);
			DynamicsProcessor.SetKneeBandwidth(3.0f);
			DynamicsProcessor.SetInputGain(0.0f);
			DynamicsProcessor.SetOutputGain(0.0f);
			DynamicsProcessor.SetAnalogMode(false);
			DynamicsProcessor.SetPeakMode(EPeakMode::Peak);
			DynamicsProcessor.SetProcessingMode(EDynamicsProcessingMode::Limiter);
		}
	};

	
}
