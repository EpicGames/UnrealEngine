// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Granulator.h"

namespace Audio
{
	FGrainEnvelope::FGrainEnvelope()
		: CurrentType(EGrainEnvelopeType::Count)
	{
	}

	FGrainEnvelope::~FGrainEnvelope()
	{
	}

	void FGrainEnvelope::GenerateEnvelope(const EGrainEnvelopeType EnvelopeType, const int32 NumFrames)
	{
		check(EnvelopeType != EGrainEnvelopeType::Count);
		check(NumFrames > 1);

		if (CurrentType != EnvelopeType)
		{
			CurrentType = EnvelopeType;

			GrainEnvelope.Reset();
			GrainEnvelope.AddUninitialized(NumFrames);

			// used already cast stack variables to avoid constant casting in loops
			const float N = (float)NumFrames;
			const float N_1 = N - 1.0f;
			float n = 0.0f;

			switch (EnvelopeType)
			{
				case EGrainEnvelopeType::Rectangular:
				{
					for (int32 i = 0; i < NumFrames; ++i)
					{
						GrainEnvelope[i] = 1.0f;
					}
				}
				break;

				case EGrainEnvelopeType::Triangle:
				{
					const float A = 0.5f * N_1;
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = 1.0f - FMath::Abs((n - A) / A);
					}
				}
				break;

				case EGrainEnvelopeType::DownwardTriangle:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = 1.0f - n / N_1;
					}
				}
				break;

				case EGrainEnvelopeType::UpwardTriangle:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = n / N_1;
					}
				}
				break;

				case EGrainEnvelopeType::ExponentialDecay:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = FMath::Pow((n - N + 1.0f) / N_1, 4.0f);
					}
				}
				break;

				case EGrainEnvelopeType::ExponentialIncrease:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = FMath::Pow(n / N_1, 4.0f);
					}
				}
				break;

				case EGrainEnvelopeType::Gaussian:
				{
					const float Denom = 0.3f * N_1 / 2.0f;
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = FMath::Exp(-0.5f * FMath::Pow((n - 0.5f * N_1) / Denom, 2.0f));
					}
				}
				break;

				case EGrainEnvelopeType::Hanning:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = 0.5f - 0.5f * FMath::Cos(2.0f * PI * n / N_1);
					}
				}
				break;

				case EGrainEnvelopeType::Lanczos:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						// sinc function sin(x)/x
						float Arg = PI * (2.0f * n / N_1 - 1.0f);
						Arg = FMath::Max(SMALL_NUMBER, Arg);
						GrainEnvelope[i] = FMath::Sin(Arg) / Arg;
					}
				}
				break;

				case EGrainEnvelopeType::Cosine:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = FMath::Sin(n * PI / N_1);
					}
				}
				break;

				case EGrainEnvelopeType::CosineSquared:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						GrainEnvelope[i] = FMath::Sin(n * PI / N_1);
						GrainEnvelope[i] *= GrainEnvelope[i];
					}
				}
				break;

				case EGrainEnvelopeType::Welch:
				{
					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						float Temp = 0.5f * N_1;
						Temp = (n - Temp) / Temp;
						Temp *= Temp;

						GrainEnvelope[i] = 1.0f - Temp;
					}
				}
				break;

				case EGrainEnvelopeType::Blackman:
				{
					const float A_0 = 0.42659f;
					const float A_1 = 0.49656f;
					const float A_2 = 0.076849f;

					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						const float Theta = 2.0f * PI * n / N_1;
						GrainEnvelope[i] = A_0 - A_1 * FMath::Cos(Theta) + A_2 * FMath::Cos(2.0f * Theta);
					}
				}
				break;

				case EGrainEnvelopeType::BlackmanHarris:
				{
					const float A_0 = 0.35875f;
					const float A_1 = 0.48828f;
					const float A_2 = 0.14158f;
					const float A_3 = 0.01168f;

					for (int32 i = 0; i < NumFrames; ++i, n += 1.0f)
					{
						const float Theta = 2.0f * PI * n / N_1;
						GrainEnvelope[i] = A_0 - A_1 * FMath::Cos(Theta) + A_2 * FMath::Cos(2.0f * Theta) - A_3 * FMath::Cos(4.0f * Theta);
					}
				}
				break;
			}
		}
	}

	float FGrainEnvelope::GetValue(const float Fraction) const
	{
		const int32 NumFrames = GrainEnvelope.Num() - 1;
		const float Index = Fraction * (float)NumFrames;
		const int32 PrevIndex = (int32)Index;
		const int32 NextIndex = FMath::Min(NumFrames, PrevIndex + 1);
		const float AlphaIndex = Index - (float)PrevIndex;

		const float* GrainEnvelopePtr = GrainEnvelope.GetData();
		return FMath::Lerp(GrainEnvelopePtr[PrevIndex], GrainEnvelopePtr[NextIndex], AlphaIndex);
	}

	FGrain::FGrain(const int32 InGrainId, FGranularSynth* InParent)
		: GrainId(InGrainId)
		, Parent(InParent)
		, CurrentPitch(0.0f)
		, CurrentFrequency(0.0f)
		, CurrentVolumeScale(0.0f)
		, CurrentPan(0.0f)
		, DurationScale(1.0f)
		, CurrentFrameCount(0)
		, EndFrameCount(0)
	{
		const int32 SampleRate = Parent->SampleRate;

		SpeakerMap.Add(0.5f);
		SpeakerMap.Add(0.5f);

		// Initialize the oscillator
		Osc.Init(SampleRate);

		// Initialize the sample buffer reader to the parent sample rate
		SampleBufferReader.Init(SampleRate);

		// We are not in scrub mode
		SampleBufferReader.SetScrubMode(false);
	}

	FGrain::~FGrain()
	{
	}

	void FGrain::Play(const FGrainData& InGrainData)
	{
		// make sure we've been initialized
		check(Parent);

		GrainData = InGrainData;

		// Setup the oscillator
		if (Parent->Mode == EGranularSynthMode::Synthesis)
		{
			Osc.Reset();
			Osc.SetType(GrainData.OscType);
			Osc.SetFrequency(GrainData.Frequency);
			Osc.Start();
		}

		CurrentVolumeScale = GrainData.Volume;
		CurrentPan = GrainData.Pan;
		CurrentPitch = GrainData.PitchScale;
		CurrentFrequency = GrainData.Frequency;

		// Setup the frame counts
		CurrentFrameCount = 0.0f;
		EndFrameCount = GrainData.DurationSeconds * Parent->SampleRate;

		Audio::GetStereoPan(CurrentPan, SpeakerMap[0], SpeakerMap[1]);

		// Get information about the buffer if there is one
		if (Parent->SampleBuffer.GetData() != nullptr)
		{
			const int16* Buffer = Parent->SampleBuffer.GetData();
			const int32 NumBufferSamples = Parent->SampleBuffer.GetNumSamples();
			const int32 BufferChannels = Parent->SampleBuffer.GetNumChannels();
			const int32 BufferSampleRate = Parent->SampleBuffer.GetSampleRate();

			SampleBufferReader.ClearBuffer();

			SampleBufferReader.SetBuffer(Buffer, NumBufferSamples, BufferChannels, BufferSampleRate);

			// Setup the sample buffer reader
			SampleBufferReader.SetPitch(CurrentPitch);

			// Where to seek the buffer reader to
			SampleBufferReader.SeekTime(GrainData.BufferSeekTime, ESeekType::FromBeginning);
		}

		FrameScratch.Reset();
		FrameScratch.AddZeroed(2);
	}

	void FGrain::SetOscType(const EOsc::Type InType)
	{
		Osc.SetType(InType);
	}

	void FGrain::SetOscFrequency(const float InFrequency)
	{
		Osc.SetFrequency(InFrequency);
	}

	void FGrain::SetOscFrequencyModuation(const float InFrequencyModulation)
	{
		Osc.SetFrequencyMod(InFrequencyModulation);
	}

	void FGrain::SetPitchModulation(const float InPitchModulation)
	{
		SampleBufferReader.SetPitch(GrainData.PitchScale * GetFrequencyMultiplier(InPitchModulation));
	}

	void FGrain::SetVolumeModulation(const float InVolumeModulation)
	{
		CurrentVolumeScale = GrainData.Volume * (1.0f + InVolumeModulation);
	}

	void FGrain::SetPanModulation(const float InPanModulation)
	{
		CurrentPan = GrainData.Pan * (1.0f + InPanModulation);

		if (CurrentPan < -1.0f)
		{
			CurrentPan += 1.0f;
		}

		if (CurrentPan > 1.0f)
		{
			CurrentPan -= 1.0f;
		}

		Audio::GetStereoPan(CurrentPan, SpeakerMap[0], SpeakerMap[1]);
	}

	void FGrain::SetDurationScale(const float InDurationScale)
	{
		DurationScale = FMath::Max(InDurationScale, 0.0f);
	}

	bool FGrain::IsDone() const
	{
		return CurrentFrameCount >= EndFrameCount;
	}

	float FGrain::GetEnvelopeValue()
	{
		if (CurrentFrameCount <= EndFrameCount)
		{
			const float DurationFraction = CurrentFrameCount / EndFrameCount;
			check(DurationFraction <= 1.0f);

			// How quickly do we read through the envelope is the duration scale
			CurrentFrameCount += DurationScale;

			return CurrentVolumeScale * Parent->GrainEnvelope.GetValue(DurationFraction);
		}

		// If we're done, just return 0.0f
		return 0.0f;
	}

	bool FGrain::GenerateFrame(float* OutStereoFrame)
	{
		if (Parent->Mode == EGranularSynthMode::Granulation)
		{
			// Generate stereo output into the scratch buffer independent of if the loaded sample is stereo or not
			SampleBufferReader.Generate(FrameScratch.GetData(), 1, 2, true);
			const float EnvelopeValue = GetEnvelopeValue();

			for (int32 Channel = 0; Channel < 2; ++Channel)
			{
				// Mix in the generated sample into the output buffer
				OutStereoFrame[Channel] += EnvelopeValue * FrameScratch[Channel] * SpeakerMap[Channel];
			}
		}
		else
		{
			// Either in synth mode or no loaded buffer
			const float NextSample = GetEnvelopeValue() * Osc.Generate();

			for (int32 Channel = 0; Channel < 2; ++Channel)
			{
				// Mix in the generated sample into the output buffer
				OutStereoFrame[Channel] += NextSample * SpeakerMap[Channel];
			}
		}

		return CurrentFrameCount > EndFrameCount;
	}

	FGranularSynth::FGranularSynth()
		: SampleRate(0)
		, NumChannels(0)
		, Mode(EGranularSynthMode::Synthesis)
		, GrainOscType(EOsc::NumOscTypes)
		, GrainEnvelopeType(EGrainEnvelopeType::Count)
		, GrainsPerSecond(1.0f)
		, GrainProbability(1.0f)
		, CurrentSpawnFrameCount(0)
		, NextSpawnFrame(0)
		, NoteDurationFrameCount(0)
		, NoteDurationFrameEnd(0)
		, CurrentPlayHeadFrame(0.0f)
		, PlaybackSpeed(1.0f)
		, NumActiveGrains(0)
		, bScrubMode(false)
	{
	}

	FGranularSynth::~FGranularSynth()
	{

	}

	void FGranularSynth::Init(const int32 InSampleRate, const int32 InNumInitialGrains)
	{
		// make sure we're not double-initializing
		check(SampleRate == 0);

		// Init the sample rate and channels. This is set when grains need to play.
		SampleRate = InSampleRate;

		// Set the granular synth to be stereo
		NumChannels = 2;

		Mode = EGranularSynthMode::Granulation;

		GainEnv.Init(SampleRate);

		Amp.Init();
		Amp.SetGain(1.0f);

		DynamicsProcessor.Init(SampleRate, 2);

		DynamicsProcessor.SetLookaheadMsec(3.0f);
		DynamicsProcessor.SetAttackTime(5.0f);
		DynamicsProcessor.SetReleaseTime(100.0f);
		DynamicsProcessor.SetThreshold(-15.0f);
		DynamicsProcessor.SetRatio(5.0f);
		DynamicsProcessor.SetKneeBandwidth(10.0f);
		DynamicsProcessor.SetInputGain(0.0f);
		DynamicsProcessor.SetOutputGain(0.0f);
		DynamicsProcessor.SetChannelLinkMode(EDynamicsProcessorChannelLinkMode::Average);
		DynamicsProcessor.SetAnalogMode(true);
		DynamicsProcessor.SetPeakMode(EPeakMode::Peak);
		DynamicsProcessor.SetProcessingMode(EDynamicsProcessingMode::Compressor);

		// Initialize some parameters
		SetGrainsPerSecond(20.0f);
		SetGrainProbability(1.0f);
		SetGrainEnvelopeType(EGrainEnvelopeType::Gaussian);
		SetGrainOscType(EOsc::Saw);
		SetGrainDuration(0.1f, {-0.01f, 0.01f});
		SetGrainPitch(1.0f, {0.9f, 1.1f});
		SetGrainFrequency(440.0f);
		SetGrainVolume(1.0f, {0.9f, 1.1f});
		SetGrainPan(0.5f, {0-.1f, 0.1f});

		SetAttackTime(100.0f);
		SetDecayTime(20.0f);
		SetSustainGain(1.0f);
		SetReleaseTime(500.0f);

		SeekingPlayheadTimeFrame.Init(SampleRate);
		SeekingPlayheadTimeFrame.SetValue(CurrentPlayHeadFrame);

		// Initialize the free grain list
		for (int32 i = 0; i < InNumInitialGrains; ++i)
		{
			GrainPool.Add(FGrain(i, this));
			FreeGrains.Add(i);
		}
	}

	void FGranularSynth::LoadSampleBuffer(const TSampleBuffer<int16>& InSampleBuffer)
	{
		SampleBuffer = InSampleBuffer;
	}

	void FGranularSynth::NoteOn(const uint32 InMidiNote, const float InVelocity, const float InDurationSec)
	{
		// Start the envelope
		GainEnv.Start();
		Amp.Reset();
		Amp.SetGain(1.0f);
		Amp.SetVelocity(InVelocity);
		Amp.SetGainEnv(1.0f);
		Amp.Update();

		// Cause a trigger right away
		CurrentSpawnFrameCount = NextSpawnFrame;

		if (InDurationSec > 0.0f)
		{
			NoteDurationFrameCount = 0;
			NoteDurationFrameEnd = (int32)(SampleRate * InDurationSec);
		}
		else
		{
			NoteDurationFrameEnd = INDEX_NONE;
		}

		SetGrainFrequency(GetFrequencyFromMidi(InMidiNote));
	}

	void FGranularSynth::NoteOff(const uint32 InMidiNote, const bool bKill)
	{
		if (bKill)
		{
			GainEnv.Kill();
		}
		else
		{
			GainEnv.Stop();
		}
	}

	void FGranularSynth::SetAttackTime(const float InAttackTimeMSec)
	{
		GainEnv.SetAttackTime(InAttackTimeMSec);
	}

	void FGranularSynth::SetDecayTime(const float InDecayTimeSec)
	{
		GainEnv.SetDecayTime(InDecayTimeSec);
	}

	void FGranularSynth::SetReleaseTime(const float InReleaseTimeSec)
	{
		GainEnv.SetReleaseTime(InReleaseTimeSec);
	}

	void FGranularSynth::SetSustainGain(const float InSustainGain)
	{
		GainEnv.SetSustainGain(InSustainGain);
	}

	void FGranularSynth::SeekTime(const float InTimeSec, const float LerpTimeSec, const ESeekType::Type InSeekType)
	{
		if (SampleBuffer.GetData() != nullptr)
		{
			float TargetPlayheadFrame = 0.0f;

			if (InSeekType == ESeekType::FromBeginning)
			{
				TargetPlayheadFrame = InTimeSec * SampleRate;
			}
			else if (InSeekType == ESeekType::FromEnd)
			{
				float NumFrames = (float)SampleBuffer.GetNumFrames();
				check(NumFrames > 0.0f);

				TargetPlayheadFrame = NumFrames - InTimeSec * SampleRate;
			}
			else
			{
				TargetPlayheadFrame = CurrentPlayHeadFrame + InTimeSec * SampleRate;
			}

			if (LerpTimeSec == 0.0f)
			{
				CurrentPlayHeadFrame = GetWrappedPlayheadPosition(TargetPlayheadFrame);
				SeekingPlayheadTimeFrame.SetValue(CurrentPlayHeadFrame);
			}
			else
			{
				// Note: this target playhead frame may be beyond the bounds of the sample buffer.
				// we will wrap as we lerp to the target value. This prevents gigantic lerping on
				// buffer boundaries
				SeekingPlayheadTimeFrame.SetValue(TargetPlayheadFrame, LerpTimeSec);
			}
		}
	}

	void FGranularSynth::SetScrubMode(const bool bInScrubMode)
	{
		bScrubMode = bInScrubMode;
	}

	float FGranularSynth::GetWrappedPlayheadPosition(float PlayheadFrame)
	{
		float NumFrames = (float)SampleBuffer.GetNumFrames();
		check(NumFrames > 0.0f);
		while (PlayheadFrame < 0.0f)
		{
			PlayheadFrame += NumFrames;
		}

		while (PlayheadFrame >= NumFrames)
		{
			PlayheadFrame -= NumFrames;
		}

		return PlayheadFrame;
	}

	void FGranularSynth::SetPlaybackSpeed(const float InPlaybackSpeed)
	{
		PlaybackSpeed = InPlaybackSpeed;
	}

	void FGranularSynth::SpawnGrain()
	{
		// Now grab a grain off the free grain list
		int32 FreeGrainId = INDEX_NONE;
		FGrain* NewActiveGrain = nullptr;
		if (FreeGrains.Num() > 0)
		{
			FreeGrainId = FreeGrains.Pop();
		}
		else
		{
			// make a new grain
			FreeGrainId = GrainPool.Add(FGrain(GrainPool.Num(), this));
		}

		check(FreeGrainId != INDEX_NONE);
		NewActiveGrain = &GrainPool[FreeGrainId];

		// Add this free grain id to the active grain list
		ActiveGrains.Add(FreeGrainId);

		// Prepare the grain struct based on the current grain probability settings
		FGrainData GrainData;

		// Set the grain's buffer seek time to the current playhead position
		GrainData.BufferSeekTime = CurrentPlayHeadFrame / SampleRate;
		GrainData.DurationSeconds = 0.001f * FMath::Max(5.0f, Duration.GetValue());
		GrainData.Frequency = Frequency.GetValue();
		GrainData.PitchScale = Pitch.GetValue();
		GrainData.Pan = Pan.GetValue();
		GrainData.Volume = Volume.GetValue();

		// Play the grain with the grain data
		NewActiveGrain->Play(GrainData);
	}

	void FGranularSynth::SetGrainsPerSecond(const float NumberOfGrainsPerSecond)
	{
		GrainsPerSecond = FMath::Max(NumberOfGrainsPerSecond, 0.0f);

		// If we're setting a postivie grains per second, compute the next spawn frame
		if (GrainsPerSecond > 0.0f)
		{
			// Update the spawn frame based on the grains per second
			NextSpawnFrame = (int32)(SampleRate / GrainsPerSecond);
		}
		else
		{
			NextSpawnFrame = INDEX_NONE;
		}
	}

	void FGranularSynth::SetGrainProbability(const float InGrainProbability)
	{
		GrainProbability = InGrainProbability;
	}

	void FGranularSynth::SetGrainEnvelopeType(const EGrainEnvelopeType InGrainEnvelopeType)
	{
		if (InGrainEnvelopeType != GrainEnvelopeType)
		{
			GrainEnvelopeType = InGrainEnvelopeType;

			// Generate a new grain envelope, 1024 frames
			GrainEnvelope.GenerateEnvelope(GrainEnvelopeType, 1024);
		}
	}

	void FGranularSynth::SetGrainOscType(const EOsc::Type InGrainOscType)
	{
		if (InGrainOscType != GrainOscType)
		{
			GrainOscType = InGrainOscType;

			for (int32 GrainId : ActiveGrains)
			{
				GrainPool[GrainId].SetOscType(InGrainOscType);
			}
		}
	}

	void FGranularSynth::SetGrainVolume(const float BaseVolume, const FVector2D VolumeRange)
	{
		Volume.Base = BaseVolume;
		Volume.Range = VolumeRange;
	}

	void FGranularSynth::SetGrainVolumeModulation(const float InVolumeModulation)
	{
		if (InVolumeModulation != Volume.Modulation)
		{
			Volume.Modulation = InVolumeModulation;

			for (int32 GrainId : ActiveGrains)
			{
				GrainPool[GrainId].SetVolumeModulation(InVolumeModulation);
			}
		}
	}

	void FGranularSynth::SetGrainPitch(const float BasePitch, const FVector2D PitchRange)
	{
		Pitch.Base = BasePitch;
		Pitch.Range = PitchRange;
	}

	void FGranularSynth::SetGrainFrequency(const float InFrequency, const FVector2D InFrequencyRange)
	{
		Frequency.Base = InFrequency;
		Frequency.Range = InFrequencyRange;
	}

	void FGranularSynth::SetGrainFrequencyModulation(const float InFrequencyModulation)
	{
		if (InFrequencyModulation != Frequency.Modulation)
		{
			Frequency.Modulation = InFrequencyModulation;

			for (int32 GrainId : ActiveGrains)
			{
				GrainPool[GrainId].SetOscFrequencyModuation(InFrequencyModulation);
			}
		}
	}

	void FGranularSynth::SetGrainPitchModulation(const float InPitchModulation)
	{
		if (InPitchModulation != Pitch.Modulation)
		{
			Pitch.Modulation = InPitchModulation;

			for (int32 GrainId : ActiveGrains)
			{
				GrainPool[GrainId].SetPitchModulation(InPitchModulation);
			}
		}
	}

	void FGranularSynth::SetGrainPan(const float BasePan, const FVector2D PanRange)
	{
		Pan.Base = BasePan;
		Pan.Range = PanRange;
	}

	void FGranularSynth::SetGrainPanModulation(const float InPanModulation)
	{
		if (InPanModulation != Pan.Modulation)
		{
			Pan.Modulation = InPanModulation;

			for (int32 GrainId : ActiveGrains)
			{
				GrainPool[GrainId].SetPanModulation(InPanModulation);
			}
		}
	}

	void FGranularSynth::SetGrainDuration(const float BaseDurationMsec, const FVector2D DurationRange)
	{
		Duration.Base = BaseDurationMsec;
		Duration.Range = DurationRange;
	}

	void FGranularSynth::SetGrainDurationScale(const float InDurationScale)
	{
		if (InDurationScale != Duration.Modulation)
		{
			Duration.Modulation = InDurationScale;

			for (int32 GrainId : ActiveGrains)
			{
				GrainPool[GrainId].SetDurationScale(InDurationScale);
			}
		}
	}

	int32 FGranularSynth::GetNumActiveGrains() const
	{
		return NumActiveGrains;
	}

	float FGranularSynth::GetCurrentPlayheadTime() const
	{
		return CurrentPlayHeadFrame;
	}

	float FGranularSynth::GetSampleDuration() const
	{
		return SampleBuffer.GetSampleDuration();
	}

	void FGranularSynth::Generate(float* OutAudiobuffer, const int32 NumFrames)
	{
		FMemory::Memzero(OutAudiobuffer, NumFrames * sizeof(float));

		if (SampleBuffer.GetData() == nullptr)
		{
			return;
		}

		// If the gain envelope is done, nothing to generate
		if (GainEnv.IsDone())
		{
			return;
		}

		NumActiveGrains = ActiveGrains.Num();

		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			// Check if we're going to spawn a grain
			// Only try to spawn grains if grains per second is non-zero
			if (GrainsPerSecond > 0.0f && CurrentSpawnFrameCount++ >= NextSpawnFrame)
			{
				// Reset the spawn frame count
				CurrentSpawnFrameCount = 0;

				// Must pass a dice roll to make a new grain
				if (FMath::FRand() < GrainProbability)
				{
					// Spawn a new grain
					SpawnGrain();
				}
			}

			// Loop through active grains and generate next frame
			const int32 SampleIndex = 2 * Frame;
			float* FrameBuffer = &OutAudiobuffer[SampleIndex];

			DeadGrains.Reset();

			// Loop through all active grains to mix into a final output buffer
			for (int32 GrainId : ActiveGrains)
			{
				if (GrainPool[GrainId].GenerateFrame(FrameBuffer))
				{
					DeadGrains.Add(GrainId);
				}
			}

			// Now apply the gain envelope for the note and the overall amp
 			Amp.Update();
 			Amp.ProcessAudio(FrameBuffer[0], FrameBuffer[1], &FrameBuffer[0], &FrameBuffer[1]);

			DynamicsProcessor.ProcessAudio(FrameBuffer, 2, FrameBuffer);

			const float NewEnvelopeValue = GainEnv.Generate();
			
			FrameBuffer[0] *= NewEnvelopeValue;
 			FrameBuffer[1] *= NewEnvelopeValue;

			// Clean up any dead grain
			for (int32 DeadGrainId : DeadGrains)
			{
				ActiveGrains.Remove(DeadGrainId);
				FreeGrains.Add(DeadGrainId);
			}

			if (Mode == EGranularSynthMode::Granulation)
			{
				// If we're lerping to a new seek playhead time frame
				if (!SeekingPlayheadTimeFrame.IsDone())
				{
					const float NewPlayheadFrame = SeekingPlayheadTimeFrame.GetNextValue();

					CurrentPlayHeadFrame = GetWrappedPlayheadPosition(NewPlayheadFrame);
				}
				else
				{
					if (!bScrubMode)
					{
						// We just increment the current playhead frame based on the playback speed
						CurrentPlayHeadFrame += PlaybackSpeed;

						// Now wrap it to the bounds of the sample buffer
						CurrentPlayHeadFrame = GetWrappedPlayheadPosition(CurrentPlayHeadFrame);
					}
				}
			}

			// Check the auto-note length logic
			if (NoteDurationFrameEnd != INDEX_NONE && NoteDurationFrameCount++ >= NoteDurationFrameEnd)
			{
				GainEnv.Stop();
			}
		}
	}
}


