// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Encoders/IAudioEncoder.h"

Audio::IAudioEncoder::IAudioEncoder(uint32 AudioBufferSlack, uint32 DataBufferSlack /*= 4096*/)
	: UncompressedAudioBuffer(AudioBufferSlack)
	, CompressedDataBuffer(DataBufferSlack)
{
}

Audio::IAudioEncoder::~IAudioEncoder()
{
}

bool Audio::IAudioEncoder::PushAudio(const float* InBuffer, int32 NumSamples, bool bEncodeIfPossible /*= true*/)
{
	if (UncompressedAudioBuffer.Remainder() < (uint32) NumSamples)
	{
		ensureAlwaysMsgf(false, TEXT("Not enough space. Please construct IAudioEncoder with a larager value for AudioBufferSlack"));
		return false;
	}
	else
	{
		UncompressedAudioBuffer.Push(InBuffer, NumSamples);
	}

	if (bEncodeIfPossible)
	{
		return EncodeIfPossible();
	}
	else
	{
		return true;
	}
}

int32 Audio::IAudioEncoder::PopData(uint8* OutData, int32 DataSize)
{
	int32 Result = CompressedDataBuffer.Pop(OutData, DataSize);

	// If result is negative, it indicates how many samples short we are
	// of the requested amount.
	return Result;
}

bool Audio::IAudioEncoder::EncodeIfPossible()
{
	while (UncompressedAudioBuffer.Num() > SamplesRequiredPerEncode())
	{
		CurrentAudioBuffer.Reset();
		CurrentAudioBuffer.AddUninitialized(SamplesRequiredPerEncode());
		UncompressedAudioBuffer.Pop(CurrentAudioBuffer.GetData(), CurrentAudioBuffer.Num());

		CurrentCompressedBuffer.Reset();
		if (!EncodeChunk(CurrentAudioBuffer, CurrentCompressedBuffer))
		{
			ensureAlwaysMsgf(false, TEXT("Encode failed!"));
			return false;
		}
		else
		{
			int32 Result = CompressedDataBuffer.Push(CurrentCompressedBuffer.GetData(), CurrentCompressedBuffer.Num());
			ensureAlwaysMsgf(Result == CurrentCompressedBuffer.Num(), TEXT("Compressed Data Buffer full. Please construct IAudioEncoder with a larger value for DataBufferSlack or call PopData more often."));
			break;
		}
	}

	return true;
}

int64 Audio::IAudioEncoder::Finalize()
{
	// Encode all remaining uncompressed audio:
	EncodeIfPossible();

	CurrentCompressedBuffer.Reset();
	EndFile(CurrentCompressedBuffer);

	int32 Remainder = CompressedDataBuffer.Push(CurrentCompressedBuffer.GetData(), CurrentCompressedBuffer.Num());
	if (Remainder < 0)
	{
		ensureAlwaysMsgf(false, TEXT("Insufficient slack for header! Please use a larger value for DataBufferSlack."));
		return Remainder;
	}
	
	return CompressedDataBuffer.Num();
}

void Audio::IAudioEncoder::Init(const FSoundQualityInfo& InQualityInfo)
{
	StartFile(InQualityInfo, CurrentCompressedBuffer);
	int32 Remainder = CompressedDataBuffer.Push(CurrentCompressedBuffer.GetData(), CurrentCompressedBuffer.Num());
	checkf(Remainder > 0, TEXT("Insufficient slack for header! Please use a larger value for DataBufferSlack."));
}

