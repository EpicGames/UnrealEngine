// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/IntegerDelay.h"
#include "DSP/BufferVectorOperations.h"

namespace Audio
{
	FIntegerDelay::FIntegerDelay(int32 InMaxNumDelaySamplesSamples, int32 InNumDelaySamples, int32 InNumInternalBufferSamples)
	:	MaxNumDelaySamples(InMaxNumDelaySamplesSamples)
		, NumDelaySamples(0)
		, NumDelayLineOffsetSamples(0)
		, NumBufferOffsetSamples(0)
		, NumInternalBufferSamples(InNumInternalBufferSamples)
	{

		// Allocate and prepare delay line for maximum delay.
		const int32 SampleCapacity = (2 * MaxNumDelaySamples) + NumInternalBufferSamples;
		const int32 MaxNumInspectSamples = MaxNumDelaySamples + NumInternalBufferSamples;
		const uint32 ByteAlignment = sizeof(float); // No need for byte alignment
		DelayLine = MakeUnique<FAlignedBlockBuffer>(SampleCapacity, MaxNumInspectSamples, ByteAlignment);
		DelayLine->AddZeros(MaxNumDelaySamples);

		// Set current delay.
		SetDelayLengthSamples(InNumDelaySamples);
	}


	// Destructor
	FIntegerDelay::~FIntegerDelay()
	{}

	void FIntegerDelay::SetDelayLengthSamples(int32 InNumDelaySamples)
	{
		checkf(InNumDelaySamples <= MaxNumDelaySamples, TEXT("InNumDelaySamples must be less than or equal to MaxNumDelaySamples"));
		checkf(InNumDelaySamples >= 0, TEXT("InNumDelaySamples must be greater than or equal to 0"));

		if (InNumDelaySamples > MaxNumDelaySamples)
		{
			InNumDelaySamples = MaxNumDelaySamples;
		}
		else if (InNumDelaySamples < 0)
		{
			InNumDelaySamples = 0;
		}

		NumDelaySamples = InNumDelaySamples;
		// Store offset location for reading samples out from delay line.
		// We keep a couple offsets around to ensure we do go over buffer bounds
		// and abide by alignment rules of FAlignedBlockBuffer
		NumDelayLineOffsetSamples = MaxNumDelaySamples - NumDelaySamples;
		NumBufferOffsetSamples = 0;
		while (0 != (NumDelayLineOffsetSamples % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER))
		{
			NumDelayLineOffsetSamples--;
			NumBufferOffsetSamples++;
		}
	}

	// Resets the delay line state, flushes buffer and resets read/write pointers.
	void FIntegerDelay::Reset() 
	{
		DelayLine->ClearSamples();
		DelayLine->AddZeros(MaxNumDelaySamples);
	}

	// Returns the current delay line length (in samples).
	int32 FIntegerDelay::GetNumDelaySamples() const 
	{
		return NumDelaySamples;
	}

	void FIntegerDelay::ProcessAudio(const Audio::FAlignedFloatBuffer& InSamples, Audio::FAlignedFloatBuffer& OutSamples)
	{
		// Prepare output buffer
		const int32 Num = InSamples.Num();
		OutSamples.Reset(Num);
		OutSamples.AddUninitialized(Num);

		ProcessAudio(TArrayView<const float>(InSamples.GetData(), InSamples.Num()), TArrayView<float>(OutSamples.GetData(), OutSamples.Num()));
	}

	void FIntegerDelay::ProcessAudio(TArrayView<const float> InSamples, TArrayView<float> OutSamples)
	{
		check(InSamples.Num() == OutSamples.Num());

		const int32 InNum = InSamples.Num();
		const float* InSampleData = InSamples.GetData();
		float* OutSampleData = OutSamples.GetData();

		// Process audio one block at a time.
		int32 LeftOver = InNum;
		int32 BufferPos = 0;
		while (LeftOver > 0)
		{
			int32 NumToProcess = FMath::Min(LeftOver, NumInternalBufferSamples);
			ProcessAudioBlock(&InSampleData[BufferPos], NumToProcess, &OutSampleData[BufferPos]);
			BufferPos += NumToProcess;
			LeftOver -= NumToProcess;
		}
	}

	void FIntegerDelay::ProcessAudioBlock(const float* InSamples, const int32 InNum, float* OutSamples)
	{
		// Update delay line.	
		DelayLine->AddSamples(InSamples, InNum);

		// Copy delayed version to output
		const float* DelayData = DelayLine->InspectSamples(InNum + NumBufferOffsetSamples, NumDelayLineOffsetSamples);

		FMemory::Memcpy(OutSamples, &DelayData[NumBufferOffsetSamples], InNum * sizeof(float));
		
		// Remove unneeded delay line.
		DelayLine->RemoveSamples(InNum);
	}

	void FIntegerDelay::PeekDelayLine(int32 InNum, Audio::FAlignedFloatBuffer& OutSamples)
	{
		int32 NumToInspect = FMath::Min(DelayLine->GetNumAvailable(), NumDelaySamples);
		NumToInspect = FMath::Min(InNum, NumToInspect);

		const float* DelaySamples = DelayLine->InspectSamples(NumToInspect);
		if (nullptr == DelaySamples)
		{
			OutSamples.Reset(0);
			return;
		}

		OutSamples.Reset(NumToInspect);
		OutSamples.AddUninitialized(NumToInspect);
		FMemory::Memcpy(OutSamples.GetData(), DelaySamples, NumToInspect * sizeof(float));
	}
}
