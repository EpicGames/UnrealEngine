// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/InterpolatedLinearPitchShifter.h"

namespace Audio
{
	FLinearPitchShifter::FLinearPitchShifter(int32 InNumChannels)
	{
		Reset(InNumChannels, 0.f, 64);
	}

	void FLinearPitchShifter::Reset(int32 InNumChannels, float InInitialPitchShiftSemitones, int32 InInterpLengthFrames)
	{
		NumChannels = InNumChannels;
		InterpLengthFrames = InInterpLengthFrames;
		InterpFramesRemaining = 0;
		PitchShiftRatio.SetValue(FMath::Pow(2.0f, InInitialPitchShiftSemitones / 12.0f), 0);

		PreviousFrame.Reset();
		if (NumChannels > 0)
		{
			PreviousFrame.AddZeroed(NumChannels);
		}

		CurrentIndex = 0.f;
	}

	int32 FLinearPitchShifter::ProcessAudio(const TArrayView<float> InputBuffer, Audio::TCircularAudioBuffer<float>& OutputBuffer)
	{
		const int32 NumInputFrames = InputBuffer.Num() / NumChannels;
		int32 OutputFramesRendered = 0; // return value
		const float* InputBufferData = InputBuffer.GetData();
		float* PreviousFrameData = PreviousFrame.GetData();

		if (0 == NumInputFrames)
		{
			return 0;
		}

		// NOTE: CurrentIndex is a float in the range (-1.0f, (float)(NumInputFrames - 1.0f))
		// the fractional portion of CurrentIndex is used to interpolate between Floor(CurrentIndex) and Ceil(CurrentIndex)
		// if CurrentIndex < -1.0f then we missed an interpolation that should have occured in the previous buffer
		check(CurrentIndex >= -1.f);
		check(PreviousFrame.Num() == NumChannels);

		// Handle interpolations between index -1.0f and 0.0f
		// i.e. interpolating between our last buffer and the current buffer
		while (CurrentIndex < 0.0f)
		{
			const float Alpha = CurrentIndex + 1.f;
			for (int32 Chan = 0; Chan < NumChannels; ++Chan)
			{
				const float A = PreviousFrameData[Chan];
				const float B = InputBufferData[Chan];
				OutputBuffer.Push(FMath::Lerp(A, B, Alpha));
			}

			CurrentIndex += GetNextIndexDelta();
			++OutputFramesRendered;
		}


		// Early Exit: copy full input buffer if no work needs to be done
		// (i.e., not interpolating and pitch shift ratio is 1.0f)
		if (!InterpFramesRemaining && FMath::IsNearlyEqual(1.0f, PitchShiftRatio.GetTarget()))
		{
			OutputBuffer.Push(InputBufferData, NumInputFrames * NumChannels);
			return (NumInputFrames + OutputFramesRendered);
		}

		// Normal case: linear interpolation across the input buffer
		const int32 LastFrameIndex = NumInputFrames - 1;
		int32 LowerFrameIndex = FMath::FloorToInt(CurrentIndex);
		check(LowerFrameIndex >= 0);
		int32 UpperFrameIndex = LowerFrameIndex + 1;

		while (UpperFrameIndex < NumInputFrames)
		{
			int32 LowerSampleIndex = NumChannels * LowerFrameIndex;
			int32 UpperSampleIndex = NumChannels * UpperFrameIndex;
			float Alpha = CurrentIndex - (float)LowerFrameIndex;
			for (int32 Chan = 0; Chan < NumChannels; ++Chan)
			{
				OutputBuffer.Push(FMath::Lerp(InputBufferData[LowerSampleIndex], InputBufferData[UpperSampleIndex], Alpha));
				LowerSampleIndex++;
				UpperSampleIndex++;
			}

			++OutputFramesRendered;
			CurrentIndex += GetNextIndexDelta();
			LowerFrameIndex = FMath::FloorToInt(CurrentIndex);
			UpperFrameIndex = LowerFrameIndex + 1;
		}

		// wrap our fractional index by the buffer size
		CurrentIndex -= (float)(NumInputFrames);

		// We may need to interpolate between the final frame of this buffer and 
		// the 0th frame of the next buffer. Cache the final frame to use on the next call
		const int32 FinalSampleIndex = LastFrameIndex * NumChannels;
		
		for (int32 Chan = 0; Chan < NumChannels; ++Chan)
		{
			PreviousFrameData[Chan] = InputBufferData[FinalSampleIndex + Chan];
		}

		return OutputFramesRendered;
	}

	void FLinearPitchShifter::UpdatePitchShift(float InNewPitchSemitones)
	{
		InterpFramesRemaining = InterpLengthFrames;
		PitchShiftRatio.SetValue(FMath::Pow(2.0f, InNewPitchSemitones / 12.0f), InterpLengthFrames);

		if (FMath::IsNearlyEqual(PitchShiftRatio.GetValue(), PitchShiftRatio.GetTarget()))
		{
			InterpFramesRemaining = 0; // already at target
		}
	}

	float FLinearPitchShifter::GetNextIndexDelta()
	{
		if (InterpFramesRemaining)
		{
			--InterpFramesRemaining;
			return PitchShiftRatio.Update();
		}

		return PitchShiftRatio.GetTarget();
	}

} // namespace Audio
