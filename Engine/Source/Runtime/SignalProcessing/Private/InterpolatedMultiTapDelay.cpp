// Copyright Epic Games, Inc. All Rights Reserved.


#include "DSP/InterpolatedMultiTapDelay.h"

#include "SignalProcessingModule.h"
#include "DSP/FloatArrayMath.h"
#include "Math/VectorRegister.h"

namespace Audio
{
	void FInterpolatedMultiTapDelay::Init(const int32 InDelayBufferSamples)
	{
		WriteIndex = 0;
		DelayLine.Reset();
		DelayLine.AddZeroed(InDelayBufferSamples);
		WrapBuffer.Reset();
		WrapBuffer.AddZeroed(InDelayBufferSamples);
	}

	void FInterpolatedMultiTapDelay::Advance(TArrayView<const float> InBuffer)
	{
		check (InBuffer.Num() % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER == 0);
		const uint32 InNumSamples = InBuffer.Num();
		const uint32 DelayBufferNumSamples = DelayLine.Num();

		if (InNumSamples <= 0)
		{
			return;
		}

		if (InNumSamples + WriteIndex > DelayBufferNumSamples)
		{
			const int32 FirstHalfSize = DelayBufferNumSamples - WriteIndex;
			const int32 SecondHalfSize = InNumSamples - FirstHalfSize;

			if (FirstHalfSize > 0)
			{
				FMemory::Memcpy(&DelayLine[WriteIndex], InBuffer.GetData(), FirstHalfSize * sizeof(float));
			}
			if (SecondHalfSize > 0)
			{
				FMemory::Memcpy(DelayLine.GetData(), &InBuffer[FirstHalfSize], SecondHalfSize * sizeof(float));
			}
			WriteIndex = SecondHalfSize;
		}
		else
		{
			FMemory::Memcpy(&DelayLine[WriteIndex], InBuffer.GetData(), InNumSamples * sizeof(float));
			WriteIndex += InNumSamples;
			WriteIndex = FMath::WrapExclusive((uint32)WriteIndex, (uint32)0, (uint32)DelayLine.Num());
		}
	}

	uint32 FInterpolatedMultiTapDelay::Read(const uint32 StartNumDelaySamples, const uint32 StartSampleFraction, const uint32 EndNumDelaySamples, TArrayView<float> OutBuffer)
	{
		const int32 OutputNumSamples = OutBuffer.Num();
		int32 DelayBufferNumSamples = DelayLine.Num();

		// likely to only run on the first frame, if not configured with enough memory
		if (OutputNumSamples > DelayBufferNumSamples)
		{
			DelayBufferNumSamples = 2 * OutputNumSamples + 1;
			DelayLine.SetNumZeroed(DelayBufferNumSamples);

			UE_LOGF(LogSignalProcessing, Warning, "FInterpolatedMultiTapDelay not configured with enough memory to process an output buffer - allocating extra space.");
		}
		
		if (DelayBufferNumSamples <= 0 || OutputNumSamples <= 0)
		{
			return 0;
		}

		int32 StartSample = WriteIndex - OutputNumSamples - FMath::Clamp(StartNumDelaySamples, 0, DelayBufferNumSamples - 1);
		int32 EndSample = WriteIndex - FMath::Clamp(EndNumDelaySamples, 0, DelayBufferNumSamples - 1);

		const float SampleStride = FMath::Clamp((float)(EndSample - StartSample) / (float)OutputNumSamples, 0.25f, 4.f);
		const uint32 FixedSampleRate = (uint32)(SampleStride * 65536.f);

		StartSample = FMath::WrapExclusive(StartSample, 0, DelayBufferNumSamples);
		
		Resampler.CurrentFrameFraction = StartSampleFraction;
		const int32 FramesNeeded = (int32)Resampler.SourceFramesNeeded(OutputNumSamples, FixedSampleRate);

		float* SourceBuffer = &DelayLine[StartSample];
		if (StartSample + FramesNeeded >= DelayBufferNumSamples)
		{
			const int32 NumSamplesToEnd = DelayBufferNumSamples - StartSample;
			const int32 SecondBufferNumSamples = FramesNeeded - NumSamplesToEnd;
			if (FramesNeeded > WrapBuffer.Num())
			{
				WrapBuffer.SetNumUninitialized(FramesNeeded);
			}
			
			FMemory::Memcpy(WrapBuffer.GetData(), &DelayLine[StartSample], NumSamplesToEnd * sizeof(float));
			// if used sensibly this shouldn't happen, but better to inject 0's than to over-read the buffer
			if (SecondBufferNumSamples > StartSample)
			{
				FMemory::Memcpy(&WrapBuffer[NumSamplesToEnd], DelayLine.GetData(), StartSample * sizeof(float));
				
				const int32 FramesRemaining = SecondBufferNumSamples - StartSample;
				FMemory::Memzero(&WrapBuffer[NumSamplesToEnd + SecondBufferNumSamples], FramesRemaining * sizeof(float));
			}
			else
			{
				FMemory::Memcpy(&WrapBuffer[NumSamplesToEnd], DelayLine.GetData(), SecondBufferNumSamples * sizeof(float));
			}
			SourceBuffer = WrapBuffer.GetData();
		}

		Resampler.ResampleMono(OutputNumSamples, FixedSampleRate, SourceBuffer, OutBuffer.GetData());

		return Resampler.CurrentFrameFraction;
	}

	void FInterpolatedMultiTapDelay::Reset()
	{
		const int32 NumElements = DelayLine.Num();
		DelayLine.Reset(NumElements);
		DelayLine.AddZeroed(NumElements);
	}

	bool FInterpolatedMultiTapDelay::IsInitialized() const
	{
		return DelayLine.GetAllocatedSize() > 0;
	}
}