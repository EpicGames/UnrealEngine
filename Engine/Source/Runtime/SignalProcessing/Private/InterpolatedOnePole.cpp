// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/InterpolatedOnePole.h"

#include "AudioDefines.h"
#include <complex>

namespace Audio
{
	static constexpr int32 Simd4Mask = 0xFFFFFFFC;

	// INTERPOLATED ONE-POLE LOW-PASS IMPLEMENTATION
	FInterpolatedLPF::FInterpolatedLPF()
	{
		Reset();
	}

	void FInterpolatedLPF::Init(float InSampleRate, int32 InNumChannels)
	{
		checkf(InNumChannels > 0, TEXT("FInterpolatedLPF::Init called with InNumChannels <= 0"));
		SampleRate = InSampleRate;
		NumChannels = InNumChannels;
		CutoffFrequency = -1.0f;
		Z1.Init(0.0f, NumChannels);
		Reset();
	}

	void FInterpolatedLPF::StartFrequencyInterpolation(const float InTargetFrequency, const int32 InInterpLength)
	{
		CurrInterpLength = InInterpLength;

		if (bIsFirstFrequencyChange)
		{
			CurrInterpLength = 0;
			bIsFirstFrequencyChange = false;
		}

		if (!FMath::IsNearlyEqual(InTargetFrequency, CutoffFrequency))
		{
			CutoffFrequency = InTargetFrequency;

			const float NormalizedFreq = FMath::Clamp(2.0f * GetCutoffFrequency() / SampleRate, 0.0f, 1.0f);
			B1Target = FMath::Exp(-PI * NormalizedFreq);
			B1Delta = (CurrInterpLength > 1) ? (B1Target - B1Curr) / static_cast<float>(CurrInterpLength) : 0.0f;
		}

		if (CurrInterpLength <= 1)
		{
			StopFrequencyInterpolation();
		}
	}

	void FInterpolatedLPF::ProcessAudioFrame(const float* RESTRICT InputFrame, float* RESTRICT OutputFrame)
	{
		float* Z1Data = Z1.GetData();
		B1Curr += B1Delta; // step forward coefficient

		/*
			[absorbing A0 coefficient]
			-----------------------------%
			Yn = Xn*A0 + B1*Z1;                <- old way
			A0 = (1-B1)

			Yn = Xn*(1-B1) + B1*Z1             <- (1 add, 1 sub, 2 mult)
			Yn = Xn - B1*Xn + B1*Z1
			Yn = Xn + B1*Z1 - B1*Xn
			Yn = Xn + B1*(Z1 - Xn)             <- (1 add, 1 sub, 1 mult)
		*/

		for (int32 i = 0; i < NumChannels; ++i)
		{
			const float InputSample = InputFrame[i];
			const float Yn = InputSample + B1Curr * (Z1Data[i] - InputSample); // LPF
			Z1Data[i] = Yn;
			OutputFrame[i] = Yn;
		}
	}

	void FInterpolatedLPF::ProcessAudioBuffer(const float *RESTRICT InputBuffer, float *RESTRICT OutputBuffer, const int32 NumSamples)
	{
		float* Z1Data = Z1.GetData();
		if (NumChannels == 1)
		{
			// keep these in registers instead of having a load\store on every loop iteration
			float B1 = B1Curr;
			float Z1D = Z1Data[0];
			float Delta = B1Delta;
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				// step forward coefficient
				B1 += Delta;

				const float InputSample = InputBuffer[SampleIndex];
				float Yn = InputSample + B1 * (Z1D - InputSample); // LPF
				Z1D = Yn;
				OutputBuffer[SampleIndex] = Yn;
			}

			Z1Data[0] = Z1D;
			B1Curr = B1;

			return;
		}

		int32 SampleIndex = 0;
		const int32 NumToSimd = NumSamples & Simd4Mask;
		if (NumChannels == 2)
		{
			VectorRegister4Float B1CurrVec = VectorLoadFloat1(&B1Curr);
			VectorRegister4Float B1DeltaVec = MakeVectorRegister(B1Delta + B1Delta, B1Delta + B1Delta, B1Delta + B1Delta, B1Delta + B1Delta);
			VectorRegister4Float Z1DataVec = MakeVectorRegister(Z1Data[0], Z1Data[1], Z1Data[0], Z1Data[1]);
			
			if (NumToSimd)
			{
				B1CurrVec = VectorAdd(B1CurrVec, MakeVectorRegister(B1Delta, B1Delta, B1Delta + B1Delta, B1Delta + B1Delta));	//b1 + delta, b1 + delta, b1 + 2x delta, b1 + 2x delta
			}

			for (; SampleIndex < NumToSimd; SampleIndex += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				VectorRegister4Float InputSample = VectorLoad(&InputBuffer[SampleIndex]);
				VectorRegister4Float Yn = VectorMultiplyAdd(B1CurrVec, VectorSubtract(Z1DataVec, InputSample), InputSample);	//Y0, Y1, garbage, garbage

				Z1DataVec = VectorShuffle(Z1DataVec, Yn, 0, 1, 0, 1);	//Z0, Z1, Y0, Y1
				Yn = VectorMultiplyAdd(B1CurrVec, VectorSubtract(Z1DataVec, InputSample), InputSample);	//Y0, Y1, Y2, Y3

				VectorStore(Yn, &OutputBuffer[SampleIndex]);
				Z1DataVec = VectorSwizzle(Yn, 2, 3, 2, 3);	//Y2, Y3, Y2, Y3

				B1CurrVec = VectorAdd(B1CurrVec, B1DeltaVec);	//b1 + 2x delta, b1 + 2x delta, b1 + 2x delta, b1 + 2x delta
			}

			float Store[4];
			VectorStore(B1CurrVec, Store);
			B1Curr = Store[0];
			VectorStore(Z1DataVec, Store);
			Z1Data[0] = Store[0];
			Z1Data[1] = Store[1];
		}
		else if (NumChannels == 4)
		{
			VectorRegister4Float B1CurrVec = VectorLoadFloat1(&B1Curr);
			VectorRegister4Float B1DeltaVec = VectorLoadFloat1(&B1Delta);
			VectorRegister4Float Z1DataVec = VectorLoad(&Z1Data[0]);

			for (; SampleIndex < NumToSimd; SampleIndex += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				B1CurrVec = VectorAdd(B1CurrVec, B1DeltaVec);

				VectorRegister4Float InputSample = VectorLoad(&InputBuffer[SampleIndex]);
				VectorRegister4Float Yn = VectorMultiplyAdd(B1CurrVec, VectorSubtract(Z1DataVec, InputSample), InputSample);

				VectorStore(Yn, &OutputBuffer[SampleIndex]);
				Z1DataVec = Yn;
			}

			float Store[4];
			VectorStore(B1CurrVec, Store);
			B1Curr = Store[0];
			VectorStore(Z1DataVec, &Z1Data[0]);
		}

		for (; SampleIndex < NumSamples; ++SampleIndex)
		{
			// cache which delay term we should be using
			const int32 ChannelIndex = SampleIndex % NumChannels;

			// step forward coefficient
			// Multiply delta by !ChannelIndex so the coefficient only accumulates at the beginning of each frame (on channel 0)
			B1Curr += B1Delta * !ChannelIndex;

			const float InputSample = InputBuffer[SampleIndex];
			const float Yn = InputSample + B1Curr * (Z1Data[ChannelIndex] - InputSample); // LPF
			Z1Data[ChannelIndex] = Yn;
			OutputBuffer[SampleIndex] = Yn;
		}
	}

	void FInterpolatedLPF::ProcessBufferInPlace(float* InOutBuffer, const int32 NumSamples)
	{
		float* Z1Data = Z1.GetData();
		if (NumChannels == 1)
		{
			// keep these in registers instead of having a load\store on every loop iteration
			float B1 = B1Curr;
			float Z1D = Z1Data[0];
			float Delta = B1Delta;
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				// step forward coefficient
				B1 += Delta;

				const float InputSample = InOutBuffer[SampleIndex];
				float Yn = InputSample + B1 * (Z1D - InputSample); // LPF
				Z1D = Yn;
				InOutBuffer[SampleIndex] = Yn;
			}

			Z1Data[0] = Z1D;
			B1Curr = B1;

			return;
		}

		int32 SampleIndex = 0;
		const int32 NumToSimd = NumSamples & Simd4Mask;
		if (NumChannels == 2)
		{
			VectorRegister4Float B1CurrVec = VectorLoadFloat1(&B1Curr);
			VectorRegister4Float B1DeltaVec = MakeVectorRegister(B1Delta + B1Delta, B1Delta + B1Delta, B1Delta + B1Delta, B1Delta + B1Delta);
			VectorRegister4Float Z1DataVec = MakeVectorRegister(Z1Data[0], Z1Data[1], Z1Data[0], Z1Data[1]);

			if (NumToSimd)
			{
				B1CurrVec = VectorAdd(B1CurrVec, MakeVectorRegister(B1Delta, B1Delta, B1Delta + B1Delta, B1Delta + B1Delta));	//b1 + delta, b1 + delta, b1 + 2x delta, b1 + 2x delta
			}

			for (; SampleIndex < NumToSimd; SampleIndex += 4)
			{
				VectorRegister4Float InputSample = VectorLoad(&InOutBuffer[SampleIndex]);
				VectorRegister4Float Yn = VectorMultiplyAdd(B1CurrVec, VectorSubtract(Z1DataVec, InputSample), InputSample);	//Y0, Y1, garbage, garbage

				Z1DataVec = VectorShuffle(Z1DataVec, Yn, 0, 1, 0, 1);	//Z0, Z1, Y0, Y1
				Yn = VectorMultiplyAdd(B1CurrVec, VectorSubtract(Z1DataVec, InputSample), InputSample);	//Y0, Y1, Y2, Y3

				VectorStore(Yn, &InOutBuffer[SampleIndex]);
				Z1DataVec = VectorSwizzle(Yn, 2, 3, 2, 3);	//Y2, Y3, Y2, Y3

				B1CurrVec = VectorAdd(B1CurrVec, B1DeltaVec);	//b1 + 2x delta, b1 + 2x delta, b1 + 2x delta, b1 + 2x delta
			}

			float Store[4];
			VectorStore(B1CurrVec, Store);
			B1Curr = Store[0];
			VectorStore(Z1DataVec, Store);
			Z1Data[0] = Store[0];
			Z1Data[1] = Store[1];
		}
		else if (NumChannels == 4)
		{
			VectorRegister4Float B1CurrVec = VectorLoadFloat1(&B1Curr);
			VectorRegister4Float B1DeltaVec = VectorLoadFloat1(&B1Delta);
			VectorRegister4Float Z1DataVec = VectorLoad(&Z1Data[0]);

			for (; SampleIndex < NumToSimd; SampleIndex += 4)
			{
				B1CurrVec = VectorAdd(B1CurrVec, B1DeltaVec);

				VectorRegister4Float InputSample = VectorLoad(&InOutBuffer[SampleIndex]);
				VectorRegister4Float Yn = VectorMultiplyAdd(B1CurrVec, VectorSubtract(Z1DataVec, InputSample), InputSample);

				VectorStore(Yn, &InOutBuffer[SampleIndex]);
				Z1DataVec = Yn;
			}

			float Store[4];
			VectorStore(B1CurrVec, Store);
			B1Curr = Store[0];
			VectorStore(Z1DataVec, &Z1Data[0]);
		}

		for (; SampleIndex < NumSamples; ++SampleIndex)
		{
			// cache which delay term we should be using
			const int32 ChannelIndex = SampleIndex % NumChannels;

			// step forward coefficient
			// Multiply delta by !ChannelIndex so the coefficient only accumulates at the beginning of each frame (on channel 0)
			B1Curr += B1Delta * !ChannelIndex;

			const float InputSample = InOutBuffer[SampleIndex];
			const float Yn = InputSample + B1Curr * (Z1Data[ChannelIndex] - InputSample); // LPF
			Z1Data[ChannelIndex] = Yn;
			InOutBuffer[SampleIndex] = Yn;
		}
	}

	void FInterpolatedLPF::Reset()
	{
		B1Curr = 0.0f;
		B1Delta = 0.0f;
		B1Target = B1Curr;
		CurrInterpLength = 0;
		ClearMemory();
		bIsFirstFrequencyChange = true;
	}

	void FInterpolatedLPF::ClearMemory()
	{
		Z1.Reset();
		Z1.AddZeroed(NumChannels);
	}

	void FInterpolatedLPF::ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		const float B1 = B1Curr;
		const float A0 = (1.0f - B1Curr);

		const int32 NumFloats = InOutComplexValues.Num();
		check(NumFloats % 2 == 0);
		for (int32 Index = 0; Index < NumFloats; Index += 2)
		{
			const float ZReal = InOutComplexValues[Index + 0];
			const float ZImag = InOutComplexValues[Index + 1];
			const std::complex<float> Z(ZReal, ZImag);

			const std::complex<float> LPF = A0 / (1.0f - B1 * (1.0f / Z));

			InOutComplexValues[Index + 0] = LPF.real();
			InOutComplexValues[Index + 1] = LPF.imag();
		}
	}


	// INTERPOLATED ONE-POLE HIGH-PASS IMPLEMENTATION
	FInterpolatedHPF::FInterpolatedHPF()
	{
		Reset();
	}

	void FInterpolatedHPF::Init(float InSampleRate, int32 InNumChannels)
	{
		checkf(InNumChannels > 0, TEXT("FInterpolatedHPF::Init called with InNumChannels <= 0"));
		SampleRate = InSampleRate;
		NyquistLimit = 0.5f * SampleRate - 1.0f;
		NumChannels = InNumChannels;
		CutoffFrequency = -1.0f;
		Z1.Init(0.0f, NumChannels);
		Reset();
	}

	void FInterpolatedHPF::StartFrequencyInterpolation(const float InTargetFrequency, const int32 InterpLength)
	{
		CurrInterpLength = InterpLength;

		if (bIsFirstFrequencyChange)
		{
			CurrInterpLength = 0;
			bIsFirstFrequencyChange = false;
		}

		if (!FMath::IsNearlyEqual(InTargetFrequency, CutoffFrequency))
		{
			CutoffFrequency = FMath::Clamp(InTargetFrequency, 0.f, NyquistLimit);

			// G computation is a reduced form of the following set of equations:
			// OmegaDigital = 2.0f * PI * CutoffFrequency;
			// OmegaAnalog = 2.0f * SampleRate * Audio::FastTan(0.5f * OmegaDigital / SampleRate);
			// G = 0.5f * OmegaAnalog / SampleRate;
			const float G = Audio::FastTan(PI * GetCutoffFrequency() / SampleRate);

			A0Target = G / (1.0f + G);
			A0Delta = (CurrInterpLength > 1) ? (A0Target - A0Curr) / static_cast<float>(CurrInterpLength) : 0.0f;
		}

		if (CurrInterpLength <= 1)
		{
			StopFrequencyInterpolation();
		}
	}

	void FInterpolatedHPF::ProcessAudioFrame(const float* RESTRICT InputFrame, float* RESTRICT OutputFrame)
	{
		float* Z1Data = Z1.GetData();
		A0Curr += A0Delta; // step forward coefficient

		for (int32 i = 0; i < NumChannels; ++i)
		{
			const float InputSample = InputFrame[i];
			const float Vn = (InputSample - Z1Data[i]) * A0Curr;
			const float LPF = Vn + Z1Data[i];
			Z1Data[i] = Vn + LPF;

			OutputFrame[i] = InputSample - LPF;
		}
	}

	void FInterpolatedHPF::ProcessAudioBuffer(const float *RESTRICT InputBuffer, float *RESTRICT OutputBuffer, const int32 NumSamples)
	{
		float* Z1Data = Z1.GetData();
		if (NumChannels == 1)
		{
			// keep these in registers instead of having a load\store on every loop iteration
			float A0 = A0Curr;
			float Z1D = Z1Data[0];
			float Delta = A0Delta;
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				// step forward coefficient
				A0 += Delta;

				const float InputSample = InputBuffer[SampleIndex];
				const float Vn = (InputSample - Z1D) * A0;
				const float LPF = Vn + Z1D;
				Z1D = Vn + LPF;

				OutputBuffer[SampleIndex] = InputSample - LPF;
			}

			Z1Data[0] = Z1D;
			A0Curr = A0;

			return;
		}

		int32 SampleIndex = 0;
		const int32 NumToSimd = NumSamples & Simd4Mask;
		if (NumChannels == 2)
		{
			VectorRegister4Float A0CurrVec = VectorLoadFloat1(&A0Curr);
			VectorRegister4Float A0DeltaVec = MakeVectorRegister(A0Delta + A0Delta, A0Delta + A0Delta, A0Delta + A0Delta, A0Delta + A0Delta);
			VectorRegister4Float Z1DataVec = MakeVectorRegister(Z1Data[0], Z1Data[1], Z1Data[0], Z1Data[1]);

			if (NumToSimd)
			{
				A0CurrVec = VectorAdd(A0CurrVec, MakeVectorRegister(A0Delta, A0Delta, A0Delta + A0Delta, A0Delta + A0Delta));	//a1 + delta, a1 + delta, a1 + 2x delta, a1 + 2x delta
			}

			for (; SampleIndex < NumToSimd; SampleIndex += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				VectorRegister4Float InputSample = VectorLoad(&InputBuffer[SampleIndex]);
				VectorRegister4Float Vn = VectorMultiply(VectorSubtract(InputSample, Z1DataVec), A0CurrVec);	//Vn1, Vn2, garbage, garbage
				VectorRegister4Float LPF = VectorAdd(Vn, Z1DataVec);	//LPF1, LPF2, garbage, garbage

				VectorRegister4Float VnLPFSum = VectorAdd(Vn, LPF);
				Z1DataVec = VectorShuffle(Z1DataVec, VnLPFSum, 0, 1, 0, 1);
				Vn = VectorMultiply(VectorSubtract(InputSample, Z1DataVec), A0CurrVec);	//Vn1, Vn2, Vn3, Vn4
				LPF = VectorAdd(Vn, Z1DataVec);	//LPF1, LPF2, LPF3, LPF4

				VnLPFSum = VectorAdd(Vn, LPF);
				Z1DataVec = VectorSwizzle(VnLPFSum, 2, 3, 2, 3);	//Vn2, Vn3, Vn2, Vn3

				VectorStore(VectorSubtract(InputSample, LPF), &OutputBuffer[SampleIndex]);

				A0CurrVec = VectorAdd(A0CurrVec, A0DeltaVec);	//a1 + 2x delta, a1 + 2x delta, a1 + 2x delta, a1 + 2x delta
			}

			float Store[4];
			VectorStore(A0CurrVec, Store);
			A0Curr = Store[0];
			VectorStore(Z1DataVec, Store);
			Z1Data[0] = Store[0];
			Z1Data[1] = Store[1];
		}
		else if (NumChannels == 4)
		{
			VectorRegister4Float A0CurrVec = VectorLoadFloat1(&A0Curr);
			VectorRegister4Float A0DeltaVec = VectorLoadFloat1(&A0Delta);
			VectorRegister4Float Z1DataVec = VectorLoad(&Z1Data[0]);

			for (; SampleIndex < NumToSimd; SampleIndex += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				A0CurrVec = VectorAdd(A0CurrVec, A0DeltaVec);

				VectorRegister4Float InputSample = VectorLoad(&InputBuffer[SampleIndex]);
				VectorRegister4Float Vn = VectorMultiply(VectorSubtract(InputSample, Z1DataVec), A0CurrVec);
				VectorRegister4Float LPF = VectorAdd(Vn, Z1DataVec);

				Z1DataVec = VectorAdd(Vn, LPF);

				VectorStore(VectorSubtract(InputSample, LPF), &OutputBuffer[SampleIndex]);
			}

			float Store[4];
			VectorStore(A0CurrVec, Store);
			A0Curr = Store[0];
			VectorStore(Z1DataVec, &Z1Data[0]);
		}

		for (; SampleIndex < NumSamples; ++SampleIndex)
		{
			// cache which delay term we should be using
			const int32 ChannelIndex = SampleIndex % NumChannels;

			// step forward coefficient
			// Multiply delta by !ChannelIndex so the coefficient only accumulates at the beginning of each frame (on channel 0)
			A0Curr += A0Delta * !ChannelIndex;

			const float InputSample = InputBuffer[SampleIndex];
			const float Vn = (InputSample - Z1Data[ChannelIndex]) * A0Curr;
			const float LPF = Vn + Z1Data[ChannelIndex];
			Z1Data[ChannelIndex] = Vn + LPF;

			OutputBuffer[SampleIndex] = InputSample - LPF;
		}
	}

	void FInterpolatedHPF::ProcessBufferInPlace(float* InOutBuffer, const int32 NumSamples)
	{
		float* Z1Data = Z1.GetData();
		if (NumSamples <= 0 || NumChannels <= 0)
		{
			return;
		}

		if (NumChannels == 1)
		{
			float A0 = A0Curr;
			float Z1D = Z1Data[0];
			float Delta = A0Delta;
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				A0 += Delta;

				const float InputSample = InOutBuffer[SampleIndex];
				const float Vn = (InputSample - Z1D) * A0;
				const float LPF = Vn + Z1D;
				Z1D = Vn + LPF;

				InOutBuffer[SampleIndex] = InputSample - LPF;
			}

			Z1Data[0] = Z1D;
			A0Curr = A0;

			return;
		}

		int32 SampleIndex = 0;
		const int32 NumToSimd = NumSamples & Simd4Mask;
		if (NumChannels == 2)
		{
			VectorRegister4Float A0CurrVec = VectorLoadFloat1(&A0Curr);
			VectorRegister4Float A0DeltaVec = MakeVectorRegister(A0Delta + A0Delta, A0Delta + A0Delta, A0Delta + A0Delta, A0Delta + A0Delta);
			VectorRegister4Float Z1DataVec = MakeVectorRegister(Z1Data[0], Z1Data[1], Z1Data[0], Z1Data[1]);

			if (NumToSimd)
			{
				A0CurrVec = VectorAdd(A0CurrVec, MakeVectorRegister(A0Delta, A0Delta, A0Delta + A0Delta, A0Delta + A0Delta));
				}

			for (; SampleIndex < NumToSimd; SampleIndex += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				VectorRegister4Float InputSample = VectorLoad(&InOutBuffer[SampleIndex]);
				VectorRegister4Float Vn = VectorMultiply(VectorSubtract(InputSample, Z1DataVec), A0CurrVec);
				VectorRegister4Float LPF = VectorAdd(Vn, Z1DataVec);

				VectorRegister4Float VnLPFSum = VectorAdd(Vn, LPF);
				Z1DataVec = VectorShuffle(Z1DataVec, VnLPFSum, 0, 1, 0, 1);
				Vn = VectorMultiply(VectorSubtract(InputSample, Z1DataVec), A0CurrVec);
				LPF = VectorAdd(Vn, Z1DataVec);

				VnLPFSum = VectorAdd(Vn, LPF);
				Z1DataVec = VectorSwizzle(VnLPFSum, 2, 3, 2, 3);

				VectorStore(VectorSubtract(InputSample, LPF), &InOutBuffer[SampleIndex]);

				A0CurrVec = VectorAdd(A0CurrVec, A0DeltaVec);
			}

			float Store[4];
			VectorStore(A0CurrVec, Store);
			A0Curr = Store[0];
			VectorStore(Z1DataVec, Store);
			Z1Data[0] = Store[0];
			Z1Data[1] = Store[1];
		}
		else if (NumChannels == 4)
		{
			VectorRegister4Float A0CurrVec = VectorLoadFloat1(&A0Curr);
			VectorRegister4Float A0DeltaVec = VectorLoadFloat1(&A0Delta);
			VectorRegister4Float Z1DataVec = VectorLoad(&Z1Data[0]);

			for (; SampleIndex < NumToSimd; SampleIndex += AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER)
			{
				A0CurrVec = VectorAdd(A0CurrVec, A0DeltaVec);

				VectorRegister4Float InputSample = VectorLoad(&InOutBuffer[SampleIndex]);
				VectorRegister4Float Vn = VectorMultiply(VectorSubtract(InputSample, Z1DataVec), A0CurrVec);
				VectorRegister4Float LPF = VectorAdd(Vn, Z1DataVec);

				Z1DataVec = VectorAdd(Vn, LPF);

				VectorStore(VectorSubtract(InputSample, LPF), &InOutBuffer[SampleIndex]);
			}

			float Store[4];
			VectorStore(A0CurrVec, Store);
			A0Curr = Store[0];
			VectorStore(Z1DataVec, &Z1Data[0]);
		}

		for (; SampleIndex < NumSamples; ++SampleIndex)
		{
			const int32 ChannelIndex = SampleIndex % NumChannels;
			A0Curr += A0Delta * !ChannelIndex;

			const float InputSample = InOutBuffer[SampleIndex];
			const float Vn = (InputSample - Z1Data[ChannelIndex]) * A0Curr;
			const float LPF = Vn + Z1Data[ChannelIndex];
			Z1Data[ChannelIndex] = Vn + LPF;

			InOutBuffer[SampleIndex] = InputSample - LPF;
		}
	}

	void FInterpolatedHPF::Reset()
	{
		A0Curr = 0.0f;
		A0Delta = 0.0f;
		CurrInterpLength = 0;
		ClearMemory();
		bIsFirstFrequencyChange = true;
	}

	void FInterpolatedHPF::ClearMemory()
	{
		Z1.Reset();
		Z1.AddZeroed(NumChannels);
	}

	void FInterpolatedHPF::ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const
	{
		const float A0 = A0Curr;
		const float A1 = A0Curr;
		const float B1 = 2.0f * A0 - 1.0f;

		const int32 NumFloats = InOutComplexValues.Num();
		check(NumFloats % 2 == 0);
		for (int32 Index = 0; Index < NumFloats; Index += 2)
		{
			const float ZReal = InOutComplexValues[Index + 0];
			const float ZImag = InOutComplexValues[Index + 1];
			const std::complex<float> Z(ZReal, ZImag);

			const std::complex<float> LPF = (A0 * Z + A1) / (B1 + Z);
			const std::complex<float> HPF = 1.0f - LPF;

			InOutComplexValues[Index + 0] = HPF.real();
			InOutComplexValues[Index + 1] = HPF.imag();
		}
	}

} // namespace Audio