// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LFO.h"
#include "DSP/Dsp.h"


namespace Audio
{
	FLFO::FLFO()
		: LFOType(ELFO::Sine)
		, LFOMode(ELFOMode::Sync)
		, ExponentialFactor(3.5f)
		, RSHCounter(INDEX_NONE)
		, RSHValue(0.0f)
		, ModScale(1.0f)
		, ModAdd(0.0f)
		, LastOutput(0.0f)
		, LoopCount(0.0f)
		, QuadLastOutput(0.0f)
		, PhaseOffset(0.0f)
		, bBipolar(true)
	{
	}

	FLFO::~FLFO() = default;

	void FLFO::Init(const float InSampleRate, const int32 InVoiceId, FModulationMatrix* InMatrix, const int32 ModMatrixStage)
	{
		IOscBase::Init(InSampleRate, InVoiceId, InMatrix, ModMatrixStage);

		if (ModMatrix)
		{
			ModNormalPhase = ModMatrix->CreatePatchSource(InVoiceId);
			ModQuadPhase = ModMatrix->CreatePatchSource(InVoiceId);

#if MOD_MATRIX_DEBUG_NAMES
			ModNormalPhase.Name = TEXT("ModNormalPhase");
			ModQuadPhase.Name = TEXT("ModQuadPhase");
#endif
		}
	}

	void FLFO::Start()
	{
		if (LFOMode == ELFOMode::Sync || LFOMode == ELFOMode::OneShot)
		{
			Reset();
		}
		else if (!bIsPlaying)
		{
			ResetPhase();
		}

		bIsPlaying = true;
	}

	void FLFO::Stop()
	{
		bIsPlaying = false;
	}

	void FLFO::Reset()
	{
		// reset base class first
		IOscBase::Reset();

		ResetPhase();

		RSHValue = 0.0f;
		RSHCounter = INDEX_NONE;
	}

	void FLFO::ResetPhase()
	{
		// Reset loop count
		LoopCount = 0.0f;

		// Set initial default phase to zero crossing and rising edge where
		// possible (omits "phase offset" input to allow for client system
		// to set to non-zero if desired).
		switch (LFOType)
		{
			case ELFO::Sine:
			case ELFO::Triangle:
			{
				Phase = bBipolar ? 0.25f : 0.0f;
			}
			break;

			case ELFO::DownSaw:
			case ELFO::UpSaw:
			{
				Phase = bBipolar ? 0.5f : 0.0f;
			}
			break;

			case ELFO::Exponential:
			{
				Phase = bBipolar ? FMath::Pow(0.5f, 1.0f / ExponentialFactor) : 0.0f;
			}
			break;

			case ELFO::RandomSampleHold:
			case ELFO::Square:
			default:
			{
				static_assert(static_cast<int32>(ELFO::NumLFOTypes) == 7, "Possible missing switch case coverage");
				Phase = 0.0f;
			}
			break;
		};
	}

	void FLFO::SetBipolar(const bool bInBipolar)
	{
		bBipolar = bInBipolar;
	}

	void FLFO::SetPhaseOffset(const float InOffset)
	{
		PhaseOffset = FMath::Fmod(FMath::Max(0.0f, InOffset), 1.0f);
	}

	void FLFO::SetType(const ELFO::Type InLFOType)
	{
		LFOType = InLFOType;
	}

	ELFO::Type FLFO::GetType() const
	{
		return LFOType;
	}

	void FLFO::SetMode(const ELFOMode::Type InLFOMode)
	{
		LFOMode = InLFOMode;
	}

	ELFOMode::Type FLFO::GetMode() const
	{
		return LFOMode;
	}

	void FLFO::SetExponentialFactor(const float InExpFactor)
	{
		ExponentialFactor = FMath::Max(InExpFactor, UE_SMALL_NUMBER);
	}

	FPatchSource FLFO::GetModSourceNormalPhase() const
	{
		return ModNormalPhase;
	}

	FPatchSource FLFO::GetModSourceQuadPhase() const
	{
		return ModQuadPhase;
	}

	float FLFO::Generate(float* QuadPhaseOutput)
	{
		// If the LFO isn't playing, return last computed value for both output & quad.
		if (!bIsPlaying)
		{
			if (QuadPhaseOutput)
			{
				*QuadPhaseOutput = QuadLastOutput;
			}

			return LastOutput;
		}

		WrapPhase();
		LastOutput = ComputeLFO(GetPhase(), QuadPhaseOutput);

		// Update the LFO phase after computing LFO values
		LoopCount += PhaseInc;
		UpdatePhase();

		// If in oneshot mode, check if wrapped and if so, turn the LFO off and compute last value.
		if (LFOMode == ELFOMode::OneShot)
		{
			if (LoopCount >= 1.0f)
			{
				bIsPlaying = false;
				LoopCount = 0.0f;
			}
		}

		// Return the output
		return LastOutput;
	}

	float FLFO::ComputeLFO(const float InPhase, float* OutQuad)
	{
		float Output = 0.0f;
		float QuadOutput = 0.0f;

		const float CurPhase = FMath::Fmod(InPhase + PhaseOffset, 1.0f);
		const float QuadPhase = FMath::Fmod(InPhase + PhaseOffset + 0.25f , 1.0f);

		switch (LFOType)
		{
			case ELFO::Sine:
			{
				// Must subtract pi and flip sign to guarantee in valid range for FastSin function,
				// yet still starts on rising edge from 0 crossing.
				auto ComputeSine = [](float InputPhase)
				{
					if (InputPhase > 0.5f)
					{
						InputPhase -= 1.0f;
					}
					const float Angle = 2.0f * InputPhase * PI;
					return 0.5f * Audio::FastSin(Angle) + 0.5f;
				};
				Output = ComputeSine(CurPhase);
				QuadOutput = ComputeSine(QuadPhase);
			}
			break;

			case ELFO::UpSaw:
			{
				Output = CurPhase;
				QuadOutput = QuadPhase;
			}
			break;

			case ELFO::DownSaw:
			{
				Output = 1.0f - CurPhase;
				QuadOutput = 1.0f - QuadPhase;
			}
			break;

			case ELFO::Square:
			{
				Output = CurPhase > PulseWidth ? 0.0f : 1.0f;
				QuadOutput = QuadPhase > PulseWidth ? 0.0f : 1.0f;
			}
			break;

			case ELFO::Triangle:
			{
				Output = 1.0f - FMath::Abs(GetBipolar(CurPhase));
				QuadOutput = 1.0f - FMath::Abs(GetBipolar(QuadPhase));
			}
			break;

			case ELFO::Exponential:
			{
				Output = FMath::Pow(CurPhase, ExponentialFactor);
				QuadOutput = FMath::Pow(QuadPhase, ExponentialFactor);
			}
			break;

			case ELFO::RandomSampleHold:
			{
				const float FrequencyThreshold = SampleRate / Freq;
				if (RSHCounter > (uint32)FrequencyThreshold)
				{
					RSHCounter = 0;
					RSHValue = FMath::FRand();
				}
				else
				{
					++RSHCounter;
				}

				Output = RSHValue;
				QuadOutput = RSHValue;
			}
			break;
		}

		if (bBipolar)
		{
			Output = GetBipolar(Output);
			QuadOutput = GetBipolar(QuadOutput);
		}

		const float MaxGain = Gain * ExternalGainMod;
		Output = Output * MaxGain;
		QuadOutput = QuadOutput * MaxGain;

		// If we have a mod matrix, then mix in the destination data
		// This allows LFO's (or envelopes, etc) to modulation this LFO
		if (ModMatrix)
		{
			ModMatrix->GetDestinationValue(VoiceId, ModScaleDest, ModAdd);
			ModMatrix->GetDestinationValue(VoiceId, ModAddDest, ModScale);

			Output = Output * ModScale + ModAdd;
			QuadOutput = QuadOutput * ModScale + ModAdd;

			// Write out the modulations
			ModMatrix->SetSourceValue(VoiceId, ModNormalPhase, Output);
			ModMatrix->SetSourceValue(VoiceId, ModQuadPhase, QuadOutput);
		}

		QuadLastOutput = QuadOutput;

		if (OutQuad)
		{
			*OutQuad = QuadOutput;
		}

		return Output;
	}
} // namespace Audio
