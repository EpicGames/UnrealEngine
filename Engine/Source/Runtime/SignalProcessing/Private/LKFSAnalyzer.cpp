// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LKFSAnalyzer.h"

#include "DSP/LKFSUtils.h"
#include "DSP/SlidingWindow.h"

namespace Audio
{
	namespace LKFSAnalyzerPrivate
	{
		float LKFSToEnergy(float InLKFS)
		{
			return FMath::Pow(10, (InLKFS + 0.691f) / 10.f);
		}

		// Calculates average value from a circular buffer. Allows for interleaved data by providing a "stride". 
		float CalculateAverageFromHistory(const TCircularBuffer<float>& InHistory, int32 InStartIndex, int32 InEndIndex, int32 InStride)
		{
			const int32 Num = (InEndIndex - InStartIndex) / InStride;
			if (Num <= 0)
			{
				return 0.0f;
			}

			float Accum = 0.0f;
			for (int32 Index = InStartIndex; Index < InEndIndex; Index += InStride)
			{
				Accum += InHistory[Index];
			}

			return Accum /= Num;
		}

		// Calculates gated average value from a circular buffer. Allows for interleaved data by providing a "stride". 
		float CalculateGatedAverageFromHistory(const TCircularBuffer<float>& InHistory, int32 InStartIndex, int32 InEndIndex, int32 InStride, float InGate)
		{
			float Accum = 0.0f;
			int32 Num = 0;

			for (int32 Index = InStartIndex; Index < InEndIndex; Index += InStride)
			{
				const float Val = InHistory[Index];

				if (Val > InGate)
				{
					Accum += Val;
					++Num;
				}
			}

			return (Num > 0) ? Accum /= Num : 0.0f;
		}

		float CalculateGatedLoudnessFromHistory(const TCircularBuffer<float>& InHistory, int32 InStartIndex, int32 InEndIndex, int32 InStride)
		{
			// Gated Loudness only examines loudnesses above a certain gating loudness. 
			//
			// It is a 2 pass algorithm 
			// Pass 1: Use a set loudness threshold of -70lkfs to determine an initial gated loudness
			// Pass 2: Use the initial gated loudness to derive a new loudness threshold and calculate the gate loudness again. 
			constexpr float InitLoudnessThreshold = -70.0f;

			float InitEnergyThreshold = LKFSToEnergy(InitLoudnessThreshold);
			const float GatedEnergyPass1 = CalculateGatedAverageFromHistory(InHistory, InStartIndex, InEndIndex, InStride, InitEnergyThreshold);

			const float NewThreshold = GatedEnergyPass1 * 0.1f;
			const float GatedEnergyPass2 = CalculateGatedAverageFromHistory(InHistory, InStartIndex, InEndIndex, InStride, NewThreshold);

			return FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(GatedEnergyPass2);
		}

		// Implementation copied from FFineGrainedPerformanceTracker::GetPercentileValue(...)
		float GetPercentileValue(TArray<float>& Samples, int32 Percentile)
		{
			ensure(Percentile >= 0 && Percentile <= 100);

			int32 Left = 0;
			int32 Right = Samples.Num() - 1;

			if (Right < 0)
			{
				return -1.0f;
			}

			const int32 PercentileClamped = FMath::Clamp(Percentile, 0, 100);
			const int32 PercentileOrdinal = FMath::RoundToInt32((float(PercentileClamped) * Right) / 100);

			// this is quickselect (see http://en.wikipedia.org/wiki/Quickselect for details).
			while (Right != Left)
			{
				// partition
				int32 MovingLeft = Left - 1;
				int32 MovingRight = Right;
				float Pivot = Samples[MovingRight];
				for (; ; )
				{
					while (Samples[++MovingLeft] < Pivot);
					while (Samples[--MovingRight] > Pivot)
					{
						if (MovingRight == Left)
						{
							break;
						}
					}

					if (MovingLeft >= MovingRight)
					{
						break;
					}

					const float Temp = Samples[MovingLeft];
					Samples[MovingLeft] = Samples[MovingRight];
					Samples[MovingRight] = Temp;
				}

				const float Temp = Samples[MovingLeft];
				Samples[MovingLeft] = Samples[Right];
				Samples[Right] = Temp;

				// now we're pivoted around MovingLeft
				// decide what part K-th largest belong to
				if (MovingLeft > PercentileOrdinal)
				{
					Right = MovingLeft - 1;
				}
				else if (MovingLeft < PercentileOrdinal)
				{
					Left = MovingLeft + 1;
				}
				else
				{
					// we hit exactly the value we need, no need to sort further
					break;
				}
			}

			return Samples[PercentileOrdinal];
		}
	} // namespace LKFSAnalyzerPrivate


	//----------------------------------------------------------------------------------------
	// FLKFSAnalyzer
	//----------------------------------------------------------------------------------------
	FLKFSAnalyzer::FLKFSAnalyzer(int32 InNumChannels, float InSampleRate, const FLKFSAnalyzerSettings& InSettings)
		: Settings(InSettings)
		, NumChannels(InNumChannels)
		, SampleRate(InSampleRate)
		, ChannelEnergyHistory(1)
		, OverallEnergyHistory(1)
		, LoudnessAnalyzer(InSampleRate, InSettings.LoudnessAnalyzerSettings)
	{
		check(NumChannels > 0);
		check(SampleRate > 0.f);
		check(InSettings.ShortTermLoudnessDuration > 0.f);
		check(InSettings.IntegratedLoudnessDuration > 0.f);
		check(InSettings.IntegratedLoudnessAnalysisPeriod> 0.f);

		int32 NumSlidingWindowFrames = LoudnessAnalyzer.GetSettings().WindowSize;

		int32 NumAnalysisFrames = FMath::CeilToInt(InSettings.AnalysisWindowDuration * SampleRate);
		int32 NumAnalysisHopFrames = FMath::CeilToInt(InSettings.AnalysisPeriod * SampleRate);

		// There may be multiple calls to the LoudnessAnalyzer to produce a single 
		// FLoudnessDatum. The sliding window hop is tuned here so that it best 
		// matches the desired AnalysisPeriod while maintaining between a 25% to 
		// 75% window overlap.
		LKFSUtils::TuneSlidingWindowHopSize(NumAnalysisHopFrames, NumSlidingWindowFrames, NumSlidingWindowHopFrames, NumAnalysisHopWindows);

		// Determine how many windows to analyze per loudness result
		NumAnalysisWindows = FMath::Max(1, ((NumAnalysisFrames - (NumSlidingWindowFrames / 2)) / NumSlidingWindowHopFrames) + 1);

		// Include NumChannels in calculating window size because windows are generated 
		// with interleaved samples and deinterleaved later. 
		int32 NumSlidingWindowSamples = NumSlidingWindowFrames * NumChannels;
		int32 NumSlidingWindowHopSamples = NumSlidingWindowHopFrames * NumChannels;

		InternalBuffer = TSlidingBuffer<float>(NumSlidingWindowSamples, NumSlidingWindowHopSamples);
		InternalWindow.AddUninitialized(NumSlidingWindowSamples);
		ChannelPerceptualEnergy.AddZeroed(NumChannels * NumAnalysisWindows);
		OverallPerceptualEnergy.AddZeroed(NumAnalysisWindows);

		// Initialize settings for calculating aggregate stats
		const int32 NumResultHopFrames = NumAnalysisHopWindows * NumSlidingWindowHopFrames;
		NumShortTermHistoryResults     = FMath::Max(1, FMath::DivideAndRoundUp(static_cast<int32>(InSettings.ShortTermLoudnessDuration * SampleRate), NumResultHopFrames));
		NumIntegratedHistoryResults    = FMath::Max(1, FMath::DivideAndRoundUp(static_cast<int32>(InSettings.IntegratedLoudnessDuration * SampleRate), NumResultHopFrames));
		NumIntegratedHopHistoryResults = FMath::Max(1, FMath::DivideAndRoundUp(static_cast<int32>(InSettings.IntegratedLoudnessAnalysisPeriod * SampleRate), NumResultHopFrames));

		const int32 MaxHistory = FMath::Max(NumIntegratedHistoryResults, NumShortTermHistoryResults);
		ChannelEnergyHistory = TCircularBuffer<float>(MaxHistory * NumChannels);
		OverallEnergyHistory = TCircularBuffer<float>(MaxHistory);

		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			AggregateLoudnessStats.Add(ChannelIdx);
		}

		AggregateLoudnessStats.Add(FLKFSAnalyzerResults::ChannelIndexOverall);

		if (InSettings.bCalculatePerChannelLoudnessRange)
		{
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
			{
				LoudnessRangeStatePerChannel.Emplace(ChannelIdx, static_cast<uint32>(NumIntegratedHistoryResults));
			}
		}

		if (InSettings.bCalculateOverallLoudnessRange)
		{
			LoudnessRangeStatePerChannel.Emplace(FLKFSAnalyzerResults::ChannelIndexOverall, static_cast<uint32>(NumIntegratedHistoryResults));
		}
	}

	const FLKFSAnalyzerSettings& FLKFSAnalyzer::GetSettings() const
	{
		return Settings;
	}

	void FLKFSAnalyzer::Analyze(TArrayView<const float> InAudio, FLKFSAnalyzerResults& OutLKFSResults)
	{
		TAutoSlidingWindow<float> SlidingWindow(InternalBuffer, InAudio, InternalWindow);

		AnalyzeSlidingWindow(SlidingWindow, OutLKFSResults);
	}

	void FLKFSAnalyzer::AnalyzeSlidingWindow(TAutoSlidingWindow<float>& InSlidingWindow, FLKFSAnalyzerResults& OutResult)
	{
		check(NumAnalysisWindows > 0);
		check(NumAnalysisHopWindows > 0);

		for (const TArray<float>& Window : InSlidingWindow)
		{
			AnalyzeWindow(Window);

			if ((NumAnalyzedBuffers >= NumAnalysisWindows) && ((NumAnalyzedBuffers - NumAnalysisWindows) % NumAnalysisHopWindows) == 0)
			{
				AddCurrentLoudnessDataToResult(OutResult);
			}
		}
	}

	void FLKFSAnalyzer::AnalyzeWindow(TArrayView<const float> InWindow)
	{
		const int32 WrappedWindowIdx = (NumAnalyzedBuffers % NumAnalysisWindows);
		const int32 ChannelPerceptualEnergyWritePos = WrappedWindowIdx * NumChannels;

		// Calculate perceptual energy
		OverallPerceptualEnergy[WrappedWindowIdx] = LoudnessAnalyzer.CalculatePerceptualEnergy(InWindow, NumChannels, MakeArrayView<float>(ChannelPerceptualEnergy.GetData() + ChannelPerceptualEnergyWritePos, NumChannels));

		// Update counters
		NumAnalyzedBuffers++;
	}

	void FLKFSAnalyzer::AddCurrentLoudnessDataToResult(FLKFSAnalyzerResults& OutResult)
	{
		AddCurrentLoudnessDataToHistory();
		UpdateAggregateLoudnessStats();

		// Accounting for all the frames positions analyzed in creating the data, determine the center frame. 
		const int32 CenterFramePos = ((2 * NumAnalyzedBuffers - NumAnalysisWindows) * NumSlidingWindowHopFrames + LoudnessAnalyzer.GetSettings().WindowSize) / 2;
		const float Timestamp = CenterFramePos / SampleRate;

		// Add per channel results
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
		{
			const int32 HistoryIndex = ((NumEnergyHistory - 1) * NumChannels) + ChannelIdx;
			const float Energy = ChannelEnergyHistory[HistoryIndex];
			const FAggregateLoudnessStats& Stats = AggregateLoudnessStats[ChannelIdx];

			OutResult.Add(
				FLKFSResult
				{
					.Channel = ChannelIdx, 
					.Timestamp = Timestamp, 
					.Energy = Energy, 
					.Loudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(Energy),
					.ShortTermLoudness = Stats.ShortTermLoudness, 
					.IntegratedLoudness = Stats.IntegratedLoudness,
					.GatedLoudness = Stats.GatedLoudness,
					.LoudnessRange = Stats.LoudnessRange,
				}
			);
		}

		// Add overall results
		{
			const int32 HistoryIndex = (NumEnergyHistory - 1);
			const float Energy = OverallEnergyHistory[HistoryIndex];
			const FAggregateLoudnessStats& Stats = AggregateLoudnessStats[FLKFSAnalyzerResults::ChannelIndexOverall];

			OutResult.Add(
				FLKFSResult
				{
					.Channel = FLKFSAnalyzerResults::ChannelIndexOverall,
					.Timestamp = Timestamp, 
					.Energy = Energy, 
					.Loudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(Energy),
					.ShortTermLoudness = Stats.ShortTermLoudness, 
					.IntegratedLoudness = Stats.IntegratedLoudness,
					.GatedLoudness = Stats.GatedLoudness,
					.LoudnessRange = Stats.LoudnessRange,
				}
			);
		}
	}

	void FLKFSAnalyzer::AddCurrentLoudnessDataToHistory()
	{
		// Compute loudness per channel and store history
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			// Determine average perceptual energy over analysis window
			float PerceptualEnergy = 0.f;
			for (int32 Pos = ChannelIdx; Pos < ChannelPerceptualEnergy.Num(); Pos += NumChannels)
			{
				PerceptualEnergy += ChannelPerceptualEnergy[Pos];
			}
			PerceptualEnergy /= (float) NumAnalysisWindows;

			// Store perceptual energy in history 
			ChannelEnergyHistory[(NumChannels * NumEnergyHistory) + ChannelIdx] = PerceptualEnergy;
		}

		// Compute loudness overall
		{
			float PerceptualEnergy = 0.f;
			for (int32 Pos = 0; Pos < OverallPerceptualEnergy.Num(); Pos++)
			{
				PerceptualEnergy += OverallPerceptualEnergy[Pos];
			}
			PerceptualEnergy /= (float) NumAnalysisWindows;

			OverallEnergyHistory[NumEnergyHistory] = PerceptualEnergy;
		}

		NumEnergyHistory++;
	}

	void FLKFSAnalyzer::UpdateAggregateLoudnessStats()
	{
		// Short term loudness is always updated, but integrated & gated loudness are 
		// updated at a lower rate. 
		bool bUpdateIntegratedLoudness = (NumEnergyHistory == 1) || (NumEnergyHistory % NumIntegratedHopHistoryResults) == 0;


		// Per channel results
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			FAggregateLoudnessStats& Stats = AggregateLoudnessStats[ChannelIdx];
			FLoudnessRange* LoudnessRange = LoudnessRangeStatePerChannel.Find(ChannelIdx);
			UpdateAggregateLoudnessStats(ChannelEnergyHistory, ChannelIdx, NumChannels, bUpdateIntegratedLoudness, Stats, LoudnessRange);
		}

		// Overall results
		{
			FAggregateLoudnessStats& Stats = AggregateLoudnessStats[FLKFSAnalyzerResults::ChannelIndexOverall];
			FLoudnessRange* LoudnessRange = LoudnessRangeStatePerChannel.Find(FLKFSAnalyzerResults::ChannelIndexOverall);
			UpdateAggregateLoudnessStats(OverallEnergyHistory, 0 /*offset*/, 1/*stride*/, bUpdateIntegratedLoudness, Stats, LoudnessRange);
		}
	}

	void FLKFSAnalyzer::UpdateAggregateLoudnessStats(const TCircularBuffer<float>& InHistory, int32 InOffset, int32 InStride, bool bInUpdateIntegratedLoudness, FAggregateLoudnessStats& OutStats, FLoudnessRange* LoudnessRange) const
	{
		const int32 EndIndex = (NumEnergyHistory * InStride) + InOffset;

		// Update short term loudness always.
		const int32 ShortTermLoudnessStartIndex = InOffset + FMath::Max(0, (NumEnergyHistory - NumShortTermHistoryResults) * InStride);
		const float ShortTermAverageEnergy = LKFSAnalyzerPrivate::CalculateAverageFromHistory(InHistory, ShortTermLoudnessStartIndex, EndIndex, InStride);
		OutStats.ShortTermLoudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(ShortTermAverageEnergy);

		// Only update long term loudness if flag is set. 
		if (bInUpdateIntegratedLoudness)
		{
			const int32 LongTermLoudnessStartIndex = InOffset + FMath::Max(0, (NumEnergyHistory - NumIntegratedHistoryResults * InStride));

			float IntegratedEnergy = LKFSAnalyzerPrivate::CalculateAverageFromHistory(InHistory, LongTermLoudnessStartIndex, EndIndex, InStride);
			OutStats.IntegratedLoudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(IntegratedEnergy);
			OutStats.GatedLoudness = LKFSAnalyzerPrivate::CalculateGatedLoudnessFromHistory(InHistory, LongTermLoudnessStartIndex, EndIndex, InStride);
		}

		// Update optional loudness range (if we have LoudnessRange state):
		if (LoudnessRange)
		{
			LoudnessRange->AddShortTermEnergyMeasurement(ShortTermAverageEnergy);

			if (bInUpdateIntegratedLoudness)
			{
				OutStats.LoudnessRange = LoudnessRange->CalculateLoudnessRange();
			}
		}
	}

	
	//----------------------------------------------------------------------------------------
	// FLKFSAnalyzerResults
	//----------------------------------------------------------------------------------------
	int32 FLKFSAnalyzerResults::GetNumChannels() const
	{
		const int32 Num = ChannelResults.Num();

		if (Num > 0)
		{
			return Num - 1; // Subtract one for storage of "Overall" channel
		}

		return 0;
	}

	void FLKFSAnalyzerResults::Add(FLKFSResult InDatum)
	{
		TArray<FLKFSResult>& ChannelData = ChannelResults.FindOrAdd(InDatum.Channel);
		ChannelData.Add(MoveTemp(InDatum));
	}

	const TArray<FLKFSResult>& FLKFSAnalyzerResults::GetChannelLoudnessResults(int32 ChannelIdx) const
	{
		static const TArray<FLKFSResult> EmptyArray;

		const TArray<FLKFSResult>* ChannelData = ChannelResults.Find(ChannelIdx);

		return ChannelData ? *ChannelData : EmptyArray;
	}

	const TArray<FLKFSResult>& FLKFSAnalyzerResults::GetLoudnessResults() const
	{
		return GetChannelLoudnessResults(ChannelIndexOverall);
	}


	//----------------------------------------------------------------------------------------
	// FLoudnessRange
	//----------------------------------------------------------------------------------------
	void FLKFSAnalyzer::FLoudnessRange::AddShortTermEnergyMeasurement(float InShortTermEnergy)
	{
		ShortTermEnergyHistory[ShortTermWriteIndex++] = InShortTermEnergy;
		ShortTermHistorySize = FMath::Min<uint32>(ShortTermHistorySize + 1, MaxShortTermHistorySize);
	}

	TOptional<FFloatInterval> FLKFSAnalyzer::FLoudnessRange::CalculateLoudnessRange() const
	{
		// Based on EBU Tech 3342
		if (ShortTermHistorySize == 0)
		{
			return NullOpt;
		}

		constexpr float AbsoluteThresholdEnergy = 1.17246636e-7f; // -70 LUFS
		constexpr float RelativeThresholdEnergy = 0.01f;		  // -20 LU 

		const uint32 LoudnessRangeStartIndex = ShortTermWriteIndex - ShortTermHistorySize;
		const uint32 LoudnessRangeEndIndex = ShortTermWriteIndex;

		// Apply the absolute-threshold gating
		const float RefEnergy = LKFSAnalyzerPrivate::CalculateGatedAverageFromHistory(
			ShortTermEnergyHistory,
			static_cast<int32>(LoudnessRangeStartIndex),
			static_cast<int32>(LoudnessRangeEndIndex),
			1 /*Stride*/,
			AbsoluteThresholdEnergy
		);

		// Apply the relative-threshold gating
		const float RelGateEnergy = FMath::Max(RefEnergy * RelativeThresholdEnergy, AbsoluteThresholdEnergy);

		TArray<float> ShortTermLoudnessRelGated;
		ShortTermLoudnessRelGated.Reserve(ShortTermHistorySize);

		for (uint32 Index = LoudnessRangeStartIndex; Index < LoudnessRangeEndIndex; ++Index)
		{
			const float ShortTermEnergy = ShortTermEnergyHistory[Index];

			if (ShortTermEnergy >= RelGateEnergy)
			{
				ShortTermLoudnessRelGated.Emplace(FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(ShortTermEnergy));
			}
		}

		if (ShortTermLoudnessRelGated.IsEmpty())
		{
			return NullOpt;
		}

		// Compute the high and low percentiles of the distribution of values in ShortTermLoudnessRelGated
		static constexpr int32 LowerPercentile = 10;
		static constexpr int32 UpperPercentile = 95;

		const float LoudnessRangeMin = LKFSAnalyzerPrivate::GetPercentileValue(ShortTermLoudnessRelGated, LowerPercentile);
		const float LoudnessRangeMax = LKFSAnalyzerPrivate::GetPercentileValue(ShortTermLoudnessRelGated, UpperPercentile);

		return FFloatInterval(LoudnessRangeMin, LoudnessRangeMax);
	}
}
