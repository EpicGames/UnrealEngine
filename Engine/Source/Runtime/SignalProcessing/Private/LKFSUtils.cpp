// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LKFSUtils.h"

#include "Math/UnrealMathUtility.h"

namespace Audio::LKFSUtils
{
	void TuneSlidingWindowHopSize(int32 InAnalysisHopFrames, int32 InWindowFrames, int32& OutBestWindowHopFrames, int32& OutBestAnalysisHopWindows) 
	{
		// The minimum sliding window hop size is the lesser of 
		// - 25% of the sliding window size
		// - The analysis hop size
		int32 MinNumSlidingWindowHopFrames = FMath::Max(1, InWindowFrames / 4);
		if (MinNumSlidingWindowHopFrames > InAnalysisHopFrames)
		{
			MinNumSlidingWindowHopFrames = InAnalysisHopFrames;
		}

		// The maximum sliding window hop size is the lesser of
		// - 75% of the sliding window size
		// - The analysis hop size
		int32 MaxNumSlidingWindowHopFrames = FMath::Max(1, (3 * InWindowFrames) / 4);
		if (MaxNumSlidingWindowHopFrames > InAnalysisHopFrames)
		{
			MaxNumSlidingWindowHopFrames = InAnalysisHopFrames;
		}

		// Iterate through the possible sizes to find the one that fits best with our settings.
		int32 BestWindowHopFrames = -1;
		int32 BestAnalysisHopWindows = -1;
		int32 BestMismatch = InAnalysisHopFrames + 2; 

		for (int32 CandidateWindowHopFrames = MinNumSlidingWindowHopFrames; CandidateWindowHopFrames <= MaxNumSlidingWindowHopFrames; CandidateWindowHopFrames++)
		{
			int32 CandidateAnalysisHopWindows = InAnalysisHopFrames / CandidateWindowHopFrames;
			// The match quality is determined by how neatly the sliding window hop size fits into the anlaysis window hop size. 
			int32 CandidateMismatch = FMath::Abs<int32>(InAnalysisHopFrames - (CandidateWindowHopFrames * CandidateAnalysisHopWindows));
			if (CandidateMismatch < BestMismatch)
			{
				BestMismatch = CandidateMismatch;
				BestWindowHopFrames = CandidateWindowHopFrames;
				BestAnalysisHopWindows = CandidateAnalysisHopWindows;
			}
		}

		// The algorithm above should always find a valid window hop
		check(BestWindowHopFrames != -1);

		OutBestWindowHopFrames = BestWindowHopFrames;
		OutBestAnalysisHopWindows = BestAnalysisHopWindows;
	}
} // namespace Audio::LKFSUtils
