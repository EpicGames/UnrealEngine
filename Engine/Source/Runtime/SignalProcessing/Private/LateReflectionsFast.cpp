// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LateReflectionsFast.h"

#include "DSP/BufferOnePoleLPF.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	// The presets for the late refelections reverb were based on a strange sample rate. This 
	// function helps translate delay values in the paper to delay values for this reverb.
	// https://ccrma.stanford.edu/~dattorro/EffectDesignPart1.pdf
	static int32 LateReflectionsGetDelaySamples(float InSampleRate, float InPresetValue) 
	{
		return (int32)((float)InSampleRate / 29761.f * InPresetValue);
	}

	FLateReflectionsFastSettings::FLateReflectionsFastSettings()
		: LateDelayMsec(0.0f)
		, LateGainDB(0.0f)
		, Bandwidth(0.5f)
		, Diffusion(0.5f)
		, Dampening(0.5f)
		, Decay(0.5f)
		, Density(0.5f)
	{}

	bool FLateReflectionsFastSettings::operator==(const FLateReflectionsFastSettings& Other) const
	{
		bool bIsEqual = (
			(Other.LateDelayMsec == LateDelayMsec) &&
			(Other.LateGainDB == LateGainDB) &&
			(Other.Bandwidth == Bandwidth) &&
			(Other.Diffusion == Diffusion) &&
			(Other.Dampening == Dampening) &&
			(Other.Decay == Decay) &&
			(Other.Density == Density));

		return bIsEqual;
	}

	bool FLateReflectionsFastSettings::operator!=(const FLateReflectionsFastSettings& Other) const
	{
		return !(*this == Other);
	}

	void FLateReflectionsPlateOutputs::ResizeAndZero(int32 InNumSamples)
	{
		for (int32 i = 0; i < NumTaps; i++)
		{
			Taps[i].Reset(InNumSamples);
			Taps[i].AddUninitialized(InNumSamples);
			FMemory::Memset(Taps[i].GetData(), 0, sizeof(float) * InNumSamples);
		}

		Output.Reset(InNumSamples);
		Output.AddUninitialized(InNumSamples);
		FMemory::Memset(Output.GetData(), 0, sizeof(float) * InNumSamples);
	}


	FLateReflectionsPlateDelays FLateReflectionsPlateDelays::DefaultLeftDelays(float InSampleRate)
	{
		FLateReflectionsPlateDelays Delays;

		Delays.NumSamplesModulatedBase = LateReflectionsGetDelaySamples(InSampleRate, 908);
		Delays.NumSamplesModulatedDelta = LateReflectionsGetDelaySamples(InSampleRate, 16);
		// Left Delay 1 should add up to 4217 and have tap outs at 353 (R0), 1190 (L4), and 3627 (R1)
		Delays.NumSamplesDelayA = LateReflectionsGetDelaySamples(InSampleRate, 353);
		Delays.NumSamplesDelayB = LateReflectionsGetDelaySamples(InSampleRate, 837);
		Delays.NumSamplesDelayC = LateReflectionsGetDelaySamples(InSampleRate, 2437);
		Delays.NumSamplesDelayD = LateReflectionsGetDelaySamples(InSampleRate, 590);
		Delays.NumSamplesAPF = LateReflectionsGetDelaySamples(InSampleRate, 2656);
		// APF Delay should have tapouts at 187 (L5) and 1228 (R2)
		Delays.NumSamplesDelayE = LateReflectionsGetDelaySamples(InSampleRate, 187);
		Delays.NumSamplesDelayF = LateReflectionsGetDelaySamples(InSampleRate, 1041);
		// Left Delay 2 should add up to 3136 and have tap outs at 1066 (L6) and 2673 (R3)
		Delays.NumSamplesDelayG = LateReflectionsGetDelaySamples(InSampleRate, 1066);
		Delays.NumSamplesDelayH = LateReflectionsGetDelaySamples(InSampleRate, 1607);
		Delays.NumSamplesDelayI = LateReflectionsGetDelaySamples(InSampleRate, 463);

		return Delays;
	}

	FLateReflectionsPlateDelays FLateReflectionsPlateDelays::DefaultRightDelays(float InSampleRate)
	{
		FLateReflectionsPlateDelays Delays;

		Delays.NumSamplesModulatedBase = LateReflectionsGetDelaySamples(InSampleRate, 672);
		Delays.NumSamplesModulatedDelta = LateReflectionsGetDelaySamples(InSampleRate, 16);
		// Right Delay 1 should add up to 4453 and have tap outs at 266 (L0), 2111 (R4), 2974 (L1)
		Delays.NumSamplesDelayA = LateReflectionsGetDelaySamples(InSampleRate, 266);
		Delays.NumSamplesDelayB = LateReflectionsGetDelaySamples(InSampleRate, 1845);
		Delays.NumSamplesDelayC = LateReflectionsGetDelaySamples(InSampleRate, 863);
		Delays.NumSamplesDelayD = LateReflectionsGetDelaySamples(InSampleRate, 1479);
		Delays.NumSamplesAPF = LateReflectionsGetDelaySamples(InSampleRate, 1800);
		// APF Delay should have tapouts at 335 (R5) and 1913 (L2)
		Delays.NumSamplesDelayE = LateReflectionsGetDelaySamples(InSampleRate, 335);
		Delays.NumSamplesDelayF = LateReflectionsGetDelaySamples(InSampleRate, 1578);
		// Right Delay 2 should add up to 3720 and have tap outs at 121 (R6) and 1996 (L3)
		Delays.NumSamplesDelayG = LateReflectionsGetDelaySamples(InSampleRate, 121);
		Delays.NumSamplesDelayH = LateReflectionsGetDelaySamples(InSampleRate, 1875);
		Delays.NumSamplesDelayI = LateReflectionsGetDelaySamples(InSampleRate, 1724);

		return Delays;
	}


	FLateReflectionsPlate::FLateReflectionsPlate(
				float InSampleRate, 
				int32 InMaxNumInternalBufferSamples, 
				const FLateReflectionsPlateDelays& InDelays)
	: 	SampleRate(InSampleRate)
		, NumInternalBufferSamples(InMaxNumInternalBufferSamples)
		, Dampening(0.0005f)
		, Decay(0.50f)
		, Density(0.0f)
		, PlateDelays(InDelays)
	{
		// NumInternalBufferSamples must be less than last delay and should be aligned to SIMD boundaries
		if (NumInternalBufferSamples > PlateDelays.NumSamplesDelayI) {
			NumInternalBufferSamples = PlateDelays.NumSamplesDelayI;
		}
		NumInternalBufferSamples -= (NumInternalBufferSamples % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);

		checkf(NumInternalBufferSamples > 0, TEXT("InDelays.DelaySample2C too small"));

		// Create the delay line for a plate
		// ModulatedAPF -> Delay1[A-D] -> LPF -> APF -> Delay2[A-C] -> Output
		//                                        |-> APFTapDelay[1-2]
		ModulatedAPF = MakeUnique<FDynamicDelayAPF>(
				-Density,
				PlateDelays.NumSamplesModulatedBase - PlateDelays.NumSamplesModulatedDelta,
				PlateDelays.NumSamplesModulatedBase + PlateDelays.NumSamplesModulatedDelta,
				NumInternalBufferSamples,
				SampleRate);
		DelayA = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayA, PlateDelays.NumSamplesDelayA);
		DelayB = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayB, PlateDelays.NumSamplesDelayB);
		DelayC = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayC, PlateDelays.NumSamplesDelayC);
		DelayD = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayD, PlateDelays.NumSamplesDelayD);
		LPF = MakeUnique<FBufferOnePoleLPF>(Dampening);
		APF = MakeUnique<FLongDelayAPF>(
				Density - 0.15f,
				PlateDelays.NumSamplesAPF,
				NumInternalBufferSamples);
		DelayE = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayE, PlateDelays.NumSamplesDelayE);
		DelayF = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayF, PlateDelays.NumSamplesDelayF);
		DelayG = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayG, PlateDelays.NumSamplesDelayG);
		DelayH = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayH, PlateDelays.NumSamplesDelayH);
		DelayI = MakeUnique<FIntegerDelay>(PlateDelays.NumSamplesDelayI, PlateDelays.NumSamplesDelayI);
	}

	FLateReflectionsPlate::~FLateReflectionsPlate()
	{
	}


	void FLateReflectionsPlate::ProcessAudioFrames(
			const FAlignedFloatBuffer& InSamples,
			const FAlignedFloatBuffer& InFeedbackSamples,
			const FAlignedFloatBuffer& InDelayModulations,
			FLateReflectionsPlateOutputs& OutPlateSamples)
	{
		
		// Prepare output
		const int32 InNum = InSamples.Num();
		OutPlateSamples.ResizeAndZero(InNum);

		checkf(InNum == InFeedbackSamples.Num(), TEXT("InSamples and InFeedbackSamples must have equal length"));
		checkf(InNum == InDelayModulations.Num(), TEXT("InSamples and InDelayModulations must have equal length"));

		// Check buffer sizes and quit early if there is an error.
		if (InNum != InFeedbackSamples.Num())
		{
			return;	
		}
		if (InNum != InDelayModulations.Num())
		{
			return;
		}

		float* DataPtr = nullptr;

		// Prepare internal buffers to handle input data.
		WorkBufferA.Reset(InNum);
		WorkBufferA.AddUninitialized(InNum);
		WorkBufferB.Reset(InNum);
		WorkBufferB.AddUninitialized(InNum);
		WorkBufferC.Reset(InNum);
		WorkBufferC.AddUninitialized(InNum);

		// Copy Feedback Samples to aligned internal buffer.
		ArrayWeightedSum(InFeedbackSamples, 1.0 - Decay, InSamples, WorkBufferA);

		// Input -> ModulatedAPF
		ModulatedAPF->ProcessAudio(WorkBufferA, InDelayModulations, WorkBufferB);

		// ModulatedAPF -> Delay1
		DelayA->ProcessAudio(WorkBufferB, OutPlateSamples.Taps[0]);
		DelayB->ProcessAudio(OutPlateSamples.Taps[0], OutPlateSamples.Taps[1]);
		DelayC->ProcessAudio(OutPlateSamples.Taps[1], OutPlateSamples.Taps[2]);
		DelayD->ProcessAudio(OutPlateSamples.Taps[2], WorkBufferA);

		// Apply dampening
		ArrayMultiplyByConstantInPlace(WorkBufferA, 1.0f - Dampening);
		
		// Delay1 -> LPF
		LPF->ProcessAudio(WorkBufferA, WorkBufferB);

		// Apply decay
		ArrayMultiplyByConstantInPlace(WorkBufferB, 1.0f - Decay);

		// LPF -> APF
		APF->ProcessAudio(WorkBufferB, WorkBufferA, WorkBufferC);
		DelayE->ProcessAudio(WorkBufferC, OutPlateSamples.Taps[3]);
		DelayF->ProcessAudio(OutPlateSamples.Taps[3], OutPlateSamples.Taps[4]);

		// APF-> Delay2
		DelayG->ProcessAudio(WorkBufferA, OutPlateSamples.Taps[5]);
		DelayH->ProcessAudio(OutPlateSamples.Taps[5], OutPlateSamples.Taps[6]);
		DelayI->ProcessAudio(OutPlateSamples.Taps[6], OutPlateSamples.Output);
	}

	void FLateReflectionsPlate::FlushAudio()
	{
		DelayA->Reset();
		DelayB->Reset();
		DelayC->Reset();
		DelayD->Reset();
		DelayE->Reset();
		DelayF->Reset();
		DelayG->Reset();
		DelayH->Reset();
		DelayI->Reset();

		LPF->FlushAudio();

		ModulatedAPF->Reset();
		APF->Reset();
	}

	void FLateReflectionsPlate::SetDampening(float InDampening)
	{
		Dampening = InDampening;
		LPF->SetG(Dampening);
	}

	void FLateReflectionsPlate::SetDecay(float InDecay)
	{
		Decay = InDecay;
	}

	void FLateReflectionsPlate::SetDensity(float InDensity)
	{
		Density = InDensity;
		ModulatedAPF->SetG(FMath::Clamp(-Density, -0.9f, 0.9f));
		APF->SetG(Density - 0.15f);
	}

	int32 FLateReflectionsPlate::GetNumInternalBufferSamples() const
	{
		return NumInternalBufferSamples;
	}

	void FLateReflectionsPlate::PeekDelayLine(int32 InNum, FAlignedFloatBuffer& OutSamples)
	{
		DelayI->PeekDelayLine(InNum, OutSamples);
	}

	// Limits on late reflections settings.
	const float FLateReflectionsFast::MaxLateDelayMsec = 2000.0f;
	const float FLateReflectionsFast::MinLateDelayMsec = 0.0f;
	const float FLateReflectionsFast::MaxLateGainDB = 0.0f;
	const float FLateReflectionsFast::MaxBandwidth = 0.99999f;
	const float FLateReflectionsFast::MinBandwidth = 0.0f;
	const float FLateReflectionsFast::MaxDampening = 0.99999f;
	const float FLateReflectionsFast::MinDampening = 0.0f;
	const float FLateReflectionsFast::MaxDiffusion = 0.99999f;
	const float FLateReflectionsFast::MinDiffusion = 0.0f;
	const float FLateReflectionsFast::MaxDecay = 1.0f;
	const float FLateReflectionsFast::MinDecay = 0.0001f;
	const float FLateReflectionsFast::MaxDensity = 1.0f;
	const float FLateReflectionsFast::MinDensity = 0.0f;

	const FLateReflectionsFastSettings FLateReflectionsFast::DefaultSettings;

	FLateReflectionsFast::FLateReflectionsFast(float InSampleRate, int32 InMaxNumInternalBufferSamples, const FLateReflectionsFastSettings& InSettings)
	:	SampleRate(InSampleRate)
		, Gain(1.0f)
		, ModulationPhase(0.0f)
		, ModulationQuadPhase(PI / 2.0f)
		, ModulationPhaseIncrement(0.0f)
		, NumInternalBufferSamples(0)
		, Settings(InSettings)
		, LeftPlateDelays(FLateReflectionsPlateDelays::DefaultLeftDelays(InSampleRate))
		, RightPlateDelays(FLateReflectionsPlateDelays::DefaultRightDelays(InSampleRate))

	{
		// Copy and clamp settings.
		Settings = InSettings;
		ClampSettings(Settings);
		
		// Create plates. 
		LeftPlate = MakeUnique<FLateReflectionsPlate>(SampleRate, InMaxNumInternalBufferSamples, LeftPlateDelays);
		RightPlate = MakeUnique<FLateReflectionsPlate>(SampleRate, InMaxNumInternalBufferSamples, RightPlateDelays);

		// The block size is limited by the delay lengths in the left & right plates.
		// Internal block size will be set to minimum of allowable sizes from plates.
		NumInternalBufferSamples = FMath::Min(LeftPlate->GetNumInternalBufferSamples(), RightPlate->GetNumInternalBufferSamples());

		// Set maximum predelay. Add 8 samples to give a little extra buffer for floating point rounding
		// differences. 
		int32 MaxDelay = (int32)(SampleRate * FLateReflectionsFast::MaxLateDelayMsec / 1000.0f) + 8;
		PreDelay = MakeUnique<FIntegerDelay>(MaxDelay, 0, NumInternalBufferSamples);

		InputLPF = MakeUnique<FBufferOnePoleLPF>(1.0f - Settings.Bandwidth);

		// make the signal decorrelators
		APF1 = MakeUnique<FLongDelayAPF>(Settings.Diffusion, LateReflectionsGetDelaySamples(SampleRate, 142), NumInternalBufferSamples);
		APF2 = MakeUnique<FLongDelayAPF>(Settings.Diffusion, LateReflectionsGetDelaySamples(SampleRate, 107), NumInternalBufferSamples);
		APF3 = MakeUnique<FLongDelayAPF>(Settings.Diffusion - 0.125f, LateReflectionsGetDelaySamples(SampleRate, 379), NumInternalBufferSamples);
		APF4 = MakeUnique<FLongDelayAPF>(Settings.Diffusion - 0.125f, LateReflectionsGetDelaySamples(SampleRate, 277), NumInternalBufferSamples);

		// Set modulation rate to 1 Hz. 
		ModulationPhaseIncrement = 2.0 * PI / SampleRate;
		
		// Resize internal buffers
		WorkBufferA.Reset(NumInternalBufferSamples);
		WorkBufferB.Reset(NumInternalBufferSamples);
		WorkBufferC.Reset(NumInternalBufferSamples);
		LeftDelayModSamples.Reset(NumInternalBufferSamples);
		RightDelayModSamples.Reset(NumInternalBufferSamples);

		WorkBufferA.AddUninitialized(NumInternalBufferSamples);
		WorkBufferB.AddUninitialized(NumInternalBufferSamples);
		WorkBufferC.AddUninitialized(NumInternalBufferSamples);
		LeftDelayModSamples.AddUninitialized(NumInternalBufferSamples);
		RightDelayModSamples.AddUninitialized(NumInternalBufferSamples);

		LeftPlateOutputs.ResizeAndZero(NumInternalBufferSamples);
		RightPlateOutputs.ResizeAndZero(NumInternalBufferSamples);


		ApplySettings();
	}

	FLateReflectionsFast::~FLateReflectionsFast()
	{}

	void FLateReflectionsFast::ClampSettings(FLateReflectionsFastSettings& InOutSettings)
	{
		// Enforce settings to be within max/min
		InOutSettings.LateDelayMsec = FMath::Clamp(InOutSettings.LateDelayMsec, MinLateDelayMsec, MaxLateDelayMsec);
		InOutSettings.LateGainDB = FMath::Min(InOutSettings.LateGainDB, FLateReflectionsFast::MaxLateGainDB);
		InOutSettings.Bandwidth = FMath::Clamp(InOutSettings.Bandwidth, MinBandwidth, MaxBandwidth);
		InOutSettings.Dampening = FMath::Clamp(InOutSettings.Dampening, MinDampening, MaxDampening);
		InOutSettings.Diffusion = FMath::Clamp(InOutSettings.Diffusion, MinDiffusion, MaxDiffusion);
		InOutSettings.Decay = FMath::Clamp(InOutSettings.Decay, MinDecay, MaxDecay);
		InOutSettings.Density = FMath::Clamp(InOutSettings.Density, MinDensity, MaxDensity);
	}

	// Sets the reverb settings, applies, and updates
	void FLateReflectionsFast::SetSettings(const FLateReflectionsFastSettings& InSettings)
	{
		Settings = InSettings;
		ClampSettings(Settings);
		ApplySettings();
	}

	void FLateReflectionsFast::ProcessAudio(const FAlignedFloatBuffer& InSamples, const int32 InNumChannels, FAlignedFloatBuffer& OutLeftSamples, FAlignedFloatBuffer& OutRightSamples)
	{
		checkf((InNumChannels == 1) || (InNumChannels == 2), TEXT("FLateReflections only supports 1 or 2 channel input audio"));

		const int32 InNum = InSamples.Num();
		const int32 InNumFrames = InNum / InNumChannels;
		
		OutLeftSamples.Reset(InNumFrames);
		OutRightSamples.Reset(InNumFrames);

		OutLeftSamples.AddUninitialized(InNumFrames);
		OutRightSamples.AddUninitialized(InNumFrames);

		const float* InSampleData = InSamples.GetData();
		float* OutLeftSampleData = OutLeftSamples.GetData();
		float* OutRightSampleData = OutRightSamples.GetData();

		int32 FramesLeftOver = InNumFrames;
		int32 InPos = 0;
		int32 OutPos = 0;
		while (FramesLeftOver > 0)
		{
			int32 FramesToProcess = FMath::Min(NumInternalBufferSamples, FramesLeftOver);

			ProcessAudioBuffer(&InSampleData[InPos], FramesToProcess, InNumChannels, &OutLeftSampleData[OutPos], &OutRightSampleData[OutPos]);

			FramesLeftOver -= FramesToProcess;
			InPos += FramesToProcess * InNumChannels;
			OutPos += FramesToProcess;
		}
	}

	void FLateReflectionsFast::FlushAudio()
	{
		PreDelay->Reset();

		InputLPF->FlushAudio();

		APF1->Reset();
		APF2->Reset();
		APF3->Reset();
		APF4->Reset();

		LeftPlate->FlushAudio();
		RightPlate->FlushAudio();
	}


	void FLateReflectionsFast::ProcessAudioBuffer(const float* InSampleData, const int32 InNumFrames, const int32 InNumChannels, float* OutLeftSampleData, float* OutRightSampleData)
	{
		checkf((1 == InNumChannels) || (2 == InNumChannels), TEXT("FLateReflections only supports 1 and 2 channel input audio"));

		// Resize internal buffers
		WorkBufferA.Reset(InNumFrames);
		WorkBufferB.Reset(InNumFrames);
		LeftDelayModSamples.Reset(InNumFrames);
		RightDelayModSamples.Reset(InNumFrames);

		WorkBufferA.AddUninitialized(InNumFrames);
		WorkBufferB.AddUninitialized(InNumFrames);
		LeftDelayModSamples.AddUninitialized(InNumFrames);
		RightDelayModSamples.AddUninitialized(InNumFrames);

		// Downmix to mono
		if (InNumChannels == 1)
		{
			FMemory::Memcpy(WorkBufferA.GetData(), InSampleData, sizeof(float) * InNumFrames);
		}
		else if (InNumChannels == 2)
		{
			BufferSum2ChannelToMonoFast(InSampleData, WorkBufferA.GetData(), InNumFrames);
		}
		else
		{
			// Shouldn't reach this line if checks are respected. 
			FMemory::Memset(OutLeftSampleData, 0, sizeof(float) * InNumFrames);
			FMemory::Memset(OutRightSampleData, 0, sizeof(float) * InNumFrames);
			return;
		}

		// Apply bandwidth, gain and channel averaging multipliers
		ArrayMultiplyByConstantInPlace(WorkBufferA, Settings.Bandwidth * Gain / InNumChannels);
		
		// Predelay
		PreDelay->ProcessAudio(WorkBufferA, WorkBufferB);

		// Input Diffusion 
		InputLPF->ProcessAudio(WorkBufferB, WorkBufferA);
		APF1->ProcessAudio(WorkBufferA, WorkBufferB);
		APF2->ProcessAudio(WorkBufferB, WorkBufferA);
		APF3->ProcessAudio(WorkBufferA, WorkBufferB);
		APF4->ProcessAudio(WorkBufferB, WorkBufferA);
		
		// The plates are connected to eachother so that the output of the left plate is fed into the input of the
		// right plate and vice versa. The delay lines of the plates are stored before processing new data through
		// the plate.
		LeftPlate->PeekDelayLine(InNumFrames, WorkBufferB);
		RightPlate->PeekDelayLine(InNumFrames, WorkBufferC);
		
		// Create the delay line modulations.
		GeneraterPlateModulations(InNumFrames, LeftDelayModSamples, RightDelayModSamples);

		// Run audio through plates
		LeftPlate->ProcessAudioFrames(WorkBufferA, WorkBufferC, LeftDelayModSamples, LeftPlateOutputs);
		RightPlate->ProcessAudioFrames(WorkBufferA, WorkBufferB, RightDelayModSamples, RightPlateOutputs);

		TArrayView<float> OutLeftSampleDataView(OutLeftSampleData, InNumFrames);
		TArrayView<float> OutRightSampleDataView(OutRightSampleData, InNumFrames);

		TArrayView<const float> LeftTaps1View(LeftPlateOutputs.Taps[1].GetData(), InNumFrames);
		TArrayView<const float> LeftTaps2View(LeftPlateOutputs.Taps[2].GetData(), InNumFrames);
		TArrayView<const float> LeftTaps3View(LeftPlateOutputs.Taps[3].GetData(), InNumFrames);
		TArrayView<const float> LeftTaps4View(LeftPlateOutputs.Taps[4].GetData(), InNumFrames);
		TArrayView<const float> LeftTaps5View(LeftPlateOutputs.Taps[5].GetData(), InNumFrames);
		TArrayView<const float> LeftTaps6View(LeftPlateOutputs.Taps[6].GetData(), InNumFrames);

		TArrayView<const float> RightTaps1View(RightPlateOutputs.Taps[1].GetData(), InNumFrames);
		TArrayView<const float> RightTaps2View(RightPlateOutputs.Taps[2].GetData(), InNumFrames);
		TArrayView<const float> RightTaps3View(RightPlateOutputs.Taps[3].GetData(), InNumFrames);
		TArrayView<const float> RightTaps4View(RightPlateOutputs.Taps[4].GetData(), InNumFrames);
		TArrayView<const float> RightTaps5View(RightPlateOutputs.Taps[5].GetData(), InNumFrames);
		TArrayView<const float> RightTaps6View(RightPlateOutputs.Taps[6].GetData(), InNumFrames);

		// Left Output
		// Channel -> [Plate and Tap#]
		// L -> +R0
		// L -> +R2
		// L -> -R4
		// L -> +R6
		// L -> -L1
		// L -> -L3
		// L -> -L5
		FMemory::Memcpy(OutLeftSampleData, RightPlateOutputs.Taps[0].GetData(), InNumFrames * sizeof(float));
		ArrayMixIn(RightTaps2View, OutLeftSampleDataView);
		ArraySubtractInPlace2(OutLeftSampleDataView, RightTaps4View);
		ArrayMixIn(RightTaps6View, OutLeftSampleDataView);
		ArraySubtractInPlace2(OutLeftSampleDataView, LeftTaps1View);
		ArraySubtractInPlace2(OutLeftSampleDataView, LeftTaps3View);
		ArraySubtractInPlace2(OutLeftSampleDataView, LeftTaps5View);
		
		// Right Output
		// Channel -> [Plate and Tap#]
		// R -> +L0
		// R -> +L2
		// R -> -L4
		// R -> +L6
		// R -> -R1
		// R -> -R3
		// R -> -R5
		FMemory::Memcpy(OutRightSampleData, LeftPlateOutputs.Taps[0].GetData(), InNumFrames * sizeof(float));
		ArrayMixIn(LeftTaps2View, OutRightSampleDataView);
		ArraySubtractInPlace2(OutRightSampleDataView, LeftTaps4View);
		ArrayMixIn(LeftTaps6View, OutRightSampleDataView);
		ArraySubtractInPlace2(OutRightSampleDataView, RightTaps1View);
		ArraySubtractInPlace2(OutRightSampleDataView, RightTaps3View);
		ArraySubtractInPlace2(OutRightSampleDataView, RightTaps5View);
	}

	void FLateReflectionsFast::ApplySettings()
	{
		// Convert gain to linear. 
		Gain = FMath::Pow(10.0f, Settings.LateGainDB/ 20.0f);

		// Convert delay to samples
		PreDelay->SetDelayLengthSamples((int32)(Settings.LateDelayMsec * SampleRate / 1000.0f));

		InputLPF->SetG(1.0f - Settings.Bandwidth);

		APF1->SetG(Settings.Diffusion);
		APF2->SetG(Settings.Diffusion);
		APF3->SetG(Settings.Diffusion - 0.125f);
		APF4->SetG(Settings.Diffusion - 0.125f);

		LeftPlate->SetDensity(Settings.Density);
		LeftPlate->SetDampening(Settings.Dampening);
		LeftPlate->SetDecay(Settings.Decay);

		RightPlate->SetDensity(Settings.Density);
		RightPlate->SetDampening(Settings.Dampening);
		RightPlate->SetDecay(Settings.Decay);
	}


	void FLateReflectionsFast::GeneraterPlateModulations(const int32 InNum, FAlignedFloatBuffer& OutLeftDelays, FAlignedFloatBuffer& OutRightDelays)
	{
		OutLeftDelays.Reset(InNum);
		OutRightDelays.Reset(InNum);
		OutLeftDelays.AddUninitialized(InNum);
		OutRightDelays.AddUninitialized(InNum);

		// Generate quadrature plate modulations to help break up hi-freq modes in plates. 
		float* LeftModData = OutLeftDelays.GetData();
		float* RightModData = OutRightDelays.GetData();
		for (int32 i = 0; i < InNum; i++)
		{		
			float NormalPhaseOut = 0.5f * FastSin(ModulationPhase) + 0.5f;
			float QuadPhaseOut = 0.5f * FastSin(ModulationQuadPhase) + 0.5f;
			ModulationPhase += ModulationPhaseIncrement;

			if (ModulationPhase > PI)
			{
				ModulationPhase -= 2 * PI;
			}

			ModulationQuadPhase += ModulationPhaseIncrement;
			if (ModulationQuadPhase > PI)
			{
				ModulationQuadPhase -= 2 * PI;
			}
			
			LeftModData[i] = LeftPlateDelays.NumSamplesModulatedBase + (NormalPhaseOut * LeftPlateDelays.NumSamplesModulatedDelta);
			RightModData[i] = RightPlateDelays.NumSamplesModulatedBase + (QuadPhaseOut * RightPlateDelays.NumSamplesModulatedDelta);
		}
	}
}
