// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LinearInterpFractionalDelay.h"
#include "DSP/Dsp.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FLinearInterpFractionalDelay::FLinearInterpFractionalDelay(int32 InMaxDelay, int32 InMaxNumInternalBufferSamples)
	: MaxDelay(InMaxDelay)
	, NumInternalBufferSamples(InMaxNumInternalBufferSamples)
	, UpperDelayPos(nullptr)
	, LowerDelayPos(nullptr)
	{
		checkf(MaxDelay > 0, TEXT("InMaxDelay must be greater than zero"));
		if (MaxDelay < 1)
		{
			MaxDelay = 1;
		}

		while (0 != (NumInternalBufferSamples % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER))
		{
			NumInternalBufferSamples--;
		}
		if (NumInternalBufferSamples < 1)
		{
			NumInternalBufferSamples = AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER;
		}

		// Allocate and prepare delay line for maximum delay.
		DelayLine = MakeUnique<FAlignedBlockBuffer>((2 * (MaxDelay + 1)) + NumInternalBufferSamples, MaxDelay + NumInternalBufferSamples + 1);
		DelayLine->AddZeros(MaxDelay + 1);
		
		IntegerDelayOffsets.Reset(NumInternalBufferSamples);
		IntegerDelayOffsets.AddUninitialized(NumInternalBufferSamples);

		for (int32 i = 0; i < NumInternalBufferSamples; i++)
		{
			IntegerDelayOffsets[i] = i + MaxDelay;
		}

		UpperDelayPos = (int*)FMemory::Malloc(4 * sizeof(int), AUDIO_BUFFER_ALIGNMENT);
		LowerDelayPos = (int*)FMemory::Malloc(4 * sizeof(int), AUDIO_BUFFER_ALIGNMENT);
	}


	// Destructor
	FLinearInterpFractionalDelay::~FLinearInterpFractionalDelay()
	{
		FMemory::Free(UpperDelayPos);
		FMemory::Free(LowerDelayPos);
	}


	// Resets the delay line state, flushes buffer and resets read/write pointers.
	void FLinearInterpFractionalDelay::Reset() 
	{
		DelayLine->ClearSamples();
		DelayLine->AddZeros(MaxDelay + 1);
	}


	void FLinearInterpFractionalDelay::ProcessAudio(const FAlignedFloatBuffer& InSamples, const FAlignedFloatBuffer& InDelays, FAlignedFloatBuffer& OutSamples)
	{
		const int32 InNum = InSamples.Num();
		checkf(InNum == InDelays.Num(), TEXT("Input buffers must be equal length"));

		// Prepare output buffer
		OutSamples.Reset(InNum);
		OutSamples.AddUninitialized(InNum);

		if (InNum != InDelays.Num())
		{
			// Return empty buffer on invalid input.
			if (InNum > 0)
			{
				FMemory::Memset(OutSamples.GetData(), 0, sizeof(float) * InNum);
			}
			return;
		}


		float* OutSampleData = OutSamples.GetData();
		const float* InSampleData = InSamples.GetData();
		const float* InDelayData = InDelays.GetData();

		// Process audio one block at a time.
		int32 LeftOver = InNum;
		int32 BufferPos = 0;
		while (LeftOver > 0)
		{
			int32 NumToProcess = FMath::Min(LeftOver, NumInternalBufferSamples);
			ProcessAudioBlock(&InSampleData[BufferPos], &InDelayData[BufferPos], NumToProcess, &OutSampleData[BufferPos]);
			BufferPos += NumToProcess;
			LeftOver -= NumToProcess;
		}
	}

	void FLinearInterpFractionalDelay::ProcessAudioBlock(const float* InSamples, const float* InDelays, const int32 InNum, float* OutSamples)
	{
		checkf(0 == (InNum % 4), TEXT("Array length must be multiple of 4"));

		// Update delay line.
		DelayLine->AddSamples(InSamples, InNum);

		const float* DelayData = DelayLine->InspectSamples(InNum + MaxDelay + 1);
		const int32* IntegerDelayOffsetData = IntegerDelayOffsets.GetData();

		ArrayLerpFractionalDelay(InSamples, InDelays, DelayData, IntegerDelayOffsetData, UpperDelayPos, LowerDelayPos, InNum, OutSamples, (float)MaxDelay);

		// Remove unneeded delay line.
		DelayLine->RemoveSamples(InNum);
	}
}
