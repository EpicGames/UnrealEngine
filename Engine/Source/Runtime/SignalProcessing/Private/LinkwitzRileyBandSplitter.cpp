// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LinkwitzRileyBandSplitter.h"
#include "DSP/FloatArrayMath.h"

#ifndef TWO_PI
#define TWO_PI (6.28318530718)
#endif
namespace Audio
{
	void FLinkwitzRileyBandSplitter::Init(const int32 InChannels, const float InSampleRate, const EFilterOrder InFilterOrder, const bool bInPhaseCompensate, const TArray<float>& InCrossovers)
	{
		NumBands = InCrossovers.Num() + 1;
		NumChannels = InChannels;

		if (NumBands <= 1)
		{
			return;
		}

		FilterOrder = InFilterOrder;
		SampleRate = InSampleRate;

		BandFilters.Reset(NumBands);
		BandFilters.AddDefaulted(NumBands);

		Crossovers.Reset(InCrossovers.Num());
		for (float Crossover : InCrossovers)
		{
			Crossovers.Add({ Crossover, GetBandwidthFromQ(GetQ(InFilterOrder)) });
		}

		// initalize each filter
		// each band gets an AP filter for each subsquent band after its HP/LP filters
		//	0	L	LA	LAA	LAAA etc
		//		H	HL	HLA	HLAA
		//			H	HL	HLA
		//				H	HL
		//					H

		BandFilters[0].Filters.AddDefaulted(NumBands - 1);

		for (int32 BandId = 1; BandId < NumBands; BandId++)
		{
			const int32 NumFilters = NumBands - BandId;
			BandFilters[BandId].Filters.AddDefaulted( bInPhaseCompensate ? NumFilters : FMath::Clamp(NumFilters, 1, 2) );
		}

		// band 0 special case
		BandFilters[0][0].Init(FilterOrder, SampleRate, NumChannels, Crossovers[0].Frequency, EBiquadFilter::ButterworthLowPass, Crossovers[0].Bandwidth);

		for (int32 FilterId = 1; FilterId < BandFilters[0].Filters.Num(); FilterId++)
		{
			BandFilters[0][FilterId].Init(FilterOrder, SampleRate, NumChannels, Crossovers[FilterId].Frequency, EBiquadFilter::AllPass, Crossovers[FilterId].Bandwidth);
		}

		// final band special case
		BandFilters[NumBands - 1][0].Init(FilterOrder, SampleRate, NumChannels, Crossovers[Crossovers.Num() - 1].Frequency, EBiquadFilter::ButterworthHighPass, Crossovers[Crossovers.Num() - 1].Bandwidth);

		if (NumBands <= 2)
		{
			return;
		}

		for (int32 BandId = 1; BandId < NumBands - 1; BandId++)
		{
			BandFilters[BandId][0].Init(FilterOrder, SampleRate, NumChannels, Crossovers[BandId - 1].Frequency, EBiquadFilter::ButterworthHighPass, Crossovers[BandId - 1].Bandwidth);
			BandFilters[BandId][1].Init(FilterOrder, SampleRate, NumChannels, Crossovers[BandId].Frequency, EBiquadFilter::ButterworthLowPass, Crossovers[BandId].Bandwidth);

			for (int32 FilterId = 2; FilterId < BandFilters[BandId].Filters.Num(); FilterId++)
			{
				const int32 CrossoverId = BandId + FilterId - 1; //band 1 crossovers are 0 - 1 - 2 - 4 etc, band 2 crossovers are 1 - 2 - 3 etc
				BandFilters[BandId][FilterId].Init(FilterOrder, SampleRate, NumChannels, Crossovers[CrossoverId].Frequency, EBiquadFilter::AllPass, Crossovers[CrossoverId].Bandwidth);
			}
		}
	}

	// copy in -> shared
	//
	// if not band 0 - process first filter pair (hp) in place to reuse for next buffer
	//
	// copy result to new work buffer to not affect next bands
	//
	// process filters after 0
	//
	// invert if Band % 2 && fourpole
	//
	// output
	void FLinkwitzRileyBandSplitter::ProcessAudioFrame(const float* InBuffer, FMultibandBuffer& OutBuffer)
	{
		if (NumBands <= 1)
		{
			// passthrough
			CopyToBuffer(OutBuffer[0], InBuffer, NumChannels);

			return;
		}

		SharedBuffer.Reset(NumChannels);
		SharedBuffer.AddZeroed(NumChannels);
		float* SharedBufferPtr = SharedBuffer.GetData();

		CopyToBuffer(SharedBufferPtr, InBuffer, NumChannels);

		BandWorkBuffer.Reset(NumChannels);
		BandWorkBuffer.AddZeroed(NumChannels);
		float* BandBufferPtr = BandWorkBuffer.GetData();

		for (int32 BandId = 0; BandId < NumBands; BandId++)
		{
			int32 FilterId = 0;

			// apply first filter before copying on bands > 0 so the filtered signal can be reused
			if (BandId > 0)
			{
				BandFilters[BandId][0].ProcessAudioFrame(SharedBufferPtr, SharedBufferPtr);
				FilterId++;
			}

			CopyToBuffer(BandBufferPtr, SharedBufferPtr, NumChannels);

			for (; FilterId < BandFilters[BandId].Filters.Num(); FilterId++)
			{
				BandFilters[BandId][FilterId].ProcessAudioFrame(BandBufferPtr, BandBufferPtr);
			}

			constexpr int32 IsOddBitMask = 0x00000001;
			if ((static_cast<int32>(FilterOrder) & IsOddBitMask) && (BandId & IsOddBitMask))
			{
				InvertBuffer(BandBufferPtr, NumChannels);
			}

			CopyToBuffer(OutBuffer[BandId], BandBufferPtr, NumChannels);
		}
	}

	void FLinkwitzRileyBandSplitter::ProcessAudioBuffer(const float* InBuffer, FMultibandBuffer& OutBuffer, const int32 NumFrames)
	{
		check(OutBuffer.NumBands > 0);
		check(OutBuffer.NumSamples >= NumFrames * NumChannels);

		const int32 NumSamples = NumChannels * NumFrames;

		if (NumBands <= 1)
		{
			// passthrough
			CopyToBuffer(OutBuffer[0], InBuffer, NumSamples);

			return;
		}

		SharedAlignedBuffer.SetNumZeroed(NumSamples, EAllowShrinking::No);
		BandAlignedBuffer.SetNumUninitialized(NumSamples, EAllowShrinking::No);

		float* const SharedBufferPtr = SharedAlignedBuffer.GetData();
		float* const BandBufferPtr = BandAlignedBuffer.GetData();

		CopyToBuffer(SharedBufferPtr, InBuffer, NumSamples);

		for (int32 BandId = 0; BandId < NumBands; BandId++)
		{
			int32 FilterId = 0;

			// apply first filter before copying on bands > 0 so the filtered signal can be reused
			if (BandId > 0)
			{
				BandFilters[BandId][0].ProcessAudioBuffer(SharedBufferPtr, SharedBufferPtr, NumFrames);
				FilterId++;
			}

			CopyToBuffer(BandBufferPtr, SharedBufferPtr, NumSamples);

			for (; FilterId < BandFilters[BandId].Filters.Num(); FilterId++)
			{
				BandFilters[BandId][FilterId].ProcessAudioBuffer(BandBufferPtr, BandBufferPtr, NumFrames);
			}

			constexpr int32 IsOddBitMask = 0x00000001;
			if ((static_cast<int32>(FilterOrder) & IsOddBitMask) && (BandId & IsOddBitMask))
			{
				TArrayView<float> BandBufferView(BandBufferPtr, NumSamples);
				ArrayMultiplyByConstantInPlace(BandBufferView, -1.f);
			}

			CopyToBuffer(OutBuffer[BandId], BandBufferPtr, NumSamples);
		}
	}


	void FLinkwitzRileyBandSplitter::SetCrossovers(const TArray<float>& InCrossoverFrequencies)
	{
		if (InCrossoverFrequencies.Num() != Crossovers.Num())
		{
			return;
		}

		Crossovers.Reset(InCrossoverFrequencies.Num());
		for (float Crossover : InCrossoverFrequencies)
		{
			Crossovers.Add({ Crossover, GetBandwidthFromQ(GetQ(FilterOrder)) });
		}

		// perform same loop over band filters as in init, but only call SetFrequency instead of initializing it
		// band 0
		BandFilters[0][0].SetParams(EBiquadFilter::ButterworthLowPass, Crossovers[0].Frequency, Crossovers[0].Bandwidth);

		for (int32 FilterId = 1; FilterId < BandFilters[0].Filters.Num(); FilterId++)
		{
			BandFilters[0][FilterId].SetParams(EBiquadFilter::AllPass, Crossovers[FilterId].Frequency, Crossovers[FilterId].Bandwidth);
		}

		// final band
		BandFilters[NumBands - 1][0].SetParams(EBiquadFilter::ButterworthHighPass, Crossovers[Crossovers.Num() - 1].Frequency, Crossovers[Crossovers.Num() - 1].Bandwidth);

		if (NumBands <= 2)
		{
			return;
		}

		// intermediate bands, if any
		for (int32 BandId = 1; BandId < NumBands - 1; BandId++)
		{
			BandFilters[BandId][0].SetParams(EBiquadFilter::ButterworthHighPass, Crossovers[BandId - 1].Frequency, Crossovers[BandId - 1].Bandwidth);
			BandFilters[BandId][1].SetParams(EBiquadFilter::ButterworthLowPass, Crossovers[BandId].Frequency, Crossovers[BandId].Bandwidth);

			for (int32 FilterId = 2; FilterId < BandFilters[BandId].Filters.Num(); FilterId++)
			{
				const int32 CrossoverId = BandId + FilterId - 1; //band 1 crossovers are 0 - 1 - 2 - 4 etc, band 2 crossovers are 1 - 2 - 3 etc
				BandFilters[BandId][FilterId].SetParams(EBiquadFilter::AllPass, Crossovers[CrossoverId].Frequency, Crossovers[CrossoverId].Bandwidth);
			}
		}
	}

	void FLinkwitzRileyBandSplitter::CopyToBuffer(float* Destination, const float* Origin, const int32 NumSamples)
	{
		FMemory::Memcpy(Destination, Origin, NumSamples * sizeof(float));
	}

	void FLinkwitzRileyBandSplitter::InvertBuffer(float* Buffer, const int32 NumSamples)
	{
		for (int32 Sample = 0; Sample < NumSamples; Sample++)
		{
			Buffer[Sample] *= -1.f;
		}
	}

	float FLinkwitzRileyBandSplitter::GetQ(EFilterOrder InFilterOrder)
	{
		/*
		* a two-pole filter naturally crosses its corner frequency at -3db with a Q of 1.
		* with a Q of -3db, that makes it -6db, which is our target for LR crossover filters
		* 
		* with stacked filters, that -3db point adds per filter, but the goal is still -6db,
		* so four-pole filters automatically reach the correct level at the crossover
		* 
		* passed that, the Q needs to raise the crossover point back to -6, but now it must be split over several stacked filters, so:
		* 
		* Q(db) = (3db * NumFilters - 6db) / NumFilters
		*/

		const int32 NumFilters = static_cast<int32>(InFilterOrder);

		if (ensure(NumFilters > 0) == false)
		{
			return 1.f;
		}

		const float Qdb = (3.f * NumFilters - 6.f) / NumFilters;

		return Audio::ConvertToLinear(Qdb);
	}
}
