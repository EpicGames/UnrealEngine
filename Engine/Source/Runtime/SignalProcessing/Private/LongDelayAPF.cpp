// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LongDelayAPF.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FLongDelayAPF::FLongDelayAPF(float InG, int32 InNumDelaySamples, int32 InMaxNumInternalBufferSamples)
	:	G(InG),
		NumDelaySamples(InNumDelaySamples),
		NumInternalBufferSamples(InMaxNumInternalBufferSamples)
	{
		// NumInternalBufferSamples must be less than or equal to the length of the delay
		if (NumInternalBufferSamples > NumDelaySamples)
		{
			NumInternalBufferSamples = NumDelaySamples;

			// Block size must be divisible by simd alignment to support simd operations.
			NumInternalBufferSamples -= (NumInternalBufferSamples % AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		}

		checkf(NumInternalBufferSamples > 1, TEXT("Invalid internal buffer length"));

		// Allocate delay line
		DelayLine = MakeUnique<FAlignedBlockBuffer>(2 * NumInternalBufferSamples + NumDelaySamples, NumInternalBufferSamples);
		DelayLine->AddZeros(NumDelaySamples);

		// Allocate internal buffer
		WorkBuffer.Reset(NumInternalBufferSamples);
		WorkBuffer.AddUninitialized(NumInternalBufferSamples);
	}

	FLongDelayAPF::~FLongDelayAPF()
	{}

	void FLongDelayAPF::ProcessAudio(FAlignedFloatBuffer& Samples)
	{
		const int32 InNum = Samples.Num();

		float* InData = Samples.GetData();
		float* OutDelayData = WorkBuffer.GetData();

		// Process audio one block at a time.
		int32 LeftOver = InNum;
		int32 BufferIndex = 0;
		while (LeftOver != 0)
		{
			// Determine block size for this loop.
			int32 NumToProcess = FMath::Min<int32>(NumInternalBufferSamples, LeftOver);
			const float* InDelayData = DelayLine->InspectSamples(NumToProcess);

			ProcessAudioBlock(&InData[BufferIndex], InDelayData, NumToProcess, &InData[BufferIndex], OutDelayData);

			// Update delay line with new data.
			DelayLine->RemoveSamples(NumToProcess);
			DelayLine->AddSamples(OutDelayData, NumToProcess);

			LeftOver -= NumToProcess;
			BufferIndex += NumToProcess;
		}
	}

	void FLongDelayAPF::ProcessAudio(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples)
	{
		const float* InData = InSamples.GetData();
		const int32 InNum = InSamples.Num();
		float* OutDelayData = WorkBuffer.GetData();

		// Resize output
		OutSamples.Reset(InNum);
		OutSamples.AddUninitialized(InNum);
		float* OutData = OutSamples.GetData();


		// Process audio one block at a time.
		int32 LeftOver = InNum;
		int32 BufferIndex = 0;
		while (LeftOver != 0) 
		{
			// Determine block size for this loop.
			int32 NumToProcess = FMath::Min<int32>(NumInternalBufferSamples, LeftOver);
			const float* InDelayData = DelayLine->InspectSamples(NumToProcess);

			ProcessAudioBlock(&InData[BufferIndex], InDelayData, NumToProcess, &OutData[BufferIndex], OutDelayData);

			// Update delay line with new data.
			DelayLine->RemoveSamples(NumToProcess);
			DelayLine->AddSamples(OutDelayData, NumToProcess);

			LeftOver -= NumToProcess;
			BufferIndex += NumToProcess;
		}
	}

	void FLongDelayAPF::ProcessAudio(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples, FAlignedFloatBuffer& OutDelaySamples)
	{
		const float* InData = InSamples.GetData();
		const int32 InNum = InSamples.Num();

		// Resize output buffers
		OutSamples.Reset(InNum);
		OutSamples.AddUninitialized(InNum);
		OutDelaySamples.Reset(InNum);
		OutDelaySamples.AddUninitialized(InNum);

		float* OutData = OutSamples.GetData();
		float* OutDelayData = OutDelaySamples.GetData();

		// Process audio one block at a time.
		int32 LeftOver = InNum;
		int32 BufferIndex = 0;
		while (LeftOver != 0) 
		{
			int32 NumToProcess = FMath::Min<int32>(NumInternalBufferSamples, LeftOver);
			const float* InDelayData = DelayLine->InspectSamples(NumToProcess);

			ProcessAudioBlock(&InData[BufferIndex], InDelayData, NumToProcess, &OutData[BufferIndex], &OutDelayData[BufferIndex]);
			
			// Update internal delay line.
			DelayLine->RemoveSamples(NumToProcess);
			DelayLine->AddSamples(&OutDelayData[BufferIndex], NumToProcess);

			LeftOver -= NumToProcess;
			BufferIndex += NumToProcess;
		}
	}

	void FLongDelayAPF::ProcessAudioBlock(const float* InSamples, const float* InDelaySamples, const int32 InNum, float* OutSamples, float* OutDelaySamples)
	{
		ArrayAPFLongDelayProcess(InSamples, InDelaySamples, InNum, OutSamples, OutDelaySamples, G);
	}

	void FLongDelayAPF::Reset() 
	{
		DelayLine->ClearSamples();
		DelayLine->AddZeros(NumDelaySamples);
	}

	int32 FLongDelayAPF::GetNumInternalBufferSamples() const
	{
		return NumInternalBufferSamples;
	}
}
