// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/LoudnessAnalyzer.h"

#include "AudioMixer.h"
#include "DSP/AudioChannelUtils.h"
#include "DSP/DeinterleaveView.h"
#include "DSP/FloatArrayMath.h"
#include "GenericPlatform/GenericPlatformMath.h"

namespace LoudnessAnalyzerPrivate
{
	const float Log10Scale = 1.0f / FMath::Loge(10.f);
}

namespace Audio
{
    /* Equal loudness curves from: ANSI Standards S1.4-1983 and S1.42-2001 */
    /* https://en.wikipedia.org/wiki/A-weighting */
    float GetEqualLoudnessAWeightForFrequency(const float Freq)
    {
        const float FreqSqrd = Freq * Freq;

        return 1.258925411794167f * ((148840000.f * FreqSqrd * FreqSqrd) / ((FreqSqrd + 424.36f) * (FreqSqrd + 148840000.f) * FMath::Sqrt((FreqSqrd + 11599.29f) * (FreqSqrd + 544496.41f))));
    }

    float GetEqualLoudnessBWeightForFrequency(const float Freq)
    {
        const float FreqSqrd = Freq * Freq;

        return 1.019764760044717f * ((148840000.f * Freq * FreqSqrd) / ((FreqSqrd + 424.36f) * (FreqSqrd + 148840000.f) * FMath::Sqrt(FreqSqrd + 25122.25f)));
    }

    float GetEqualLoudnessCWeightForFrequency(const float Freq)
    {
        const float FreqSqrd = Freq * Freq;

        return 1.006931668851804f * ((148840000.f * FreqSqrd) / ((FreqSqrd + 424.36f) * (FreqSqrd + 148840000.f)));
    }

    float GetEqualLoudnessDWeightForFrequency(const float Freq)
    {
        const float FreqSqrd = Freq * Freq;
        const float H1 = (1037918.48f - FreqSqrd);
        const float H2 = (9837328.f - FreqSqrd);
        const float H = (H1 + 1080768.16f * FreqSqrd) / (H2 + 11723776.f * FreqSqrd);
		return (Freq / 6.8966888496476e-5f) * FMath::Sqrt(H / ((FreqSqrd + 79919.29f) * (FreqSqrd + 1345600.f)));
    }

	float CalculatePolynomial(const double Freq, const double x0, const double x1, const double x2)
    {
	    const double CosFreq = FMath::Cos(Freq);
    	return x1 * x1 + (x0 - x2) * (x0 - x2) + 2.0 * x1 * (x0 + x2) * CosFreq + 4.0 * x0 * x2 * CosFreq * CosFreq;
    }

	float GetEqualLoudnessKWeightForFrequency(const float Freq)
    {
	    constexpr float OriginalSampleRateOfKFilter = 48000.f;
    	constexpr float Nyquist = OriginalSampleRateOfKFilter / 2.f;

    	if (Freq > Nyquist)
    	{
    		/**
    		 * 24kHz is not audible and does not contribute to perceived loudness.
    		 * This may deviate from other implementations because ITU-R BS.1770-5
    		 * does not prescribe how to handle higher sampling rates.
    		 */
    		return 0.f;
    	}
    	if (Freq < 0.f)
    	{
    		return 0.f;
    	}

    	const float NormalizedFrequency = Freq / OriginalSampleRateOfKFilter * 2.f * PI;

    	float PreFilterPower = 0.f;
	    {
	    	constexpr float a0 =  1.53512485958697f;
	    	constexpr float a1 = -2.69169618940638f;
	    	constexpr float a2 =  1.19839281085285f;
	    	constexpr float b0 =  1.f;
	    	constexpr float b1 = -1.69065929318241f;
	    	constexpr float b2 =  0.73248077421585f;

	    	const float Numerator = CalculatePolynomial(NormalizedFrequency, a0, a1, a2);
	    	const float Denominator = CalculatePolynomial(NormalizedFrequency, b0, b1, b2);

	    	PreFilterPower = Denominator != 0.f ? Numerator / Denominator : 0.f;
	    }

    	float RLBFilterPower = 0.f;
	    {
	    	constexpr float a0 =  1.f;
	    	constexpr float a1 = -2.f;
	    	constexpr float a2 =  1.f;
	    	constexpr float b0 =  1.f;
	    	constexpr float b1 = -1.99004745483398f;
	    	constexpr float b2 =  0.99007225036621f;

	    	const float Numerator = CalculatePolynomial(NormalizedFrequency, a0, a1, a2);
	    	const float Denominator = CalculatePolynomial(NormalizedFrequency, b0, b1, b2);

	    	RLBFilterPower = Denominator != 0.f ? Numerator / Denominator : 0.f;
	    }

    	// Convert power to magnitude
    	return FMath::Sqrt(FMath::Abs(PreFilterPower * RLBFilterPower));
    }

    float GetEqualLoudnessNoneWeightForFrequency(const float Freq)
    {
        return 1.0f;
    }
    

	/****************************************************************************/
	/***********************    FLoudnessAnalyzer    ****************************/
	/****************************************************************************/

    FLoudnessAnalyzer::FLoudnessAnalyzer(float InSampleRate, const FLoudnessAnalyzerSettings& InSettings)
    :   Settings(InSettings)
    {
		checkf(FMath::IsPowerOfTwo(Settings.FFTSize), TEXT("FFT size must be a power of two"));
		checkf(Settings.FFTSize >= 8, TEXT("FFT size must be atleast 8"))
        check(InSampleRate > 0);
		

		// Create FFT
		FFFTSettings FFTSettings;
		FFTSettings.Log2Size = CeilLog2(Settings.FFTSize);
		FFTSettings.bArrays128BitAligned = true;
		FFTSettings.bEnableHardwareAcceleration = true;

		ActualFFTSize = 1 << FFTSettings.Log2Size;

		checkf(FFFTFactory::AreFFTSettingsSupported(FFTSettings), TEXT("No fft algorithm supports fft settings."));
		FFT = FFFTFactory::NewFFTAlgorithm(FFTSettings);
		if (FFT.IsValid())
		{
			checkf(Settings.WindowSize <= ActualFFTSize, TEXT("WindowSize %d must be less than or equal to FFTSize %d"), Settings.WindowSize, ActualFFTSize);

			if (Settings.WindowSize <= 0)
			{
				Settings.WindowSize = ActualFFTSize;
			}
		
			// Initialize the analysis window
			Window.AddUninitialized(Settings.WindowSize);
			GenerateWindow(Window.GetData(), Settings.WindowType, Window.Num(), 1 /* NumChannels */, false /* bIsPeriodic */);
			
			// Size internal buffers
			WindowedSamples.AddZeroed(FFT->NumInputFloats()); // Zero samples to apply zero buffer in case actual FFT is larger than provided settings.
			ComplexSpectrum.AddUninitialized(FFT->NumOutputFloats());
			RealSpectrum.AddUninitialized(ComplexSpectrum.Num() / 2);

			FFTFreqSize = RealSpectrum.Num();

			// Determine which fft bins will be used for calculating loudness
			const float FreqPerBin = InSampleRate / FFT->NumOutputFloats();
			MinFreqIndex = static_cast<int32>(FMath::RoundHalfFromZero(Settings.MinAnalysisFrequency / FreqPerBin));
			MinFreqIndex = FMath::Max(0, MinFreqIndex);
			MaxFreqIndex = static_cast<int32>(FMath::RoundHalfFromZero(Settings.MaxAnalysisFrequency / FreqPerBin)) + 1;
			MaxFreqIndex = FMath::Min(MaxFreqIndex, FFTFreqSize);

			CurveWeights.Reset(FFTFreqSize);
			CurveWeights.AddUninitialized(FFTFreqSize);

			float* CurveWeightsPtr = CurveWeights.GetData();

			const float FFTSize = static_cast<float>(ActualFFTSize);

			for (int32 i = 0; i < FFTFreqSize; i++)
			{
				const float Freq = static_cast<float>(i) * InSampleRate / FFTSize;
				const float Weighting = GetEqualLoudnessWeightForFrequency(Freq);

				// Curve designed for magnitude domain, but applied to power spectrum. The curve is squared to be applied to power spectrum.
				CurveWeightsPtr[i] = Weighting * Weighting;
			}

			EnergyScale = CalculateEnergyScaling();
		}
    }

    const FLoudnessAnalyzerSettings& FLoudnessAnalyzer::GetSettings() const
    {
        return Settings;
    }

    float FLoudnessAnalyzer::CalculatePerceptualEnergy(TArrayView<const float> InView)
    {
		check(InView.Num() == Window.Num());

		float Total = 0.f;

		if (FFT.IsValid())
		{
			// Copy input samples and apply window
			ArrayMultiply(InView, Window, TArrayView<float>(WindowedSamples.GetData(), Window.Num()));

			FFT->ForwardRealToComplex(WindowedSamples.GetData(), ComplexSpectrum.GetData());

			ArrayComplexToPower(ComplexSpectrum, RealSpectrum);
		
			const float* PowerSpectrumPtr = RealSpectrum.GetData();
			const float* CurveWeightsPtr = CurveWeights.GetData();
			for (int32 i = MinFreqIndex; i < MaxFreqIndex; i++)
			{
				Total += PowerSpectrumPtr[i] * CurveWeightsPtr[i];
			}

			// Normalize by FFT Scaling and by number of samples to calculate mean squared value.
			Total *= EnergyScale;
		}

        return Total;
    }

    float FLoudnessAnalyzer::CalculateLoudness(TArrayView<const float> InView)
    {
        float Energy = CalculatePerceptualEnergy(InView);
		return ConvertPerceptualEnergyToLoudness(Energy);
    }

	float FLoudnessAnalyzer::ConvertPerceptualEnergyToLoudness(float InPerceptualEnergy) 
	{
        return 10.0f * FMath::Loge(InPerceptualEnergy) * LoudnessAnalyzerPrivate::Log10Scale;
	}

	float FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(float InPerceptualEnergy) 
	{
        return (10.0f * FMath::Loge(InPerceptualEnergy) * LoudnessAnalyzerPrivate::Log10Scale) - 0.691f;
	}

	float FLoudnessAnalyzer::GetEqualLoudnessWeightForFrequency(float InHz) const
	{
		switch (Settings.LoudnessCurveType)
		{
			case ELoudnessCurveType::A:
				return GetEqualLoudnessAWeightForFrequency(InHz);

			case ELoudnessCurveType::B:
				return GetEqualLoudnessBWeightForFrequency(InHz);

			case ELoudnessCurveType::C:
				return GetEqualLoudnessCWeightForFrequency(InHz);

			case ELoudnessCurveType::D:
				return GetEqualLoudnessDWeightForFrequency(InHz);

			case ELoudnessCurveType::K:
				return GetEqualLoudnessKWeightForFrequency(InHz);

			case ELoudnessCurveType::None:
				return GetEqualLoudnessNoneWeightForFrequency(InHz);

			default:
				{
					checkNoEntry();
				}
				return GetEqualLoudnessNoneWeightForFrequency(InHz);
		}
	}

	float FLoudnessAnalyzer::CalculateEnergyScaling() const
	{
		switch (Settings.ScalingMethod)
		{
			case ELoudnessAnalyzerScalingMethod::BackCompatible:
				return CalculateEnergyScalingBackCompatible();

			case ELoudnessAnalyzerScalingMethod::Corrected:
				return CalculateEnergyScalingCorrected();

			default:
				{
					checkNoEntry(); // Unimplemented normalization method
				}
				return CalculateEnergyScalingCorrected();
		};
	}

	float FLoudnessAnalyzer::CalculateEnergyScalingBackCompatible() const
	{
		check(FFT.IsValid()); // FFT must be initialized before calling this function 

		const float FFTScaling = GetPowerSpectrumScaling(FFT->Size(), FFT->ForwardScaling(), EFFTScaling::None);
		const float SizeScaling = 1.f / static_cast<float>(Settings.FFTSize);
		const float CurveAt1Khz = GetEqualLoudnessWeightForFrequency(1000.f);
		float CurveScaling = 1.f;

		if (CurveAt1Khz > 0.f)
		{
			CurveScaling = 1.f / CurveAt1Khz; // Normalize curve to be unity at 1khz
			CurveScaling = CurveScaling * CurveScaling; // Convert amplitude normalization to power normalization
		}

		return FFTScaling * SizeScaling * CurveScaling;
	}

	float FLoudnessAnalyzer::CalculateEnergyScalingCorrected() const
	{
		check(FFT.IsValid()); // FFT must be initialized before calling this function 

		constexpr float HalfSpectrumScaling = 2.f; // When accumulating spectral energy, we ignore the negative spectrum
												   // magnitudes because they are symmetric with positive spectrum
												   // magnitudes. This results in accumulation of only half of the 
												   // spectrum. This scaling corrects that by doubling the energy
												   // after it has been accumulated.
		const float FFTScaling = GetPowerSpectrumScaling(FFT->Size(), FFT->ForwardScaling(), EFFTScaling::None);
		const float WindowScaling = CalculateWindowEnergyScaling();

		return HalfSpectrumScaling * FFTScaling * WindowScaling;
	}

	float FLoudnessAnalyzer::CalculateWindowEnergyScaling() const
	{
		check(Window.Num() > 0); // Window must be initialized before calling this function.

		// Calculate energy of window
		float WindowEnergy = 0.f;
		for (float Val : Window)
		{
			WindowEnergy += (Val * Val);
		}

		// Avoid divide by zero on empty window func
		if (WindowEnergy == 0.f)
		{
			return 1.f;
		}
		else
		{
			return 1.f / WindowEnergy;
		}
	}

	/****************************************************************************/
	/****************    FMultichannelLoudnessAnalyzer    ***********************/
	/****************************************************************************/

    FMultichannelLoudnessAnalyzer::FMultichannelLoudnessAnalyzer(float InSampleRate, const FLoudnessAnalyzerSettings& InSettings)
    :   Analyzer(InSampleRate, InSettings)
    {
        MonoBuffer.AddUninitialized(Analyzer.GetSettings().WindowSize);
    }

    float FMultichannelLoudnessAnalyzer::CalculatePerceptualEnergy(TArrayView<const float> InView, const int32 InNumChannels, TArray<float>& OutChannelEnergies)
    {
		check(InView.Num() == (InNumChannels * Analyzer.GetSettings().WindowSize));

        OutChannelEnergies.Reset(InNumChannels);
        OutChannelEnergies.AddUninitialized(InNumChannels);
        return CalculatePerceptualEnergy(InView, InNumChannels, TArrayView<float>(OutChannelEnergies));
    }

    float FMultichannelLoudnessAnalyzer::CalculatePerceptualEnergy(TArrayView<const float> InView, const int32 InNumChannels, TArrayView<float> OutChannelEnergies)
    {
		check(InView.Num() == (InNumChannels * Analyzer.GetSettings().WindowSize));
		check(OutChannelEnergies.Num() == InNumChannels);
		if (ChannelWeights.Num() != InNumChannels)
		{
			InitializeChannelWeights(InNumChannels);
		}

        // Deinterleave audio and send to mono loudness analyzer.
		TAutoDeinterleaveView<float, FAudioBufferAlignedAllocator> DeinterleaveView(InView, MonoBuffer, InNumChannels);
		for (TAutoDeinterleaveView<float>::TChannel<FAudioBufferAlignedAllocator> Channel : DeinterleaveView)
		{
            OutChannelEnergies[Channel.ChannelIndex] = Analyzer.CalculatePerceptualEnergy(Channel.Values);
		}
				
        // Combine channel energies into overall perceptual energy
        float Energy = 0.0f;
        for (int32 ChannelNum = 0; ChannelNum < InNumChannels; ChannelNum++)
        {
            Energy += ChannelWeights[ChannelNum] * OutChannelEnergies[ChannelNum];
        }

        return Energy;
    }

    float FMultichannelLoudnessAnalyzer::CalculateLoudness(TArrayView<const float> InView, const int32 InNumChannels, TArray<float>& OutChannelLoudness)
    {
		OutChannelLoudness.Reset();
		OutChannelLoudness.AddUninitialized(InNumChannels);
		return CalculateLoudness(InView, InNumChannels, TArrayView<float>(OutChannelLoudness));
    }

    float FMultichannelLoudnessAnalyzer::CalculateLoudness(TArrayView<const float> InView, const int32 InNumChannels, TArrayView<float> OutChannelLoudness)
    {
        // Convert perceptual energy to loudness 
        float Energy = CalculatePerceptualEnergy(InView, InNumChannels, OutChannelLoudness);
        for (float& Value : OutChannelLoudness)
        {
            Value = FLoudnessAnalyzer::ConvertPerceptualEnergyToLoudness(Value);
        }
        return FLoudnessAnalyzer::ConvertPerceptualEnergyToLoudness(Energy);
    }

	void FMultichannelLoudnessAnalyzer::InitializeChannelWeights(int32 InNumChannels)
	{
		ChannelWeights.SetNum(InNumChannels);
		for (int32 ChannelIdx = 0; ChannelIdx < InNumChannels; ChannelIdx++)
		{
			EAudioMixerChannel::Type ChannelType = EAudioMixerChannel::DefaultChannel;

			if (ensure(ChannelUtils::GetChannelTypeAtIndex(ChannelIdx, ChannelType)))
			{
				ChannelWeights[ChannelIdx] = ChannelUtils::GetChannelWeight(ChannelType);
			}
			else
			{
				ChannelWeights[ChannelIdx] = 1.f;
			}
		}
	}

    const FLoudnessAnalyzerSettings& FMultichannelLoudnessAnalyzer::GetSettings() const
    {
        return Analyzer.GetSettings();
    }

}
