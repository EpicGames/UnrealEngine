// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/MelScale.h"
#include "CoreMinimal.h"
#include "DSP/FloatArrayMath.h"

namespace
{
	// Fill an array with a triangle. Slopes do not need to match.
	void FillArrayWithTriangle(const float InLowerIndex, const float InCenterIndex, const float InUpperIndex, int32& OutOffsetIndex, TArray<float>& OutArray)
	{
		const int32 MinIndex = FMath::Max(0, FMath::CeilToInt(InLowerIndex)); 
		const int32 MaxIndex = FMath::FloorToInt(InUpperIndex);
		const int32 Num = FMath::Max(0, MaxIndex - MinIndex + 1);

		// Prepare output 
		OutOffsetIndex = MinIndex;
		OutArray.Reset(Num);
		if (Num < 1)
		{
			return;
		}

		OutArray.AddUninitialized(Num);
		FMemory::Memset(OutArray.GetData(), 0, sizeof(float) * Num);

		// determin upper and lower scales for triangle
		const float LowerScale = 1.f / FMath::Max(SMALL_NUMBER, InCenterIndex - InLowerIndex);
		const float UpperScale = 1.f / FMath::Max(SMALL_NUMBER, InUpperIndex - InCenterIndex);

		// Build traingle windows for each row
		float* OutArrayData = OutArray.GetData();
		for (int32 OffsetIndex = 0; OffsetIndex < Num; OffsetIndex++)
		{
			const float ActualIndex = static_cast<float>(MinIndex + OffsetIndex);
			float Value = 0.0f;
			if (ActualIndex < InCenterIndex)
			{
				Value = (ActualIndex - InLowerIndex) * LowerScale;
			}
			else
			{
				Value = (InUpperIndex - ActualIndex) * UpperScale;
			}

			OutArrayData[OffsetIndex] = Value < 0.0f ? 0.0f : Value;
		}
	}
}

namespace Audio
{
	float MelToHz(float InMel) {
		return 700.f * (FMath::Exp(InMel / 1127.f) - 1.f);
	}

	float HzToMel(float InHz) {
		return 1127.f * FMath::Loge(InHz / 700.f + 1.f);
	}

	TUniquePtr<FContiguousSparse2DKernelTransform> NewMelSpectrumKernelTransform(const FMelSpectrumKernelSettings& InSettings, const int32 InFFTSize, const float InSampleRate)
	{
		// Convert frequency bounds to mel
		float MinMel = HzToMel(InSettings.KernelMinCenterFreq);
		float MaxMel = HzToMel(InSettings.KernelMaxCenterFreq);
		float DeltaMel = (MaxMel - MinMel) / FMath::Max(1, (InSettings.NumBands - 1));

		// Only need to look at half of spectrum because magnitudes are mirrored.
		const int32 NumUsefulFrequencyBins = InFFTSize / 2 + 1;
		float BinsPerHz = (float)InFFTSize / (float)InSampleRate;
		float HzPerBin = (float)InSampleRate / (float)InFFTSize;

		// Used to define bounds of mel band windows.
		float PreviousMelCenter = MinMel - DeltaMel;
		float CurrentMelCenter = MinMel;
		float NextMelCenter = MinMel + DeltaMel;

		// Mel band windows are contiguous and sparse, so use this transform to make FFT->Mel fast.
		TUniquePtr<FContiguousSparse2DKernelTransform> KernelTransform = MakeUnique<FContiguousSparse2DKernelTransform>(NumUsefulFrequencyBins, InSettings.NumBands);

		// Create mel band windows one by one.
		for (int32 MelBandIndex = 0; MelBandIndex < InSettings.NumBands; MelBandIndex++)
		{
			// Convert Mels to Hz
			float LowerBandFreq = MelToHz(PreviousMelCenter);
			float UpperBandFreq = MelToHz(NextMelCenter);
			float CenterBandFreq = MelToHz(CurrentMelCenter);

			//  Stretch frequency bandwidth if necessary
			if (InSettings.BandWidthStretch != 1.0f)
			{
				LowerBandFreq = CenterBandFreq + InSettings.BandWidthStretch * (LowerBandFreq - CenterBandFreq);
				UpperBandFreq = CenterBandFreq + InSettings.BandWidthStretch * (UpperBandFreq - CenterBandFreq);
			}

			// Determine bounding bins
			float LowerFFTBin = LowerBandFreq * BinsPerHz;
			float UpperFFTBin = UpperBandFreq * BinsPerHz;
			float CenterFFTBin = CenterBandFreq * BinsPerHz;

			// Create a triangle in frequency domain to account for mel band.
			TArray<float> RowOffsetValues;
			int32 RowOffsetIndex = 0;
			FillArrayWithTriangle(LowerFFTBin, CenterFFTBin, UpperFFTBin, RowOffsetIndex, RowOffsetValues);

			// By default, FillArrayWithTriangle creates constant amplitude triangle. So only noralize 
			// if constant energy is chosen.
			float NormDenom = 1.f;
			switch (InSettings.Normalization)
			{
				case EMelNormalization::EqualAmplitude:
					// By default, FillArrayWithTriangle creates constant amplitude triangle. So only noralize 
					// if constant energy is chosen.
					NormDenom = 1.f;
					break;

				case EMelNormalization::EqualEuclideanNorm:
					NormDenom = FMath::Sqrt(0.5f * (UpperFFTBin - LowerFFTBin));
					break;

				case EMelNormalization::EqualEnergy:
				default:
					// weight bands to have constant energy of 1
					NormDenom = 0.5f * (UpperFFTBin - LowerFFTBin);
					break;
			}

			if ((NormDenom > 0.f) && (NormDenom != 1.f))
			{
				ArrayMultiplyByConstantInPlace(RowOffsetValues, 1.f / NormDenom);
			}

			// Check bounds of data being input into kernel
			int32 MaxNumRowOFfsetValues = FMath::Max(0, NumUsefulFrequencyBins - RowOffsetIndex);
			if (RowOffsetValues.Num() > MaxNumRowOFfsetValues)
			{
				RowOffsetValues.SetNum(MaxNumRowOFfsetValues);
			}
			if (RowOffsetIndex > NumUsefulFrequencyBins)
			{
				RowOffsetIndex = 0;
			}

			// Add row to kernel
			KernelTransform->SetRow(MelBandIndex, RowOffsetIndex, TArrayView<const float>(RowOffsetValues));

			// increment mel band counters 
			PreviousMelCenter = CurrentMelCenter;
			CurrentMelCenter = NextMelCenter;
			NextMelCenter += DeltaMel;
		}

		return KernelTransform;
	}
}
