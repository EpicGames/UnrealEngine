// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/MidiNoteQuantizer.h"
#include "Math/NumericLimits.h"

namespace Audio
{
	namespace Note
	{
		static const float B1 = 11.0f;
		static const float Bb1 = 10.0f;
		static const float A1 = 9.0f;
		static const float Ab1 = 8.0f;
		static const float G1 = 7.0f;
		static const float Gb1 = 6.0f;
		static const float F1 = 5.0f;
		static const float E1 = 4.0f;
		static const float Eb1 = 3.0f;
		static const float D1 = 2.0f;
		static const float Db1 = 1.0f;
		static const float C1 = 0.0f;
	}

	ScaleDegreeSet::ScaleDegreeSet(const TArray<float>& InScaleDegrees, const TArray<float>& InChordTones)
		: ScaleDegrees(InScaleDegrees)
		, ChordTones(InChordTones)
	{
	}

	TArrayView<float> ScaleDegreeSet::GetScaleDegreeSet(bool bChordTonesOnlyIfApplicable)
	{
		if (bChordTonesOnlyIfApplicable && ChordTones.Num())
		{
			return ChordTones;
		}

		return ScaleDegrees;
	}

	float FMidiNoteQuantizer::QuantizeMidiNote(const float InNote, const float InRoot, EMusicalScale::Scale InScale, bool bChordTonesOnlyIfApplicable)
	{
		return QuantizeMidiNote(InNote, InRoot, ScaleDegreeSetMap[InScale].GetScaleDegreeSet(bChordTonesOnlyIfApplicable));
	}

	float FMidiNoteQuantizer::QuantizeMidiNote(const float InNote, const float InRoot, const TArrayView<float> InScaleDegrees, const float InSemitoneScaleRange)
	{
		// QuantizeValueToScaleDegree() works within a single octave, so map incoming note to within the first octave
		const float NoteRootDelta = (InNote - InRoot);
		const float ClampedSemitoneScaleRange = FMath::Max(InSemitoneScaleRange, 1.0f);
		const float InNoteOctave = FMath::FloorToFloat(NoteRootDelta / ClampedSemitoneScaleRange);
		const float ValueToQuant = NoteRootDelta - (InSemitoneScaleRange * InNoteOctave);

		const float QuantizedValue = QuantizeValueToScaleDegree(ValueToQuant, InScaleDegrees);

		// reconstruct the quantized midi note given the InRoot and original octave
		return (InSemitoneScaleRange * InNoteOctave + QuantizedValue) + InRoot;
	}

	float FMidiNoteQuantizer::QuantizeValueToScaleDegree(const float InValue, const TArrayView<float> InScaleDegrees, const float InSemitoneScaleRange)
	{
		// the Set we are quantizing to should have at least 2 elements
		if (!InScaleDegrees.Num())
		{
			return InValue;
		}

		// First check the min delta against the scale range
		float QuantizedValue = InSemitoneScaleRange;
		float CurrMinDelta = FMath::Abs(InValue - InSemitoneScaleRange);

		// Then check the given scale degree array
		for (int32 i = 0; i < InScaleDegrees.Num(); ++i)
		{
			float NewDelta = FMath::Abs(InValue - InScaleDegrees[i]);
			if (NewDelta < CurrMinDelta)
			{
				CurrMinDelta = NewDelta;
				QuantizedValue = InScaleDegrees[i];
			}
		}

		return QuantizedValue;
	}


	// Initialize our note & chord arrays

	/* REQUIREMENTS:
	*	The (optional) second array defines Chord Tones, if empty, the first array will be used always
	*		- this is ideally a subset of the first array (user-facing UI/UX will imply this), but does not technically need to be and is not enforced.
	*
	*	The first array MUST have at least 1 entry or it will ensure.
	*
	*	Both array MUST be in ascending order, or quantization behavior will be broken. This is not enforced at runtime and quantization and will not warn/ensure
	*
	*	The Arrays must represent a full octave (inclusive): the lowest element being (0.0 and the highest being 12.0)
	*/
	TMap<EMusicalScale::Scale, ScaleDegreeSet> FMidiNoteQuantizer::ScaleDegreeSetMap{
		/* entry example:
		* 	{ EMusicalScale::Scale::[SCALE NAME],
			{{ [(required) SCALE DEGREES] }, { [(optional) CHORD TONES] }}}
		*/
		  { EMusicalScale::Scale::Major,
			{{ Note::C1, Note::D1, Note::E1, Note::F1, Note::G1, Note::A1, Note::B1 }, { Note::C1, Note::E1, Note::G1, Note::B1 }}}
		, { EMusicalScale::Scale::Dominant7th_Mixolydian,
			{{ Note::C1, Note::D1, Note::E1, Note::F1, Note::G1, Note::A1, Note::Bb1 }, { Note::C1, Note::E1, Note::G1, Note::Bb1 }}}
		, { EMusicalScale::Scale::Minor_Dorian,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::G1, Note::A1, Note::Bb1 }, { Note::C1, Note::Eb1, Note::G1, Note::Bb1 }}}
		, { EMusicalScale::Scale::HalfDiminished_Locrian,
			{{ Note::C1, Note::Db1, Note::Eb1, Note::F1, Note::Gb1, Note::Ab1, Note::Bb1 }, { Note::C1, Note::Eb1, Note::Gb1, Note::Bb1 }}}
		, { EMusicalScale::Scale::Diminished,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::Gb1, Note::Ab1, Note::A1, Note::B1 }, { Note::C1, Note::Eb1, Note::Gb1, Note::A1 }}}
		, { EMusicalScale::Scale::MajorPentatonic,
			{{ Note::C1, Note::D1, Note::E1, Note::G1, Note::A1}, { Note::C1, Note::E1, Note::G1 }}}
		, { EMusicalScale::Scale::Lydian,
			{{ Note::C1, Note::D1, Note::E1, Note::Gb1, Note::G1, Note::A1, Note::B1}, { Note::C1, Note::E1, Note::G1, Note::B1}}}
		, { EMusicalScale::Scale::Bebop_Major,
			{{ Note::C1, Note::D1, Note::E1, Note::Gb1, Note::G1, Note::Ab1, Note::A1, Note::B1}, { Note::C1, Note::E1, Note::G1, Note::B1}}}
		, { EMusicalScale::Scale::HarmonicMajor,
			{{ Note::C1, Note::D1, Note::E1, Note::F1, Note::G1, Note::Ab1, Note::B1}, { Note::C1, Note::E1, Note::G1, Note::B1}}}
		, { EMusicalScale::Scale::LydianAugmented,
			{{ Note::C1, Note::D1, Note::E1, Note::Gb1, Note::Ab1, Note::A1, Note::B1}, { Note::C1, Note::E1, Note::Ab1, Note::B1}}}
		, { EMusicalScale::Scale::Augmented,
			{{ Note::C1, Note::Eb1, Note::E1, Note::G1, Note::Ab1, Note::B1}, { Note::C1, Note::E1, Note::Ab1, Note::B1}}}
		, { EMusicalScale::Scale::SixthModeOfHarmonicMinor,
			{{ Note::C1, Note::Eb1, Note::E1, Note::Gb1, Note::G1, Note::A1, Note::B1}, { Note::C1, Note::E1, Note::G1, Note::B1}}}
		, { EMusicalScale::Scale::Diminished_BeginWithHalfStep,
			{{ Note::C1, Note::Db1, Note::Eb1, Note::E1, Note::Gb1, Note::G1, Note::A1, Note::Bb1}, { Note::C1, Note::E1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::Blues,
			{{ Note::C1, Note::Eb1, Note::F1, Note::Gb1, Note::G1, Note::Bb1}, { Note::C1, Note::Eb1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::Bebop_Dominant,
			{{ Note::C1, Note::D1, Note::E1, Note::F1, Note::G1, Note::A1, Note::Bb1, Note::B1}, { Note::C1, Note::E1, Note::G1, Note::B1}}}
		, { EMusicalScale::Scale::Spanish_or_Jewish,
			{{ Note::C1, Note::Db1, Note::E1, Note::F1, Note::G1, Note::Ab1, Note::Bb1}, { Note::C1, Note::E1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::LydianDominant,
			{{ Note::C1, Note::D1, Note::E1, Note::Gb1, Note::G1, Note::A1, Note::Bb1}, { Note::C1, Note::E1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::Hindu,
			{{ Note::C1, Note::D1, Note::E1, Note::F1, Note::G1, Note::Ab1, Note::Bb1}, { Note::C1, Note::E1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::WholeTone,
			{{ Note::C1, Note::D1, Note::E1, Note::Gb1, Note::Ab1, Note::Bb1}, { Note::C1, Note::E1, Note::Gb1, Note::Bb1}}}
		, { EMusicalScale::Scale::Chromatic,
			{{ Note::C1, Note::Db1, Note::D1, Note::Eb1, Note::E1, Note::F1, Note::Gb1, Note::G1, Note::Ab1, Note::A1, Note::Bb1, Note::B1}, {Note::C1, Note::D1, Note::E1, Note::Gb1, Note::Ab1, Note::Bb1 /* Every other note */}}}
		, { EMusicalScale::Scale::DiminishedWholeTone,
			{{ Note::C1, Note::Db1, Note::Eb1, Note::E1, Note::Gb1, Note::Ab1, Note::Bb1}, { Note::C1, Note::Eb1, Note::Gb1, Note::Bb1}}}
		, { EMusicalScale::Scale::MinorPentatonic,
			{{ Note::C1, Note::Eb1, Note::F1, Note::G1, Note::Bb1}, { Note::C1, Note::Eb1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::Bebop_Minor,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::G1, Note::Ab1, Note::A1, Note::B1}, { Note::C1, Note::Eb1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::MelodicMinor,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::G1, Note::A1, Note::B1}, { Note::C1, Note::Eb1, Note::G1, Note::B1}}}
		, { EMusicalScale::Scale::Bebop_MinorNumber2,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::G1, Note::Ab1, Note::A1, Note::B1}, { Note::C1, Note::Eb1, Note::G1, Note::B1}}}
		, { EMusicalScale::Scale::HarmonicMinor,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::G1, Note::Ab1, Note::B1}, { Note::C1, Note::Eb1, Note::G1, Note::B1}}}
		, { EMusicalScale::Scale::Diminished_BeginWithWholeStep,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::Gb1, Note::Ab1, Note::A1, Note::B1}, { Note::C1, Note::Eb1, Note::Gb1, Note::B1}}}
		, { EMusicalScale::Scale::Phrygian,
			{{ Note::C1, Note::Db1, Note::Eb1, Note::F1, Note::G1, Note::Ab1, Note::Bb1}, { Note::C1, Note::Eb1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::NaturalMinor_Aeolian,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::G1, Note::Ab1, Note::Bb1}, { Note::C1, Note::Eb1, Note::G1, Note::Bb1}}}
		, { EMusicalScale::Scale::HalfDiminished_LocrianNumber2,
			{{ Note::C1, Note::D1, Note::Eb1, Note::F1, Note::Gb1, Note::Ab1, Note::Bb1}, { Note::C1, Note::Eb1, Note::Gb1, Note::Bb1}}}
	};
} // namespace Audio
