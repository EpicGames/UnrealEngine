// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/ModulationMatrix.h"

namespace Audio
{
	FModulationMatrix::FModulationMatrix()
		: NumVoices(0)
	{
	}

	FModulationMatrix::~FModulationMatrix()
	{
	}

	void FModulationMatrix::Init(const int32 InNumVoices)
	{
		NumVoices = InNumVoices;
		for (int32 i = 0; i < NumVoices; ++i)
		{
			Patches.Add(TArray<FPatch*>());
			Sources.Add(TArray<float>());
			Destinations.Add(TArray<FDestData>());
		}
	}

	int32 FModulationMatrix::GetNumPatches(const int32 VoiceId) const
	{
		check(VoiceId < NumVoices);
		return Patches[VoiceId].Num();
	}

	FPatchSource FModulationMatrix::CreatePatchSource(const int32 VoiceId)
	{
		check(VoiceId < NumVoices);
		FPatchSource NewSource = Sources[VoiceId].Num();
		Sources[VoiceId].Add(0.0f);
		return NewSource;
	}

	FPatchDestination FModulationMatrix::CreatePatchDestination(const int32 VoiceId, const int32 Stage, const float DefaultDepth)
	{
		check(VoiceId < NumVoices);
		FPatchDestination NewDestination = Destinations[VoiceId].Num();
		NewDestination.Depth = DefaultDepth;
		NewDestination.Stage = Stage;
		Destinations[VoiceId].Add(FDestData());
		return NewDestination;
	}

	bool FModulationMatrix::ValidatePatch(const int32 VoiceId, FPatch* InPatch)
	{
		// Validate that the patch has valid source/destinations
		const FPatchSource& PatchSource = InPatch->Source;
		if (PatchSource.Id >= (uint32)Sources[VoiceId].Num())
		{
			return false;
		}

		for (int32 i = 0; i < InPatch->Destinations.Num(); ++i)
		{
			const FPatchDestination& PatchDest = InPatch->Destinations[i];
			if (PatchDest.Id >= (uint32)Destinations[VoiceId].Num())
			{
				return false;
			}
		}
		return true;
	}

	bool FModulationMatrix::AddPatch(const int32 VoiceId, FPatch* InPatch)
	{
		check(VoiceId < NumVoices);
		if (!ValidatePatch(VoiceId, InPatch))
		{
			return false;
		}

		Patches[VoiceId].Add(InPatch);
		return true;
	}

	bool FModulationMatrix::RemovePatch(const int32 VoiceId, FPatch* InPatch)
	{
		check(VoiceId < NumVoices);
		if (!ValidatePatch(VoiceId, InPatch))
		{
			return false;
		}

		//reset destination values
		for (int32 PatchDest = 0; PatchDest < InPatch->Destinations.Num(); ++PatchDest)
		{
			const FPatchDestination& Destination = InPatch->Destinations[PatchDest];
			Destinations[VoiceId][Destination.Id].bDirty = false;
			Destinations[VoiceId][Destination.Id].Value = 0.0f;
		}

		Patches[VoiceId].Remove(InPatch);
		return true;
	}

	void FModulationMatrix::ResetPatchSourceState()
	{
		for (TArray<float>& SourceVoiceStates : Sources)
		{
			for (float& Value : SourceVoiceStates)
			{
				Value = 0.0f;
			}
		}
	}

	void FModulationMatrix::ClearPatches(const int32 VoiceId)
	{
		check(VoiceId < NumVoices);

		for (FDestData& Destination : Destinations[VoiceId]) 
		{

			Destination.bDirty = false;
			Destination.Value  = 0.0f;
		}

		Patches[VoiceId].Reset();
	}

	bool FModulationMatrix::SetSourceValue(const int32 VoiceId, const FPatchSource& Source, const float Value)
	{
		check(VoiceId < NumVoices);
		if (Source.Id < (uint32)Sources[VoiceId].Num())
		{
			Sources[VoiceId][Source.Id] = Value;
			return true;
		}
		return false;
	}

	bool FModulationMatrix::GetDestinationValue(const int32 VoiceId, const FPatchDestination& Destination, float& OutValue) const
	{
		check(VoiceId < NumVoices);
		if (Destination.Id < (uint32)Destinations[VoiceId].Num())
		{
			const FDestData& Data = Destinations[VoiceId][Destination.Id];
			if (Data.bDirty)
			{
				OutValue = Data.Value;
				return true;
			}
		}
		return false;
	}

	void FModulationMatrix::ResetDestinations(const int32 VoiceId)
	{
		check(VoiceId < NumVoices);
		for (int32 i = 0; i < Patches[VoiceId].Num(); ++i)
		{
			FPatch* Patch = Patches[VoiceId][i];
			for (int32 PatchDest = 0; PatchDest < Patch->Destinations.Num(); ++PatchDest)
			{
				const FPatchDestination& Destination = Patch->Destinations[PatchDest];
				Destinations[VoiceId][Destination.Id].bDirty = false;
				Destinations[VoiceId][Destination.Id].Value = 0.0f;
			}
		}
	}

	void FModulationMatrix::Update(const int32 VoiceId, const int32 Stage)
	{
		// Make sure we clear out all the data in the matrix destinations since we're
		// about to mix new destination data into it.
		ResetDestinations(VoiceId);

		for (int32 i = 0; i < Patches[VoiceId].Num(); ++i)
		{
			FPatch * Patch = Patches[VoiceId][i];

			// Only perform the patch if its enabled
			if (!Patch->bEnabled)
			{
				continue;
			}

			// Check the desired stage (if set)
			if (Stage == INDEX_NONE)
			{
				continue;
			}

			// Loop through the patches destinations and "mix" in the transformed mod source's value.
			// Note that if multiple patches write to the same destination (s), they 
			// will accumulate the mod value.

			// This is the value written by the source			
			const float ModValue = Sources[VoiceId][Patch->Source.Id];

			for (int32 DestIndex = 0; DestIndex < Patch->Destinations.Num(); ++DestIndex)
			{
				const FPatchDestination& Dest = Patch->Destinations[DestIndex];

				if (Dest.Stage == Stage)
				{
					// Perform a scale-add operation using the modulation range
					float ModValueScaled = ModValue * Dest.Depth;

					Destinations[VoiceId][Dest.Id].Value += ModValueScaled;
					Destinations[VoiceId][Dest.Id].bDirty = true;
				}
			}
		}
	}
}
