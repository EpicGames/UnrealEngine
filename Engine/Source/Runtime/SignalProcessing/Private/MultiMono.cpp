// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/MultiMono.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	void MultiMonoMixUpOrDown(
		TArrayView<const float> InSrc, const TArrayView<float> InDst, const int32 NumFrames, TArrayView<const float> MixGains,
			const int32 NumSrcChannels, const int32 NumDstChannels)
	{
		MultiMonoMixUpOrDown(
			MakeMultiMonoPointersFromView(InSrc, NumFrames, NumSrcChannels),
			MakeMultiMonoPointersFromView(InDst, NumFrames, NumDstChannels),
			NumFrames, 
			MixGains);
	}

	void MultiMonoMixUpOrDown(TArrayView<const float*> InSrc, TArrayView<float*> InDst, const int32 NumFrames,TArrayView<const float> MixGains)
	{
		const int32 NumSrcChannels = InSrc.Num();
		const int32 NumDstChannels = InDst.Num();
		checkSlow(NumFrames > 0);
		checkSlow(MixGains.Num() == NumSrcChannels * NumDstChannels);
		
		for (int32 DstCh = 0; DstCh < NumDstChannels; ++DstCh)
		{
			const TArrayView<float> Dst = MakeArrayView(InDst[DstCh], NumFrames);
			FMemory::Memzero(Dst.GetData(), NumFrames * sizeof(float));
			for (int32 SrcCh = 0; SrcCh < NumSrcChannels; ++SrcCh)
			{
				if (const float ChannelGain = MixGains[DstCh + (SrcCh * NumDstChannels)]; !FMath::IsNearlyZero(ChannelGain))
				{
					const TArrayView<const float> Src = MakeArrayView(InSrc[SrcCh], NumFrames);
					ArrayMixIn(Src, Dst, ChannelGain);
				}
			}
		}	
	}
}
