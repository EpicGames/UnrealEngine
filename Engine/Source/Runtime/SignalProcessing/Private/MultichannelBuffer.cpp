// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/MultichannelBuffer.h"

#include "Algo/AllOf.h"
#include "Containers/Array.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"

namespace Audio
{
	namespace MultichannelBufferPrivate
	{
		template<typename BufferType>
		int32 GetMultichannelBufferNumFrames(const TArray<BufferType>& InBuffer)
		{
			if (InBuffer.Num() > 0)
			{
				const int32 NumFrames = InBuffer[0].Num();
				
				// Check that all of the buffers contain the same frame count.
				checkf(Algo::AllOf(InBuffer, [&NumFrames](const BufferType& Buffer) { return NumFrames == Buffer.Num(); }), TEXT("Buffers contain inconsistent frame sizes"));

				return NumFrames;
			}

			return 0;
		}
	}

	void SetMultichannelBufferSize(int32 InNumChannels, int32 InNumFrames, FMultichannelBuffer& OutBuffer)
	{
		OutBuffer.SetNum(InNumChannels, EAllowShrinking::No);
		for (FAlignedFloatBuffer& Buffer : OutBuffer)
		{
			Buffer.SetNumUninitialized(InNumFrames, EAllowShrinking::No);
		}
	}

	void SetMultichannelCircularBufferCapacity(int32 InNumChannels, int32 InNumFrames, FMultichannelCircularBuffer& OutBuffer)
	{
		OutBuffer.SetNum(InNumChannels, EAllowShrinking::No);
		for (TCircularAudioBuffer<float>& Buffer : OutBuffer)
		{
			Buffer.SetCapacity(InNumFrames);
		}
	}

	int32 GetMultichannelBufferNumFrames(const FMultichannelBuffer& InBuffer)
	{
		return MultichannelBufferPrivate::GetMultichannelBufferNumFrames(InBuffer);
	}

	int32 GetMultichannelBufferNumFrames(const FMultichannelCircularBuffer& InBuffer)
	{
		return MultichannelBufferPrivate::GetMultichannelBufferNumFrames(InBuffer);
	}

	int32 GetMultichannelBufferNumFrames(const FMultichannelBufferView& InBuffer)
	{
		return MultichannelBufferPrivate::GetMultichannelBufferNumFrames(InBuffer);
	}

	FMultichannelBufferView MakeMultichannelBufferView(FMultichannelBuffer& InBuffer)
	{
		FMultichannelBufferView View;

		View.Reset(InBuffer.Num());
		for (FAlignedFloatBuffer& ChannelBuffer: InBuffer)
		{
			View.Emplace(ChannelBuffer);
		}

		return View;
	}

	FMultichannelBufferView MakeMultichannelBufferView(FMultichannelBuffer& InBuffer, int32 InStartFrameIndex, int32 InNumFrames)
	{
		FMultichannelBufferView View;

		View.Reset(InBuffer.Num());
		for (FAlignedFloatBuffer& ChannelBuffer: InBuffer)
		{
			check(InBuffer.Num() <= (InStartFrameIndex + InNumFrames));
			View.Emplace(&ChannelBuffer.GetData()[InStartFrameIndex], InNumFrames);
		}

		return View;
	}

	FMultichannelBufferView SliceMultichannelBufferView(const FMultichannelBufferView& View, int32 InStartFrameIndex, int32 InNumFrames)
	{
		FMultichannelBufferView OutView;
		for (int32 ChannelIndex = 0; ChannelIndex < View.Num(); ChannelIndex++)
		{
			OutView.Emplace(View[ChannelIndex].Slice(InStartFrameIndex, InNumFrames));
		}
		return OutView;
	}

	void ShiftMultichannelBufferView(int32 InNumFrames, FMultichannelBufferView& View)
	{
		int32 NumRemainingFrames = GetMultichannelBufferNumFrames(View);
		int32 NumFramesToShift = FMath::Clamp(InNumFrames, 0, NumRemainingFrames);
		int32 NumFramesToKeep = NumRemainingFrames - NumFramesToShift;

		for (int32 ChannelIndex = 0; ChannelIndex < View.Num(); ChannelIndex++)
		{
			View[ChannelIndex] = View[ChannelIndex].Slice(NumFramesToShift, NumFramesToKeep);
		}
	}
}
