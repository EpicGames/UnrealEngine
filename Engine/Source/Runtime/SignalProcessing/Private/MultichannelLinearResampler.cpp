// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/MultichannelLinearResampler.h"

#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/MultichannelBuffer.h"
#include "HAL/PlatformMath.h"
#include "Math/UnrealMathUtility.h"

namespace Audio
{
	const float FMultichannelLinearResampler::MaxFrameRatio = 100.f;
	const float FMultichannelLinearResampler::MinFrameRatio = 0.001f;

	FMultichannelLinearResampler::FMultichannelLinearResampler(int32 InNumChannels)
		: NumChannels(InNumChannels)
	{
	}

	void FMultichannelLinearResampler::SetFrameRatio(float InRatio, int32 InDesiredNumFramesToInterpolate)
	{
		if (ensureMsgf((InRatio >= MinFrameRatio) && (InRatio <= MaxFrameRatio), TEXT("The frame ratio (%f) must be between %f and %f."), InRatio, MinFrameRatio, MaxFrameRatio))
		{
			if ((InDesiredNumFramesToInterpolate <= 1) || FMath::IsNearlyEqual(InRatio, CurrentFrameRatio))
			{
				// Set frame ratio immediately. 
				CurrentFrameRatio = InRatio;
				TargetFrameRatio = InRatio;
				FrameRatioFrameDelta = 0.f;
				NumFramesToInterpolate = 0;
			}
			else
			{
				// Interpolate frame ratio over output frames. 
				TargetFrameRatio = InRatio;
				NumFramesToInterpolate = InDesiredNumFramesToInterpolate;
				FrameRatioFrameDelta = (TargetFrameRatio - CurrentFrameRatio) / (NumFramesToInterpolate);

				// Frame ratio frame deltas which are very small cause numerical
				// stability issues when mapping between input and output frame 
				// indices. If the frame delta is below a threshold, we reduce the
				// number of frames to interpolate over until the frame ratio delta
				// is within an acceptable range.
				while ((NumFramesToInterpolate > 1) && (FMath::Abs(FrameRatioFrameDelta) < MinFrameRatioFrameDelta))
				{
					NumFramesToInterpolate /= 2;
					if (NumFramesToInterpolate > 0)
					{
						FrameRatioFrameDelta = (TargetFrameRatio - CurrentFrameRatio) / (NumFramesToInterpolate);
					}
				}
			}
		}
	}

	float FMultichannelLinearResampler::MapOutputFrameToInputFrame(float InOutputFrameIndex) const
	{
		// Sample index mapping requires a bit of math because the frame 
		// ratio is linearly interpolated over some number of output samples. These
		// equations are roughly equivalent to physics equations for position,
		// velocity and acceleration, but differ in respect to using summations
		// as opposed to integrals. 
		//
		// To derive equations for mapping, do the following:
		// 1. Define the frame ratio as a function of the output frame index
		//
		// 		v_o  is the initial frame ratio.
		// 		v_t  is the target frame ratio.
		// 		M    is the number of output samples for interpolating between v_o and v_t
		// 		A[t] is the frame ratio delta per frame.
		// 		V[t] is the frame ratio at the output sample at index t.
		// 		
		// 		D = (v_t - v_o) / M
		// 		A[t] = D                 for 0 <= t < M
		// 		A[t] = 0                 for t >= M
		// 		V[t] = v_o + t * A[t]
		//
		//	2. To map from an output index (y) to an input index (x), take the 
		//	summation of R(t) from 0 to y.
		//		
		//		Ti    is the input index 
		//		Ti_o  is the initial input index offset
		//		To    is the output index
		//
		//		Ti = Sum(V[t] from t=0 to t=To) + Ti_o
		//
		//		Ti = To * v_o + (To * (To + 1) / 2) * D                 for 0 <= To < M
		//		Ti = M * v_o + (M * (M + 1) / 2) * D + (To - M) * v_t   for To >= M
		//
		//	3. To map from an input index to and output index, use prior equations
		//	and solve for To.
		//
		//		P = Ti_o + M * v_o + (M * (M + 1) / 2) * D    Final input frame where interpolation is occuring.
		//		Qa = D / 2                                    Temp value for solving quadratic equation 
		//		Qb = v_o + D / 2                              Temp value for solving quadratic equation
		//		Qc = Ti_o - Ti                                Temp value for solving quadratic equation
		//
		//
		//		To = (-Qb + sqrt(Qb^2 - 4 * Qa * Qc)) / (2 * Qa)                      for 0 < Ti < P
		//	    To = (Ti - Ti_o) - M * v_o - (M * (M + 1) / 2) * D + M * v_t) / v_t   for Ti >= P
		//
		checkf(InOutputFrameIndex >= 0.f, TEXT("Frame index mapping function is only value for outputs frames greater than or equal to 0"));

		float InputFrameIndex = 0.f;
		if (NumFramesToInterpolate > 0)
		{
			if (InOutputFrameIndex < NumFramesToInterpolate)
			{
				// Frame ratio interpolation is still occurring at the output frame index.
				const float AccumulationOfFrameDeltas = FrameRatioFrameDelta * (InOutputFrameIndex * (InOutputFrameIndex + 1.f) / 2.f);
				InputFrameIndex = InOutputFrameIndex * CurrentFrameRatio + AccumulationOfFrameDeltas;
			}
			else
			{
				// Frame ratio interpolation occurred, but has reached the target frame ratio by the output frame index.
				const float AccumulationOfFrameDeltas = FrameRatioFrameDelta * (NumFramesToInterpolate * (NumFramesToInterpolate + 1.f) / 2.f);
				InputFrameIndex = NumFramesToInterpolate * CurrentFrameRatio + AccumulationOfFrameDeltas + (InOutputFrameIndex - NumFramesToInterpolate) * TargetFrameRatio;
			}
		}
		else
		{
			// No interpolation is happening. The math is quite a bit simpler. 
			InputFrameIndex = TargetFrameRatio * InOutputFrameIndex;
		}

		// Apply current internal offset. 
		InputFrameIndex += CurrentInputFrameIndex;
		return InputFrameIndex;
	}

	float FMultichannelLinearResampler::MapInputFrameToOutputFrame(float InInputFrameIndex) const
	{
		checkf(InInputFrameIndex >= -1.f, TEXT("Frame index mapping function is only value for inputs frames greater than or equal to -1"));

		// See comments in "MapOutputFrameToInputFrame(..)" for derivation of these formulas. 

		float OutputFrameIndex = 0.f;
		if (NumFramesToInterpolate > 0)
		{
			const float AccumulationOfFrameDeltas = (NumFramesToInterpolate * (NumFramesToInterpolate + 1.f) / 2.f) * FrameRatioFrameDelta;
			const float NumInputFramesToInterpolate = CurrentInputFrameIndex + NumFramesToInterpolate * CurrentFrameRatio + AccumulationOfFrameDeltas;

			if (InInputFrameIndex < NumInputFramesToInterpolate)
			{
				// Use double when solving for quadratic to handle numerical stability issues. 
				const double QuadA = FrameRatioFrameDelta / 2.;
				const double QuadB = CurrentFrameRatio + FrameRatioFrameDelta / 2.;
				const double QuadC = CurrentInputFrameIndex - InInputFrameIndex;
				OutputFrameIndex = static_cast<float>((-QuadB + FMath::Sqrt(QuadB * QuadB - 4. * QuadA * QuadC)) / (2. * QuadA));
			}
			else
			{
				OutputFrameIndex = InInputFrameIndex - CurrentInputFrameIndex - NumFramesToInterpolate * CurrentFrameRatio - AccumulationOfFrameDeltas + NumFramesToInterpolate * TargetFrameRatio;
				OutputFrameIndex /= TargetFrameRatio;
			}
		}
		else if (TargetFrameRatio > 0.f)
		{
			OutputFrameIndex = (InInputFrameIndex - CurrentInputFrameIndex) / TargetFrameRatio;
		}

		return OutputFrameIndex;
	}

	int32 FMultichannelLinearResampler::GetNumInputFramesNeededToProduceOutputFrames(int32 InNumOutputFrames) const
	{
		if (InNumOutputFrames > 0)
		{
			const int32 NumBufferFrames = GetNumBufferFramesToProduceOutputFrames(InNumOutputFrames);
			return FMath::CeilToInt(MapOutputFrameToInputFrame(InNumOutputFrames - 1)) + NumBufferFrames; 
		}
		return 0;
	}

	int32 FMultichannelLinearResampler::GetNumBufferFramesToProduceOutputFrames(int32 InNumOutputFrames) const
	{
		// buffer frames to deal with numerical accuracy issues when  calculating 
		// large number of output frames.
		check(InNumOutputFrames > 0);
		return FMath::Max(1, InNumOutputFrames / 100) + 1;
	}

	template<typename OutputMultichannelBufferType>
	int32 FMultichannelLinearResampler::ProcessAndConsumeAudioInternal(FMultichannelCircularBuffer& InAudio, OutputMultichannelBufferType& OutAudio)
	{
		checkf(InAudio.Num() == OutAudio.Num(), TEXT("Input/output channel count mismatch."));
		checkf(InAudio.Num() == NumChannels, TEXT("Incorrect audio channel count."));

		int32 NumOutputFrames = GetMultichannelBufferNumFrames(OutAudio);
		const int32 NumAvailableInputFrames = GetMultichannelBufferNumFrames(InAudio);
		int32 NumInputFramesRequired = GetNumInputFramesNeededToProduceOutputFrames(NumOutputFrames);

		if (NumAvailableInputFrames < NumInputFramesRequired)
		{
			// Update number of frames to generate based on available number of samples
			// When FrameRatioFrameDelta is small (1e-5 or less), MapInputFrameToOutputFrame(...)
			// becomes prone to numerical precision issues. 
			// To avoid errors due to precision issues, use the maximum frame rate
			// to determine the number of output frames which can be safely generated
			// from the given number of input frames. . 
			
			const int32 NumBufferFrames = GetNumBufferFramesToProduceOutputFrames(NumOutputFrames);
			NumOutputFrames = FMath::FloorToInt((NumAvailableInputFrames - NumBufferFrames) / FMath::Max(CurrentFrameRatio, TargetFrameRatio)) - 1;
			NumOutputFrames = FMath::Max(NumOutputFrames, 0);
			NumInputFramesRequired = NumAvailableInputFrames;
			checkf(NumInputFramesRequired > FMath::CeilToInt(MapOutputFrameToInputFrame(NumOutputFrames - 1)), TEXT("Invalid calculation. Required input frames (%d) does not satisfy need for input frames (%f)"), NumInputFramesRequired, MapOutputFrameToInputFrame(NumOutputFrames - 1));
		}

		if (NumOutputFrames > 0)
		{
			float FinalInputFrameIndex = 0.f;
			int32 NumFramesToPop = 0;

			// Copy input buffers into work buffer since circular buffers 
			// are not ensured to hold entire array contiguously.
			WorkBuffer.SetNumUninitialized(NumInputFramesRequired, EAllowShrinking::No);

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
			{
				const int32 NumFramesCopied = InAudio[ChannelIndex].Peek(WorkBuffer.GetData(), NumInputFramesRequired);
				check(NumFramesCopied == NumInputFramesRequired);

				FinalInputFrameIndex = ProcessChannelAudioInternal(WorkBuffer, TArrayView<float>(OutAudio[ChannelIndex].GetData(), NumOutputFrames));
				NumFramesToPop = FMath::Floor(FinalInputFrameIndex);

				// Remove unneeded input audio.
				InAudio[ChannelIndex].Pop(NumFramesToPop);
			}

			// Update current frame ratio
			if (NumFramesToInterpolate > 0)
			{
				if (NumFramesToInterpolate > NumOutputFrames)
				{
					CurrentFrameRatio += NumOutputFrames * FrameRatioFrameDelta;
					NumFramesToInterpolate -= NumOutputFrames;
				}
				else
				{
					NumFramesToInterpolate = 0;
					CurrentFrameRatio = TargetFrameRatio;
				}
			}

			// Increment frame counters.
			CurrentInputFrameIndex = FinalInputFrameIndex - NumFramesToPop;
			NumFramesToInterpolate -= FMath::Min(NumFramesToInterpolate, NumOutputFrames);
		}

		return NumOutputFrames;
	}

	int32 FMultichannelLinearResampler::ProcessAndConsumeAudio(FMultichannelCircularBuffer& InAudio, FMultichannelBuffer& OutAudio)
	{
		return ProcessAndConsumeAudioInternal(InAudio, OutAudio);
	}

	int32 FMultichannelLinearResampler::ProcessAndConsumeAudio(FMultichannelCircularBuffer& InAudio, TArray<TArrayView<float>>& OutAudio)
	{
		return ProcessAndConsumeAudioInternal(InAudio, OutAudio);
	}

	float FMultichannelLinearResampler::ProcessChannelAudioInternal(TArrayView<const float> InAudio, TArrayView<float> OutAudio)
	{
		const int32 NumOutputFrames = OutAudio.Num();

		if (NumOutputFrames < 1)
		{
			return 0.f;
		}

		float* OutAudioData = OutAudio.GetData();
		const float* InAudioData = InAudio.GetData();

		if ((CurrentFrameRatio == 1.f) && (CurrentInputFrameIndex == 0.f) && (0 == NumFramesToInterpolate))
		{
			// No interpolation is needed. Samples can be copied directly.
			FMemory::Memcpy(OutAudioData, InAudioData, NumOutputFrames * sizeof(float));
			return NumOutputFrames;
		}
		else
		{
			float InputFrameRatio = CurrentFrameRatio;
			float InputFrameIndex = CurrentInputFrameIndex;
			int32 LowerFrameIndex = 0;

			checkf(InputFrameIndex >= 0.f, TEXT("Input frame index references discarded data"));

			// Handle frames where samples are interpolated. 
			int32 NumFramesComputed = FMath::Min(NumOutputFrames, NumFramesToInterpolate);// +1);
			for (int32 OutputFrameIndex = 0; OutputFrameIndex < NumFramesComputed; OutputFrameIndex++)
			{
				LowerFrameIndex = (int32)InputFrameIndex;
				float Alpha = InputFrameIndex - LowerFrameIndex;
				OutAudioData[OutputFrameIndex] = FMath::Lerp(InAudioData[LowerFrameIndex], InAudioData[LowerFrameIndex + 1], Alpha);
				
				InputFrameRatio += FrameRatioFrameDelta;
				InputFrameIndex += InputFrameRatio;
			}

			// Handle frames where sample rate is constant.	
			if (NumFramesComputed < NumOutputFrames)
			{
				InputFrameRatio = TargetFrameRatio;
				for (int32 OutputFrameIndex = NumFramesComputed; OutputFrameIndex < NumOutputFrames; OutputFrameIndex++)
				{
					LowerFrameIndex = (int32)InputFrameIndex;
					float Alpha = InputFrameIndex - LowerFrameIndex;
					OutAudioData[OutputFrameIndex] = FMath::Lerp(InAudioData[LowerFrameIndex], InAudioData[LowerFrameIndex + 1], Alpha);
					InputFrameIndex += InputFrameRatio;
					
				}
			}

			// Check for buffer over run
			checkf((LowerFrameIndex + 1) < InAudio.Num(), TEXT("Buffer overrun in multichannel linear resampler. Attempt to read index %d of array with %d elements. FrameRatio: %f, FrameRatioDelta: %f, NumFramesToInterpolate: %d"), LowerFrameIndex + 1, InAudio.Num(), CurrentFrameRatio, FrameRatioFrameDelta, NumFramesToInterpolate);

			return InputFrameIndex;
		}
	}
}

