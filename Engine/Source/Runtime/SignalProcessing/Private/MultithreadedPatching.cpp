// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/MultithreadedPatching.h"

#include "DSP/FloatArrayMath.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/Event.h"

static int32 MultithreadedPatchingPushCallsPerOutputCleanupCheckCVar = 256;
FAutoConsoleVariableRef CVarMultithreadedPatchingPushCallsPerOutputCleanupCheck(
	TEXT("au.MultithreadedPatching.PushCallsPerOutputCleanupCheck"),
	MultithreadedPatchingPushCallsPerOutputCleanupCheckCVar,
	TEXT("Number of push calls (usually corrisponding to audio block updates)\n")
	TEXT("before checking if an output is ready to be destroyed. Default = 256"),
	ECVF_Default);

namespace Audio
{
	TAtomic<int32> FPatchOutput::PatchIDCounter(0);

	FPatchOutput::FPatchOutput(int32 InMaxCapacity, float InGain /*= 1.0f*/)
		: InternalBuffer(InMaxCapacity)
		, TargetGain(InGain)
		, PreviousGain(InGain)
		, PatchID(++PatchIDCounter)
		, NumAliveInputs(0)
		, SamplesPushedEvent(nullptr)
	{

	}


	FPatchOutput::FPatchOutput()
		: InternalBuffer(0)
		, TargetGain(0.0f)
		, PreviousGain(0.0f)
		, PatchID(INDEX_NONE)
		, NumAliveInputs(0)
		, SamplesPushedEvent(nullptr)
	{
	}

	FPatchOutput::~FPatchOutput()
	{
		if (FEvent* Event = SamplesPushedEvent.exchange(nullptr, std::memory_order_acq_rel))
		{
			FPlatformProcess::ReturnSynchEventToPool(Event);
		}
	}

	int32 FPatchOutput::PopAudio(float* OutBuffer, int32 NumSamples, bool bUseLatestAudio)
	{
		if (IsInputStale())
		{
			return INDEX_NONE;
		}

		const int32 CurrSamplesAvailable = GetNumSamplesAvailable();
		const int32 CurrCapacity = InternalBuffer.GetCapacity();
		const int32 CurrSamplesFilled = CurrCapacity - CurrSamplesAvailable;
		
		if (bUseLatestAudio && InternalBuffer.Num() > (uint32) NumSamples)
		{
			InternalBuffer.SetNum((uint32)NumSamples);
		}

		int32 PopResult = InternalBuffer.Pop(OutBuffer, NumSamples);
		TArrayView<float> OutBufferView(OutBuffer, PopResult);

		// Apply gain stage.
		float TG = TargetGain, PG = PreviousGain;
		if (FMath::IsNearlyEqual(TG, PG))
		{
			ArrayMultiplyByConstantInPlace(OutBufferView, PreviousGain);
		}
		else
		{
			ArrayFade(OutBufferView, PreviousGain, TargetGain);
			PreviousGain = TargetGain;
		}
		
		return PopResult;
	}

	bool FPatchOutput::IsInputStale() const
	{
		return NumAliveInputs.load(std::memory_order_relaxed) == 0;
	}

	int32 FPatchOutput::PushAudioToInternalBuffer(const float* InBuffer, int32 NumSamples)
	{
		const int32 NumSamplesPushed = InBuffer ? InternalBuffer.Push(InBuffer, NumSamples) : InternalBuffer.PushZeros(NumSamples);

		// Check to see if we need to notify anybody waiting on this patch output getting filled
		if (FEvent* Event = SamplesPushedEvent.load(std::memory_order_acquire))
		{
			Event->Trigger();
		}

		const int32 CurrSamplesAvailable = GetNumSamplesAvailable();
		const int32 CurrCapacity = InternalBuffer.GetCapacity();
		const int32 CurrSamplesFilled = CurrCapacity - CurrSamplesAvailable;
		
		return NumSamplesPushed;
	}

	void FPatchOutput::EmptyInternalBuffer()
	{
		// Don't do an Empty() because then we would need to re-initialize.
		InternalBuffer.Pop(GetNumSamplesAvailable());
	}

	int32 FPatchOutput::MixInAudio(float* OutBuffer, int32 NumSamples, bool bUseLatestAudio)
	{
		if (IsInputStale())
		{
			return INDEX_NONE;
		}

		MixingBuffer.SetNumUninitialized(NumSamples, EAllowShrinking::No);
		int32 PopResult = 0;
		
		if (bUseLatestAudio && InternalBuffer.Num() > (uint32)NumSamples)
		{
			InternalBuffer.SetNum(((uint32)NumSamples));
			PopResult = InternalBuffer.Peek(MixingBuffer.GetData(), NumSamples);
		}
		else
		{
			PopResult = InternalBuffer.Pop(MixingBuffer.GetData(), NumSamples);
		}

		TArrayView<const float> MixingBufferView(MixingBuffer.GetData(), PopResult);
		TArrayView<float> OutBufferView(OutBuffer, PopResult);

		float TG = TargetGain, PG = PreviousGain;
		if (FMath::IsNearlyEqual(TG, PG))
		{
			ArrayMixIn(MixingBufferView, OutBufferView, PreviousGain);
		}
		else
		{
			ArrayMixIn(MixingBufferView, OutBufferView, PreviousGain, TargetGain);
			PreviousGain = TargetGain;
		}

		return PopResult;
	}


	int32 FPatchOutput::GetNumSamplesAvailable() const
	{
		return InternalBuffer.Num();
	}

	int32 FPatchOutput::GetNumSamplesOverhead() const
	{
		return InternalBuffer.Remainder();
	}

	bool FPatchOutput::WaitUntilNumSamplesAvailable(int32 InNumSamplesToWaitFor, uint32 TimeOutMilliseconds)
	{
		// Samples are ready if there are enough of them available, or the input is stale.
		// Conceptually, a stale input contributes silence, so its samples are known.
		if (IsInputStale() || GetNumSamplesAvailable() >= InNumSamplesToWaitFor)
		{
			return true;
		}

		// Samples are not ready if the timeout is zero or the
		// internal buffer can't hold the number of requested samples.
		if (!TimeOutMilliseconds || InternalBuffer.GetCapacity() < uint32(InNumSamplesToWaitFor))
		{
			return false;
		}

		// Determine if an event is ready to signal when
		// samples are submitted.  Provide one if not.
		FEvent* Event = SamplesPushedEvent.load(std::memory_order_acquire);
		if (!Event)
		{
			FEvent* NewEvent = FPlatformProcess::GetSynchEventFromPool(false);
			if (SamplesPushedEvent.compare_exchange_strong(Event, NewEvent, std::memory_order_release, std::memory_order_acquire))
			{
				// Use the new event.
				Event = NewEvent;
			}
			else
			{
				// Use the existing event and return the new one.
				FPlatformProcess::ReturnSynchEventToPool(NewEvent);
			}
		}

		// Calculate when the timeout period will end.
		double CurrentTime = FPlatformTime::Seconds();
		double WaitStopTime = CurrentTime + 0.001 * TimeOutMilliseconds;

		// First determine if samples are available.
		// They might have become available before the event was ready to signal.
		while (!IsInputStale() && GetNumSamplesAvailable() < InNumSamplesToWaitFor)
		{
			// Wait for the next sample submission.
			Event->Wait(TimeOutMilliseconds);

			// If the timeout is not infinite, update it.
			if (TimeOutMilliseconds != MAX_uint32)
			{
				CurrentTime = FPlatformTime::Seconds();

				// Return if no time remains, indicating whether samples are ready.
				double TimeOutRemainder = (WaitStopTime - CurrentTime) * 1000.;
				if (!(TimeOutRemainder > 0.))
				{
					return IsInputStale() || GetNumSamplesAvailable() >= InNumSamplesToWaitFor;
				}

				// Truncating to an integer will cause a spin-wait
				// when there's less than a millisecond remaining.
				TimeOutMilliseconds = uint32(TimeOutRemainder);
			}
		}

		// Samples are ready.
		return true;
	}

	FPatchInput::FPatchInput(const FPatchOutputStrongPtr& InOutput)
		: OutputHandle(InOutput)
	{
		if (OutputHandle.IsValid())
		{
			OutputHandle->NumAliveInputs.fetch_add(1, std::memory_order_relaxed);
		}
	}

	FPatchInput::FPatchInput(const FPatchInput& InOther)
		: OutputHandle(InOther.OutputHandle)
	{
		if (OutputHandle.IsValid())
		{
			OutputHandle->NumAliveInputs.fetch_add(1, std::memory_order_relaxed);
		}
	}

	FPatchInput::FPatchInput(FPatchInput&& Other)
		: OutputHandle(MoveTemp(Other.OutputHandle))
		, PushCallsCounter(Other.PushCallsCounter)
	{
		Other.PushCallsCounter = 0;
	}

	FPatchInput& FPatchInput::operator=(const FPatchInput& Other)
	{
		FPatchInput NewPatchInput(Other);
		Swap(NewPatchInput, *this);
		return *this;
	}

	FPatchInput& FPatchInput::operator=(FPatchInput&& Other)
	{
		FPatchInput NewPatchInput(MoveTemp(Other));
		Swap(NewPatchInput, *this);
		return *this;
	}

	FPatchInput::~FPatchInput()
	{
		Reset();
	}

	int32 FPatchInput::PushAudio(const float* InBuffer, int32 NumSamples)
	{
		if (!OutputHandle.IsValid())
		{
			return INDEX_NONE;
		}

		int32 SamplesPushed = OutputHandle->PushAudioToInternalBuffer(InBuffer, NumSamples);

		// Periodically check to see if the output handle has been destroyed and clean it up.
		// If the buffer is full, check as well to determine if it is possibly due to the
		// output going stale between periodic push call counter checks.
		const int32 PushCallsPerOutputCleanupCheck = FMath::Max(1, MultithreadedPatchingPushCallsPerOutputCleanupCheckCVar);
		PushCallsCounter = (PushCallsCounter + 1) % PushCallsPerOutputCleanupCheck;
		if (PushCallsCounter == 0 || SamplesPushed == 0)
		{
			if (OutputHandle.IsUnique())
			{
				// Deletes the output as it is the last remaining handle
				OutputHandle.Reset();
				SamplesPushed = INDEX_NONE;
			}
		}

		return SamplesPushed;
	}

	void FPatchInput::ClearAudio()
	{
		if (!OutputHandle.IsValid())
		{
			return;
		}

		OutputHandle->EmptyInternalBuffer();
	}

	/** Returns the current number of samples buffered in this input. */
	int32 FPatchInput::GetNumSamplesAvailable() const
	{
		if (!OutputHandle.IsValid())
		{
			return 0;
		}

		return OutputHandle->GetNumSamplesAvailable();
	}

	int32 FPatchInput::GetNumSamplesOverhead() const
	{
		if (!OutputHandle.IsValid())
		{
			return 0;
		}

		return OutputHandle->GetNumSamplesOverhead();
	}

	void FPatchInput::SetGain(float InGain)
	{
		if (!OutputHandle.IsValid())
		{
			return;
		}

		OutputHandle->TargetGain = InGain;
	}

	bool FPatchInput::IsOutputStillActive() const
	{
		return OutputHandle.IsValid() && !OutputHandle.IsUnique();
	}

	bool FPatchInput::IsValid() const
	{
		return OutputHandle.IsValid();
	}

	void FPatchInput::Reset()
	{
		if (OutputHandle.IsValid())
		{
			if (OutputHandle->NumAliveInputs.fetch_sub(1, std::memory_order_relaxed) == 1)
			{
				if (FEvent* Event = OutputHandle->SamplesPushedEvent.load(std::memory_order_acquire))
				{
					// Signal that the sample status should be checked.
					Event->Trigger();
				}
			}
			OutputHandle.Reset();
		}
	}

	FPatchInput FPatchMixer::AddNewInput(int32 InMaxLatencyInSamples, float InGain)
	{
		FScopeLock ScopeLock(&PendingNewInputsCriticalSection);

		const int32 NewPatchIndex = PendingNewInputs.Add(MakeShared<FPatchOutput, ESPMode::ThreadSafe>(InMaxLatencyInSamples, InGain));
		return FPatchInput(PendingNewInputs[NewPatchIndex]);
	}

	void FPatchMixer::AddNewInput(const FPatchInput& InPatchInput)
	{
		if (!InPatchInput.OutputHandle.IsValid())
		{
			return;
		}

		FScopeLock ScopeLock(&PendingNewInputsCriticalSection);

		PendingNewInputs.Add(InPatchInput.OutputHandle);
	}

	void FPatchMixer::RemovePatch(const FPatchInput& InPatchInput)
	{
		// If the output is already disconnected, early exit.
		if (!InPatchInput.OutputHandle.IsValid())
		{
			return;
		}

		FScopeLock ScopeLock(&InputDeletionCriticalSection);
		DisconnectedInputs.Add(InPatchInput.OutputHandle->PatchID);
	}

	int32 FPatchMixer::PopAudio(float* OutBuffer, int32 OutNumSamples, bool bUseLatestAudio)
	{
		FScopeLock ScopeLock(&CurrentPatchesCriticalSection);

		CleanUpDisconnectedPatches();
		ConnectNewPatches();

		FMemory::Memzero(OutBuffer, OutNumSamples * sizeof(float));
		int32 MaxPoppedSamples = 0;

		for (FPatchOutputStrongPtr& OutputPtr : CurrentInputs)
		{
			const int32 NumPoppedSamples = OutputPtr->MixInAudio(OutBuffer, OutNumSamples, bUseLatestAudio);
			MaxPoppedSamples = FMath::Max(NumPoppedSamples, MaxPoppedSamples);
		}

		return MaxPoppedSamples;
	}

	int32 FPatchMixer::Num()
	{
		FScopeLock ScopeLock(&CurrentPatchesCriticalSection);

		CleanUpDisconnectedPatches();
		ConnectNewPatches();

		return CurrentInputs.Num();
	}

	int32 FPatchMixer::MaxNumberOfSamplesThatCanBePopped()
	{
		FScopeLock ScopeLock(&CurrentPatchesCriticalSection);

		CleanUpDisconnectedPatches();
		ConnectNewPatches();

		if (CurrentInputs.IsEmpty())
		{
			return INDEX_NONE;
		}

		// Iterate through our inputs and see which input has the least audio buffered.
		uint32 SmallestNumSamplesBuffered = TNumericLimits<uint32>::Max();
		
		for (FPatchOutputStrongPtr& Output : CurrentInputs)
		{
			if (Output.IsValid())
			{
				SmallestNumSamplesBuffered = FMath::Min(SmallestNumSamplesBuffered, Output->InternalBuffer.Num());
			}
		}

		return SmallestNumSamplesBuffered;
	}

	bool FPatchMixer::WaitUntilNumSamplesAvailable(int32 NumSamples, uint32 TimeOutMilliseconds)
	{
		FScopeLock ScopeLock(&CurrentPatchesCriticalSection);

		CleanUpDisconnectedPatches();
		ConnectNewPatches();

		if (CurrentInputs.IsEmpty())
		{
			// Samples are ready if there are enough of them available, or the input is stale.
			// Conceptually, a stale input contributes silence, so its samples are known.
			return true;
		}

		// Calculate when the timeout period will end.
		double CurrentTime = FPlatformTime::Seconds();
		double WaitStopTime = CurrentTime + 0.001 * TimeOutMilliseconds;
		for (FPatchOutputStrongPtr& Output : CurrentInputs)
		{
			bool bSamplesAvailable;
			do
			{
				bSamplesAvailable = Output->WaitUntilNumSamplesAvailable(NumSamples, TimeOutMilliseconds);

				// If the timeout is not infinite, update it.
				if (TimeOutMilliseconds != MAX_uint32)
				{
					CurrentTime = FPlatformTime::Seconds();

					// Return if no time remains and samples are not ready.
					double TimeOutRemainder = (WaitStopTime - CurrentTime) * 1000.;
					if (!bSamplesAvailable && !(TimeOutRemainder > 0.))
					{
						return false;
					}

					// If samples were ready, but the timeout has expired,
					// the wait could still succeed if all the subsequent inputs are ready.
					TimeOutMilliseconds = TimeOutRemainder > 0. ? uint32(TimeOutRemainder) : 0;
				}
			}
			while (!bSamplesAvailable);
		}

		return true;
	}

	void FPatchMixer::DisconnectAllInputs()
	{
		FScopeLock ScopeLock(&CurrentPatchesCriticalSection);
		CurrentInputs.Reset();
	}

	void FPatchMixer::ConnectNewPatches()
	{
		FScopeLock ScopeLock(&PendingNewInputsCriticalSection);

		// If AddNewPatch is called in a separate thread, wait until the next PopAudio call to do this work.
		// Todo: convert this to move semantics to avoid copying the shared pointer around.
		for (FPatchOutputStrongPtr& Patch : PendingNewInputs)
		{
			CurrentInputs.Add(Patch);
		}

		PendingNewInputs.Reset();
	}

	void FPatchMixer::CleanUpDisconnectedPatches()
	{
		FScopeLock PendingInputDeletionScopeLock(&InputDeletionCriticalSection);

		 // Callers of this function must have CurrentPatchesCriticalSection locked so that 
		 // this is not causing a race condition.
		for (const FPatchOutputStrongPtr& Patch : CurrentInputs)
		{
			check(Patch.IsValid());

			if (Patch->IsInputStale())
			{
				DisconnectedInputs.Add(Patch->PatchID);
			}
		}

		// Iterate through all of the PatchIDs to be cleaned up.
		for (const int32& PatchID : DisconnectedInputs)
		{
			bool bInputRemoved = false;

			// First, make sure that the patch isn't in the pending new patchs we haven't added yet:
			{
				FScopeLock PendingNewInputsScopeLock(&PendingNewInputsCriticalSection);
				for (int32 Index = 0; Index < PendingNewInputs.Num(); Index++)
				{
					checkSlow(CurrentInputs[Index].IsValid());

					if (PatchID == PendingNewInputs[Index]->PatchID)
					{
						PendingNewInputs.RemoveAtSwap(Index);
						bInputRemoved = true;
						break;
					}
				}
			}

			if (bInputRemoved)
			{
				continue;
			}

			// Disconnect stale patches.
			for (int32 Index = 0; Index < CurrentInputs.Num(); Index++)
			{
				checkSlow(CurrentInputs[Index].IsValid());

				if (PatchID == CurrentInputs[Index]->PatchID)
				{
					CurrentInputs.RemoveAtSwap(Index);
					break;
				}
			}
		}

		DisconnectedInputs.Reset();
	}

	FPatchOutputStrongPtr FPatchSplitter::AddNewPatch(int32 MaxLatencyInSamples, float InGain)
	{
		FPatchOutputStrongPtr StrongOutputPtr = MakeShared<FPatchOutput, ESPMode::ThreadSafe>(MaxLatencyInSamples * 2, InGain);
		{
			FScopeLock ScopeLock(&PendingOutputsCriticalSection);
			PendingOutputs.Add(StrongOutputPtr);
		}

		return StrongOutputPtr;
	}

	void FPatchSplitter::AddNewPatch(FPatchOutputStrongPtr&& InPatchOutputStrongPtr)
	{
		FScopeLock ScopeLock(&PendingOutputsCriticalSection);
		PendingOutputs.Add(MoveTemp(InPatchOutputStrongPtr));
	}

	void FPatchSplitter::AddNewPatch(const FPatchOutputStrongPtr& InPatchOutputStrongPtr)
	{
		FScopeLock ScopeLock(&PendingOutputsCriticalSection);
		PendingOutputs.Add(InPatchOutputStrongPtr);
	}

	int32 FPatchSplitter::Num()
	{
		FScopeLock ScopeLock(&ConnectedOutputsCriticalSection);
		AddPendingPatches();
		return ConnectedOutputs.Num();
	}

	int32 FPatchSplitter::MaxNumberOfSamplesThatCanBePushed()
	{
		FScopeLock ScopeLock(&ConnectedOutputsCriticalSection);
		AddPendingPatches();

		if (ConnectedOutputs.IsEmpty())
		{
			return INDEX_NONE;
		}

		// Iterate over our outputs and get the smallest remainder of all of our circular buffers.
		uint32 SmallestRemainder = TNumericLimits<uint32>::Max();
		for (FPatchInput& Input : ConnectedOutputs)
		{
			const uint32 InputRemainder = Input.OutputHandle->InternalBuffer.Remainder();
			if (InputRemainder > 0 || Input.IsOutputStillActive())
			{
				SmallestRemainder = FMath::Min(SmallestRemainder, InputRemainder);
			}
		}

		return SmallestRemainder;
	}

	void FPatchSplitter::AddPendingPatches()
	{
		FScopeLock ScopeLock(&PendingOutputsCriticalSection);
		ConnectedOutputs.Append(MoveTemp(PendingOutputs));
	}

	int32 FPatchSplitter::PushAudio(const float* InBuffer, int32 InNumSamples)
	{
		FScopeLock ScopeLock(&ConnectedOutputsCriticalSection);
		AddPendingPatches();

		if (ConnectedOutputs.IsEmpty())
		{
			return INDEX_NONE;
		}

		int32 MinimumSamplesPushed = TNumericLimits<int32>::Max();
		for (int32 Index = ConnectedOutputs.Num() - 1; Index >= 0; Index--)
		{
			FPatchInput& ConnectedOutput = ConnectedOutputs[Index];
			const int32 NumSamplesPushed = ConnectedOutput.PushAudio(InBuffer, InNumSamples);
			if (NumSamplesPushed == INDEX_NONE)
			{
				ConnectedOutputs.RemoveAtSwap(Index, EAllowShrinking::No);
			}
			else
			{
				MinimumSamplesPushed = FMath::Min(MinimumSamplesPushed, NumSamplesPushed);
			}
		}
		return MinimumSamplesPushed;
	}

	FPatchOutputStrongPtr FPatchMixerSplitter::AddNewOutput(int32 MaxLatencyInSamples, float InGain)
	{
		return Splitter.AddNewPatch(MaxLatencyInSamples, InGain);
	}

	void FPatchMixerSplitter::AddNewOutput(const FPatchOutputStrongPtr& InPatchOutputStrongPtr)
	{
		Splitter.AddNewPatch(InPatchOutputStrongPtr);
	}

	FPatchInput FPatchMixerSplitter::AddNewInput(int32 MaxLatencyInSamples, float InGain)
	{
		return Mixer.AddNewInput(MaxLatencyInSamples, InGain);
	}

	void FPatchMixerSplitter::AddNewInput(FPatchInput& InInput)
	{
		Mixer.AddNewInput(InInput);
	}

	void FPatchMixerSplitter::RemovePatch(const FPatchInput& InInput)
	{
		Mixer.RemovePatch(InInput);
	}

	void FPatchMixerSplitter::ProcessAudio()
	{
		const int32 NumSamplesToPop = Mixer.MaxNumberOfSamplesThatCanBePopped();
		const int32 NumSamplesToPush = Splitter.MaxNumberOfSamplesThatCanBePushed();
		const int32 NumSamplesToForward = FMath::Min(NumSamplesToPush, NumSamplesToPop);
		
		if (NumSamplesToForward <= 0)
		{
			// Early exit when there are either no inputs or no outputs
			// connected, or one of the inputs has not pushed any audio yet.
			return;
		}

		IntermediateBuffer.Reset();
		IntermediateBuffer.AddUninitialized(NumSamplesToForward);

		// Mix down inputs
		int32 PopResult = Mixer.PopAudio(IntermediateBuffer.GetData(), NumSamplesToForward, false);
		check(PopResult == NumSamplesToForward);
		
		// Push audio to outputs
		Splitter.PushAudio(IntermediateBuffer.GetData(), NumSamplesToForward);
	}
}
