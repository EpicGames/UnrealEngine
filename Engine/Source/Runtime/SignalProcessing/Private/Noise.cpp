// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Noise.h"
#include "DSP/Dsp.h"
#include "HAL/PlatformTime.h"

namespace Audio
{
	FWhiteNoise::FWhiteNoise(int32 InRandomSeed)
		: RandomStream{ InRandomSeed }
	{}

	FWhiteNoise::FWhiteNoise()
		: FWhiteNoise{ static_cast<int32>(FPlatformTime::Cycles()) }
	{}

	FPinkNoise::FPinkNoise(int32 InRandomSeed)
		: Noise{ InRandomSeed }
		, X_Z{ 0,0,0,0 }
		, Y_Z{ 0,0.0,0 }
		, A0{ 1.0f }	// Not used.
	{
		static_assert(UE_ARRAY_COUNT(X_Z) == 4, "sizeof(X_Z)==4");
		static_assert(UE_ARRAY_COUNT(Y_Z) == 4, "sizeof(Y_Z)==4");
	}

	FPinkNoise::FPinkNoise()
		: FPinkNoise{static_cast<int32>(FPlatformTime::Cycles())}
	{}

	float FPinkNoise::Generate()
	{
		// Filter Coefficients based on:
		// https://ccrma.stanford.edu/~jos/sasp/Example_Synthesis_1_F_Noise.html
		static constexpr float A[3] { -2.494956002f,2.017265875f, -0.522189400f };
		static constexpr float B[4] { 0.049922035f,-0.095993537f,0.050612699f,-0.004408786f};	

		X_Z[0] = Noise.Generate(); // Xn

		float Yn = 
			  X_Z[0]*B[0] 
			+ X_Z[1]*B[1] 
			+ X_Z[2]*B[2] 
			+ X_Z[3]*B[3]

			- Y_Z[0]*A[0] 
			- Y_Z[1]*A[1] 
			- Y_Z[2]*A[2];

		// Shuffle feed-forward state by one.
		X_Z[3] = X_Z[2];
		X_Z[2] = X_Z[1];
		X_Z[1] = X_Z[0];

		// Shuffle feed-back state by one.
		Y_Z[3] = Y_Z[2]; 
		Y_Z[2] = Y_Z[1];
		Y_Z[1] = Y_Z[0];
		Y_Z[0] = Yn;

		return Yn;
	}
}
