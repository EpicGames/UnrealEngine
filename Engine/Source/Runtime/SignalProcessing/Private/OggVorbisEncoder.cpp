// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Encoders/OggVorbisEncoder.h"

#if PLATFORM_SUPPORTS_VORBIS_ENCODE
#pragma pack(push, 8)
#include "vorbis/vorbisenc.h"
#include "vorbis/vorbisfile.h"
#pragma pack(pop)


struct FOggVorbisEncoderPrivateState
{
	ogg_stream_state StreamState;
	ogg_page CurrentPage;
	ogg_packet CurrentPacket;

	vorbis_info BitstreamSettings;
	vorbis_dsp_state DspState;
	vorbis_block CurrentBlock;

	FOggVorbisEncoderPrivateState(const FSoundQualityInfo& InInfo)
	{
		vorbis_info_init(&BitstreamSettings);
		if (vorbis_encode_init_vbr(&BitstreamSettings, InInfo.NumChannels, InInfo.SampleRate, ((float)InInfo.Quality) / 100.0f))
		{
			UE_LOGF(LogTemp, Warning, " Error initializing Ogg Vorbis encoder!");
		}

		// Init analyzer:
		vorbis_analysis_init(&DspState, &BitstreamSettings);

		// Init the current block:
		vorbis_block_init(&DspState, &CurrentBlock);

		// Init stream encoder with null serial number:
		ogg_stream_init(&StreamState, 0);
	}

	~FOggVorbisEncoderPrivateState()
	{
		ogg_stream_clear(&StreamState);
		vorbis_block_clear(&CurrentBlock);
		vorbis_dsp_clear(&DspState);
		vorbis_info_clear(&BitstreamSettings);
	}

	void PushPacket(ogg_packet& InPacket)
	{
		int32 Result = ogg_stream_packetin(&StreamState, &InPacket);
		ensureAlwaysMsgf(Result == 0, TEXT("Pushing packet to the Ogg Stream failed. Make sure Ogg Stream was properly initialized."));
	}

	// Pop all pages available to DataToAppendTo
	void PopPages(TArray<uint8>& DataToAppendTo)
	{
		// Serialize out ogg pages until we get the EOS page.
		do
		{
			int32 BytesWritten = ogg_stream_pageout(&StreamState, &CurrentPage);
			
			// If ogg_stream_pageout returned 0, there are no pages left to pop.
			// Otherwise, append this page to DataToAppendTo.
			if (BytesWritten == 0)
			{
				break;
			}
			else
			{
				DataToAppendTo.Append((uint8*)CurrentPage.header, CurrentPage.header_len);
				DataToAppendTo.Append((uint8*)CurrentPage.body, CurrentPage.body_len);
			}
		} while (!ogg_page_eos(&CurrentPage));
	}

	// Similar to PopPages, but will ensure that the next packet we push will be on
	// a fresh page.
	void FlushPages(TArray<uint8>& DataToAppendTo)
	{
		while (true)
		{
			// Flush stream to page:
			int32 BytesPopped = ogg_stream_flush(&StreamState, &CurrentPage);

			// If the stream is finished, exit. Otherwise, append the page to DataToAppendTo.
			if (!BytesPopped)
			{
				return;
			}
			else
			{
				DataToAppendTo.Append((uint8*)CurrentPage.header, CurrentPage.header_len);
				DataToAppendTo.Append((uint8*)CurrentPage.body, CurrentPage.body_len);
			}
		}
	}

private:
	FOggVorbisEncoderPrivateState();
};

FOggVorbisEncoder::FOggVorbisEncoder(const FSoundQualityInfo& InInfo, int32 AverageBufferCallbackSize)
	: Audio::IAudioEncoder(AverageBufferCallbackSize * 4, 65536 * 4) // Vorbis ogg pages can be relatively large- up to 256 kb.
	, NumChannels(InInfo.NumChannels)
	, PrivateState(nullptr)
{
	Init(InInfo);
}

int32 FOggVorbisEncoder::GetCompressedPacketSize() const
{
	// For now, we return 0 because we are not able to chunk Ogg Vorbis streams
	// into independent chunks.
	return 0;
}

int64 FOggVorbisEncoder::SamplesRequiredPerEncode() const
{
	// Based on AudioFormatOgg.cpp- there we typically analyze 1024 samples at a time before encoding.
	return 1024;
}

bool FOggVorbisEncoder::StartFile(const FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutFileStart)
{
	check(OutFileStart.Num() == 0);
	check(PrivateState == nullptr);

	// Init all state:
	PrivateState = new FOggVorbisEncoderPrivateState(InQualityInfo);
	
	// Create a new comment to insert at the beginning of the file:
	vorbis_comment EncoderComment;
	vorbis_comment_init(&EncoderComment);
	vorbis_comment_add_tag(&EncoderComment, "ENCODER", "UnrealEngine4Runtime");

	// Generate headers:
	ogg_packet HeaderPacket;
	ogg_packet CommHeaderPacket;
	ogg_packet CodeHeaderPacket;
	vorbis_analysis_headerout(&PrivateState->DspState, &EncoderComment, &HeaderPacket, &CommHeaderPacket, &CodeHeaderPacket);
	
	// Clean up comment.
	vorbis_comment_clear(&EncoderComment);

	// Push header packets to Ogg stream:
	PrivateState->PushPacket(HeaderPacket);
	PrivateState->PushPacket(CommHeaderPacket);
	PrivateState->PushPacket(CodeHeaderPacket);

	// We need to start the actual Vorbis on a fresh page, so
	// serialize out the Ogg pages required for the header and then flush:
	PrivateState->FlushPages(OutFileStart);

	return true;
}

bool FOggVorbisEncoder::EncodeChunk(const TArray<float>& InAudio, TArray<uint8>& OutBytes)
{
	check(InAudio.Num() <= 1024);
	check(PrivateState != nullptr);

	// First, we have to analyze our input buffer:
	const int32 NumFrames = InAudio.Num() / NumChannels;
	float** AnalysisBuffer = vorbis_analysis_buffer(&PrivateState->DspState, NumFrames);

	// Deinterleave for Ogg Vorbis encoder:
	for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
		{
			AnalysisBuffer[ChannelIndex][FrameIndex] = InAudio[FrameIndex * NumChannels + ChannelIndex];
		}
	}

	vorbis_analysis_wrote(&PrivateState->DspState, NumFrames);

	// Separate AnalysisBuffer into separate blocks, then chunk those blocks into Ogg pages.
	while (vorbis_analysis_blockout(&PrivateState->DspState, &PrivateState->CurrentBlock) == 1)
	{
		// Perform actual analysis:
		vorbis_analysis(&PrivateState->CurrentBlock, NULL);
		// Then determine the bitrate on this block.
		vorbis_bitrate_addblock(&PrivateState->CurrentBlock);

		// Flush all available vorbis blocks into ogg packets, then append the resulting pages
		// to our output buffer:
		while (vorbis_bitrate_flushpacket(&PrivateState->DspState, &PrivateState->CurrentPacket))
		{
			PrivateState->PushPacket(PrivateState->CurrentPacket);
			PrivateState->PopPages(OutBytes);
		}
	}

	return true;
}

bool FOggVorbisEncoder::EndFile(TArray<uint8>& OutBytes)
{
	check(PrivateState != nullptr);

	// Finish handing over any compressed data.
	PrivateState->FlushPages(OutBytes);

	// Finalize the DSP analyzer:
	vorbis_analysis_wrote(&PrivateState->DspState, 0);

	// Then clear all resources.
	delete PrivateState;
	PrivateState = nullptr;

	return true;
}

#endif // PLATFORM_SUPPORTS_VORBIS_ENCODE
