// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Encoders/OpusEncoder.h"

#if PLATFORM_SUPPORTS_OPUS_CODEC 

THIRD_PARTY_INCLUDES_START
#include "opus.h"
#include "ogg/ogg.h"
THIRD_PARTY_INCLUDES_END

// Utility to ensure that any values we serialize are little endian:
template<typename T>
static void EnsureIsLittleEndian(T& InValue)
{
#if !PLATFORM_LITTLE_ENDIAN
	// Reverse bytes here. Stubbed out.
	ensureAlwaysMsgf(false, TEXT("Implement EnsureIsLittleEndian to allow Opus Encoding on big endian platforms"));
	//InValue = ByteReverse<T>(InValue);
#endif
}

// For .opus files, the preskip duration is recommended to be 10 milliseconds:
static constexpr float PreskipDuration = 0.08f;

struct ChannelLayout {
	int32 StreamCount;
	int32 CoupledStreamCount;
	uint8 Mapping[8];
};

/* Index will be NumChannels-1.*/
static const ChannelLayout ChannelLayouts[8] = {
{ 1, 0,{ 0 } },                      /* 1: mono */
{ 1, 1,{ 0, 1 } },                   /* 2: stereo */
{ 2, 1,{ 0, 1, 2 } },                /* 3: 1-d surround */
{ 2, 2,{ 0, 1, 2, 3 } },             /* 4: quadraphonic surround */
{ 3, 2,{ 0, 1, 4, 2, 3 } },          /* 5: 5-channel surround */
{ 4, 2,{ 0, 1, 4, 5, 2, 3 } },       /* 6: 5.1 surround */
{ 4, 3,{ 0, 1, 4, 6, 2, 3, 5 } },    /* 7: 6.1 surround */
{ 5, 3,{ 0, 1, 6, 7, 2, 3, 4, 5 } }, /* 8: 7.1 surround */
};

// Header and comment information, as defined here:
// https://tools.ietf.org/html/rfc7845#section-5.1
static TArray<uint8> GenerateHeaderPacket(const int32 NumChannels, uint32 SampleRate)
{
	// Some sanity checks for numeric limits:
	check(NumChannels <= 8);
	static_assert((PreskipDuration * 48000.0f) > 0 && (PreskipDuration * 48000.0f) < TNumericLimits<uint16>::Max(), "PreskipDuration is beyond what we can fit in the header.");

	TArray<uint8> Data;

	// The header starts with the 8 character string "OpusHead":
	uint8 MagicNumber[8] = { 0x4F, 0x70, 0x75, 0x73, 0x48, 0x65, 0x61, 0x64 };
	Data.Append(MagicNumber, 8);

	uint8 Version = 0x01;
	Data.Append(&Version, sizeof(uint8));

	uint8 ChannelCount = static_cast<uint8>(NumChannels);
	Data.Append(&ChannelCount, sizeof(uint8));

	uint16 Preskip = static_cast<uint16>(FMath::FloorToInt(PreskipDuration * 48000.0f));
	EnsureIsLittleEndian(Preskip);
	Data.Append((uint8*) &Preskip, sizeof(uint16));

	EnsureIsLittleEndian(SampleRate);
	Data.Append((uint8*)&SampleRate, sizeof(uint32));

	int16 OutputGain = 0;
	EnsureIsLittleEndian(OutputGain);
	Data.Append((uint8*)&OutputGain, sizeof(int16));

	uint8 ChannelMapping = NumChannels - 1;
	Data.Append(&ChannelMapping, sizeof(uint8));

	// Append channel map to header:
	const ChannelLayout& Layout = ChannelLayouts[NumChannels - 1];

	uint8 StreamCount = Layout.StreamCount;
	Data.Append(&StreamCount, sizeof(uint8));

	uint8 CoupledCount = Layout.CoupledStreamCount;
	Data.Append(&CoupledCount, sizeof(uint8));

	Data.Append(Layout.Mapping, NumChannels);

	return Data;
}

// For now, this returns an empty vendor string and comment list.
// However, this can later be fully implemented using this specification:
// https://tools.ietf.org/html/rfc7845#section-5.2
static TArray<uint8> GenerateCommentPacket()
{
	TArray<uint8> Data;

	// The header starts with the 8 character string "OpusTags":
	uint8 MagicNumber[8] = { 0x4F, 0x70, 0x75, 0x73, 0x54, 0x61, 0x67, 0x73 };
	Data.Append(MagicNumber, 8);
	
	// Stub out vendor string and comment list.
	uint8 VendorStringLength = 0;
	Data.Append(&VendorStringLength, sizeof(uint8));
	
	uint8 CommentListLength = 0;
	Data.Append(&CommentListLength, sizeof(uint8));

	return Data;
}

class FOpusEncoderPrivateState
{
public:
	OpusEncoder* Encoder;
	TArray<uint8> EncoderMemory;


	FOpusEncoderPrivateState(const FSoundQualityInfo& InInfo, bool bUseForVOIP)
	{
#if PLATFORM_IOS
		checkf(false, TEXT("Opus encoding currently not supported on iOS."));
#else
		EncoderMemory.Reset();
		EncoderMemory.AddUninitialized(opus_encoder_get_size(InInfo.NumChannels));
		Encoder = (OpusEncoder*) EncoderMemory.GetData();
		const int32 Application = bUseForVOIP ? OPUS_APPLICATION_VOIP : OPUS_APPLICATION_AUDIO;

		int32 Error = opus_encoder_init(Encoder, InInfo.SampleRate, InInfo.NumChannels, Application);

		if (Error == OPUS_OK)
		{
			// Default values for encoding: 
			
			// Turn on variable bit rate encoding
			const int32 UseVbr = 1;
			opus_encoder_ctl(Encoder, OPUS_SET_VBR(UseVbr));

			// Turn off constrained VBR
			const int32 UseCVbr = 0;
			opus_encoder_ctl(Encoder, OPUS_SET_VBR_CONSTRAINT(UseCVbr));

			// Complexity (1-10)
			const int32 Complexity = FMath::Clamp(InInfo.Quality / 10, 0, 10);
			opus_encoder_ctl(Encoder, OPUS_SET_COMPLEXITY(Complexity));

			// Forward error correction
			const int32 InbandFEC = 0;
			opus_encoder_ctl(Encoder, OPUS_SET_INBAND_FEC(InbandFEC));
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Error encountered initializing Opus."));
			Encoder = nullptr;
			EncoderMemory.Reset();
		}
#endif //PLATFORM_IOS
	}

private:
	FOpusEncoderPrivateState()
	{
	}
};

/**
 * This class takes opus packets and packs them in to Ogg pages.
 */
class FOggEncapsulator
{
private:
	ogg_stream_state StreamState;
	ogg_page CurrentPage;

public:
	ogg_packet CurrentPacket;

	FOggEncapsulator()
	{
		// Init stream encoder with null serial number:
		ogg_stream_init(&StreamState, 0);
	}

	~FOggEncapsulator()
	{
		ogg_stream_clear(&StreamState);
	}

	void PushPacket(ogg_packet& InPacket)
	{
		int32 Result = ogg_stream_packetin(&StreamState, &InPacket);
		ensureAlwaysMsgf(Result == 0, TEXT("Pushing packet to the Ogg Stream failed. Make sure Ogg Stream was properly initialized."));
	}

	// Pop all pages available to DataToAppendTo
	void PopPages(TArray<uint8>& DataToAppendTo)
	{
		// Serialize out ogg pages until we get the EOS page.
		do
		{
			int32 BytesWritten = ogg_stream_pageout(&StreamState, &CurrentPage);

			// If ogg_stream_pageout returned 0, there are no pages left to pop.
			// Otherwise, append this page to DataToAppendTo.
			if (BytesWritten == 0)
			{
				break;
			}
			else
			{
				DataToAppendTo.Append((uint8*)CurrentPage.header, CurrentPage.header_len);
				DataToAppendTo.Append((uint8*)CurrentPage.body, CurrentPage.body_len);
			}
		} while (!ogg_page_eos(&CurrentPage));
	}

	// Similar to PopPages, but will ensure that the next packet we push will be on
	// a fresh page.
	void FlushPages(TArray<uint8>& DataToAppendTo)
	{
		while (true)
		{
			// Flush stream to page:
			int32 BytesPopped = ogg_stream_flush(&StreamState, &CurrentPage);

			// If the stream is finished, exit. Otherwise, append the page to DataToAppendTo.
			if (!BytesPopped)
			{
				return;
			}
			else
			{
				DataToAppendTo.Append((uint8*)CurrentPage.header, CurrentPage.header_len);
				DataToAppendTo.Append((uint8*)CurrentPage.body, CurrentPage.body_len);
			}
		}
	}
};

FOpusEncoder::FOpusEncoder(const FSoundQualityInfo& InInfo, int32 AverageBufferCallbackSize, EOpusFrameSizes InFrameSize /*= EOpusFrameSizes::MediumLow*/, EOpusMode InMode /*= EOpusMode::File*/)
	: Audio::IAudioEncoder(AverageBufferCallbackSize * 4, 65536)
	, LastValidFrameSize(0)
	, NumChannels(InInfo.NumChannels)
	, SampleRate(InInfo.SampleRate)
	, UncompressedFrameSize(GetNumSamplesForEncode(InFrameSize))
	, PrivateOpusState(nullptr)
	, PrivateOggEncapsulator(nullptr)
{
	PrivateOpusState = new FOpusEncoderPrivateState(InInfo, InMode == EOpusMode::VoiceStream);
	
	if (InMode == EOpusMode::File)
	{
		// Initialize the Ogg Encapsulator.
		PrivateOggEncapsulator = new FOggEncapsulator();
	}

	Init(InInfo);
}

FOpusEncoder::~FOpusEncoder()
{
	if (PrivateOpusState != nullptr)
	{
		delete PrivateOpusState;
		PrivateOpusState = nullptr;
	}

	if (PrivateOggEncapsulator != nullptr)
	{
		delete PrivateOggEncapsulator;
		PrivateOggEncapsulator = nullptr;
	}
}

int32 FOpusEncoder::GetNumSamplesForEncode(EOpusFrameSizes InFrameSize) const
{
	switch (InFrameSize)
	{
	case EOpusFrameSizes::Min:
		return ((float)SampleRate) * 0.0025f * NumChannels;
	case EOpusFrameSizes::Small:
		return ((float)SampleRate) * 0.005f * NumChannels;
	case EOpusFrameSizes::MediumLow:
		return ((float)SampleRate) * 0.01f * NumChannels;
	case EOpusFrameSizes::MediumHigh:
		return ((float)SampleRate) * 0.02f * NumChannels;
	case EOpusFrameSizes::High:
		return ((float)SampleRate) * 0.04f * NumChannels;
	case EOpusFrameSizes::Max:
		return ((float)SampleRate) * 0.06f * NumChannels;
	default:
		checkf(false, TEXT("Invalid frame size!"));
		return 0;
	}
}

int32 FOpusEncoder::GetNumSamplesForPreskip()
{
	return FMath::FloorToInt(((float)SampleRate) * PreskipDuration) * NumChannels;
}

int32 FOpusEncoder::GetCompressedPacketSize() const
{
	// Will return 0 if we haven't encoded any frames yet.
	return LastValidFrameSize;
}

int64 FOpusEncoder::SamplesRequiredPerEncode() const
{
	return UncompressedFrameSize;
}

bool FOpusEncoder::StartFile(const FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutFileStart)
{
	// If we're output a file, we're start with the ogg file.
	if (PrivateOggEncapsulator != nullptr)
	{
		// Generate header packet:
		ogg_packet HeaderPacket;
		TArray<uint8> PacketData = GenerateHeaderPacket(InQualityInfo.NumChannels, InQualityInfo.SampleRate);
		HeaderPacket.packet = PacketData.GetData();
		HeaderPacket.bytes = PacketData.Num();
		HeaderPacket.b_o_s = 1;
		HeaderPacket.e_o_s = 0;
		HeaderPacket.granulepos = 0;
		HeaderPacket.packetno = PacketIndex;
		PacketIndex++;

		PrivateOggEncapsulator->PushPacket(HeaderPacket);

		// Generate comment packet:
		ogg_packet CommentHeaderPacket;
		PacketData = GenerateCommentPacket();
		CommentHeaderPacket.packet = PacketData.GetData();
		CommentHeaderPacket.bytes = PacketData.Num();
		CommentHeaderPacket.b_o_s = 0;
		CommentHeaderPacket.e_o_s = 0;
		CommentHeaderPacket.granulepos = 0;
		CommentHeaderPacket.packetno = PacketIndex;
		PacketIndex++;

		PrivateOggEncapsulator->PushPacket(CommentHeaderPacket);

		// Flush all header pages out to ensure we start on a new page:
		PrivateOggEncapsulator->FlushPages(OutFileStart);

		// Inject silence into the encoder to satisfy out Preskip duration:
		TArray<float> Zeros;
		Zeros.AddZeroed(GetNumSamplesForPreskip());
		PushAudio(Zeros.GetData(), Zeros.Num());
	}

	// Otherwise, we don't need to do anything.
	return true;
}

bool FOpusEncoder::EncodeChunk(const TArray<float>& InAudio, TArray<uint8>& OutBytes)
{
#if PLATFORM_IOS
	// libOpus libraries must be compiled for all iOS archs before we can support Opus encoding.
	return false;
#else
	check(UncompressedFrameSize == InAudio.Num());

	const int32 NumFrames = InAudio.Num() / NumChannels;

	// Since Opus won't know how big the encoded chunk will be,
	// We allocate enough for the full decompressed buffer in OutFileStart,
	// then trim for what the actual compressed size was.
	OutBytes.Reset();
	OutBytes.AddUninitialized(InAudio.Num() * sizeof(float));

	int32 CompressedSize = opus_encode_float(PrivateOpusState->Encoder, InAudio.GetData(), NumFrames, OutBytes.GetData(), OutBytes.Num());
	
	if (CompressedSize < 0)
	{
		// If opus_encode_float returns a negative value, it means this isn't the number of compressed bytes,
		// but rather an error code !!
		const char* ErrorStr = opus_strerror(CompressedSize);
		ensureAlwaysMsgf(false, TEXT("Failed to encode Opus: error %d: %s"), CompressedSize, ANSI_TO_TCHAR(ErrorStr));
		
		OutBytes.Reset();
		return false;
	}
	else if (PrivateOggEncapsulator == nullptr)
	{
		// If we are just providing a stream of opus frames, we're done.
		check(CompressedSize < OutBytes.Num());
		check(CompressedSize != 0);

		// Trim to our actual output. Don't allow shrinking since we're going to be using this array again.
		OutBytes.SetNum(CompressedSize, EAllowShrinking::No);
		LastValidFrameSize = CompressedSize;
		return true;
	}
	else
	{
		OutBytes.SetNum(CompressedSize, EAllowShrinking::No);
		LastValidFrameSize = CompressedSize;

		// If we are encoding a .opus file, we need to push opus packets to an ogg stream.
		// Wrap opus frame in an ogg packet:
		ogg_packet& Packet = PrivateOggEncapsulator->CurrentPacket;
		Packet.packet = OutBytes.GetData();
		Packet.bytes = OutBytes.Num();
		Packet.b_o_s = 0;
		Packet.e_o_s = 0;
		Packet.granulepos = GranulePos;
		Packet.packetno = PacketIndex;

		// Copy data into the ogg stream:
		PrivateOggEncapsulator->PushPacket(Packet);
		
		// Replace OutBytes with the resulting Ogg Pages.
		OutBytes.Reset();
		PrivateOggEncapsulator->PopPages(OutBytes);

		GranulePos += NumFrames;
		PacketIndex++;

		return true;
	}
#endif // PLATFORM_IOS
}

bool FOpusEncoder::EndFile(TArray<uint8>& OutBytes)
{
	if (PrivateOggEncapsulator)
	{
		// If we are encoding an opus file, append a final empty packet to indicate the end of the stream.
		ogg_packet& Packet = PrivateOggEncapsulator->CurrentPacket;
		Packet.packet = nullptr;
		Packet.bytes = 0;
		Packet.b_o_s = 0;
		Packet.e_o_s = 1;
		Packet.granulepos = GranulePos;
		Packet.packetno = PacketIndex;

		PrivateOggEncapsulator->PopPages(OutBytes);

		// We're done, so clean up all state.
		delete PrivateOggEncapsulator;
		PrivateOggEncapsulator = nullptr;
	}

	return true;
}


#endif // !PLATFORM_TVOS
