// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Osc.h"
#include "DSP/Dsp.h"

namespace Audio
{
	IOscBase::IOscBase()
		: VoiceId(0)
		, SampleRate(44100.0f)
		, Nyquist(0.5f * SampleRate)
		, Freq(440.0f)
		, BaseFreq(1.0f)
		, Gain(1.0f)
		, ExternalGainMod(1.0f)
		, Phase(0.0f)
		, PhaseInc(0.0f)
		, PulseWidthBase(0.5f)
		, PulseWidthMod(0.0f)
		, PulseWidth(0.0f)
		, ModMatrix(nullptr)
		, FollowerOsc(nullptr)
		, bIsPlaying(false)
		, bChanged(false)
	{
	}

	IOscBase::IOscBase(const IOscBase&) = default;
	IOscBase::~IOscBase() = default;

	void IOscBase::Init(const float InSampleRate, const int32 InVoiceId, FModulationMatrix* InMatrix, const int32 ModMatrixStage)
	{
		VoiceId = InVoiceId;
		SetSampleRate(InSampleRate);

		bChanged = true;

		// Set up the patch destinations for the mod matrix if we've been given a mod matrix
		ModMatrix = InMatrix;
		if (ModMatrix)
		{
			ModFrequencyDest = ModMatrix->CreatePatchDestination(VoiceId, ModMatrixStage, 50.0f);
			ModPulseWidthDest = ModMatrix->CreatePatchDestination(VoiceId, ModMatrixStage, 1.0f);
			ModGainDest = ModMatrix->CreatePatchDestination(VoiceId, ModMatrixStage, 1.0f);
			ModAddDest = ModMatrix->CreatePatchDestination(VoiceId, ModMatrixStage, 50.0f);
			ModScaleDest = ModMatrix->CreatePatchDestination(VoiceId, ModMatrixStage, 1.0f);

#if MOD_MATRIX_DEBUG_NAMES
			ModFrequencyDest.Name = TEXT("ModFrequencyDest");
			ModPulseWidthDest.Name = TEXT("ModPulseWidthDest");
			ModGainDest.Name = TEXT("ModGainDest");
			ModAddDest.Name = TEXT("ModAddDest");
			ModScaleDest.Name = TEXT("ModScaleDest");
#endif
		}
	}

	void IOscBase::SetFrequency(const float InFreqBase)
	{ 
		if (InFreqBase != BaseFreq)
		{
			BaseFreq = InFreqBase;
			bChanged = true;
		}
	}

	void IOscBase::SetFrequencyMod(const float InFreqMod)
	{
		if (InFreqMod != FreqData.ExternalMod)
		{
			FreqData.ExternalMod = InFreqMod;
			bChanged = true;
		}
	}

	void IOscBase::SetNote(const float InNote)
	{
		const float MidiFreq = GetFrequencyFromMidi(InNote);
		SetFrequency(MidiFreq);
	}

	void IOscBase::SetCents(const float InCents)
	{
		if (FreqData.Cents != InCents)
		{
			FreqData.Cents = InCents;
			bChanged = true;
		}
	}

	void IOscBase::SetOctave(const float InOctave)
	{
		if (FreqData.Octave != InOctave)
		{
			FreqData.Octave = InOctave;
			bChanged = true;
		}
	}

	void IOscBase::SetSampleRate(const float InSampleRate)
	{
		if (InSampleRate != SampleRate)
		{
			SampleRate = InSampleRate;
			Nyquist = 0.5f * SampleRate;

			bChanged = true;
		}
	}

	void IOscBase::SetSemitones(const float InSemiTone)
	{
		if (FreqData.Semitones != InSemiTone)
		{
			FreqData.Semitones = InSemiTone;
			bChanged = true;
		}
	}

	void IOscBase::SetDetune(const float InDetune)
	{
		if (FreqData.Detune != InDetune)
		{
			FreqData.Detune = InDetune;
			bChanged = true;
		}
	}

	void IOscBase::SetPitchBend(const float InPitchBend)
	{
		if (FreqData.PitchBend != InPitchBend)
		{
			FreqData.PitchBend = InPitchBend;
			bChanged = true;
		}
	}

	void IOscBase::SetFreqScale(const float InFreqScale)
	{
		if (FreqData.Scale != InFreqScale)
		{
			FreqData.Scale = InFreqScale;
			bChanged = true;
		}
	}

	void IOscBase::Update()
	{
		// Compute the final output frequency

		if (ModMatrix)
		{
			bChanged |= ModMatrix->GetDestinationValue(VoiceId, ModFrequencyDest, FreqData.Mod);
			bChanged |= ModMatrix->GetDestinationValue(VoiceId, ModPulseWidthDest, PulseWidthMod);
		}

		if (bChanged)
		{
			bChanged = false;

			float FreqModSum = FreqData.Mod + FreqData.ExternalMod + FreqData.Detune + FreqData.PitchBend + 12.0f * FreqData.Octave + FreqData.Semitones + 0.01f * FreqData.Cents;
			float PulseWidthSum = PulseWidthBase + PulseWidthMod;

			PulseWidth = FMath::Clamp(PulseWidthSum, 0.02f, 0.98f);
			Freq = BaseFreq * FreqData.Scale * GetFrequencyMultiplier(FreqModSum);
			Freq = FMath::Clamp(Freq, -Nyquist, Nyquist);

			// Update the phase increment
			PhaseInc = Freq / SampleRate;
		}

	}

	void IOscBase::SetPulseWidth(const float InPulseWidth)
	{
		if (InPulseWidth != PulseWidthBase)
		{
			PulseWidthBase = FMath::Clamp(InPulseWidth, 0.0f, 1.0f);
			bChanged = true;
		}
	}

	void IOscBase::ResetPhase()
	{
		Phase = 0.0f;
	}

	void IOscBase::SetSlaveOsc(IOscBase* InSlaveOsc)
	{
		FollowerOsc = InSlaveOsc;
	}

	void IOscBase::SetFollowerOsc(IOscBase* InFollowerOsc)
	{
		FollowerOsc = InFollowerOsc;
	}

	void IOscBase::Reset()
	{
		Phase = 0.0f;
		ExternalGainMod = 1.0f;
		PulseWidthMod = 0.0f;
		FreqData.PitchBend = 0.0f;
		FreqData.Mod = 0.0f;
		FreqData.ExternalMod = 0.0f;
		FreqData.Detune = 0.0f;
		bChanged = true;
	}

	FOsc::FOsc()
		: TriangleSign(-1.0f)
		, DPW_z1(0.0f)
		, PulseWidthLerped(0.5f)
		, OscType(EOsc::Sine)
	{
	}
	
	FOsc::~FOsc()
	{
	}

	void FOsc::Start()
	{
		Reset();
		bIsPlaying = true;
		Update();
	}

	void FOsc::Stop()
	{
		bIsPlaying = false;
	}

	void FOsc::Reset()
	{
		IOscBase::Reset();

		// For these types our phase starts at 0.5
		if (OscType == EOsc::Saw || OscType == EOsc::Triangle)
		{
			Phase = 0.5f;
		}

		TriangleSign = -1.0f;
		DPW_z1 = 0.0f;
	}

	void FOsc::Update()
	{
		IOscBase::Update();

		PulseWidthLerped = PulseWidth;
	}

	float FOsc::Generate(float* AuxOutput)
	{
		if (!bIsPlaying)
		{
			return 0.0f;
		}

		float Output = 0.0f;
		const bool bWrapped = WrapPhase();

		switch (OscType)
		{
			case EOsc::Sine:
			{
				const float Radians = 2.0f * Phase * PI - PI;
				Output = FastSin3(-1.0f * Radians);
			}
			break;

			case EOsc::Saw:
			{
				// Two-sided wave-shaped sawtooth
				static const float A = FastTanh(1.5f);
				Output = GetBipolar(Phase);
				Output = FastTanh(1.5f * Output) / A;
				Output += PolySmooth(Phase, PhaseInc);
			}
			break;

			case EOsc::Square:
			{
				// First generate a smoothed sawtooth
				float SquareSaw1 = GetBipolar(Phase);
				SquareSaw1 += PolySmooth(Phase, PhaseInc);

				float CurrentPulseWidth = PulseWidthLerped.GetNextValue();

				// Create a second sawtooth that is phase-shifted based on the pulsewidth
				float NewPhase = 0.0f;
				if (PhaseInc > 0.0f)
				{
					NewPhase = Phase + CurrentPulseWidth;
					if (NewPhase >= 1.0f)
					{
						NewPhase -= 1.0f;
					}
				}
				else
				{
					NewPhase = Phase - CurrentPulseWidth;
					if (NewPhase <= 0.0f)
					{
						NewPhase += 1.0f;
					}
				}
				
				float SquareSaw2 = GetBipolar(NewPhase);
				SquareSaw2 += PolySmooth(NewPhase, PhaseInc);

				// Subtract 2 saws, then apply DC correction 
				// Simplified version of 
				// float Output = 0.5f * SquareSaw1 - 0.5f * SquareSaw2;
				// Output = 2.0f * (Output + CurrentPulseWidth) - 1.0f;
				return SquareSaw1 - SquareSaw2 + 2.0f * (CurrentPulseWidth - 0.5f);
			}

			case EOsc::Triangle:
			{
				// Square a simple saw wave, differentiate (add prev sample)
				// Then scale by a 
				if (bWrapped)
				{
					// Flip the sign of the square mod
					TriangleSign *= -1.0f;
				}

				// Get a saw wave
				const float Saw = GetBipolar(Phase);
				const float SawSquaredInvMod = (1.0f - Saw * Saw) * TriangleSign;

				// Perform differentiation by subtracting current squared saw
				const float Differentiated = SawSquaredInvMod - DPW_z1;
				DPW_z1 = SawSquaredInvMod;
				Output = Differentiated * SampleRate / (4.0f*Freq*(1.0f - PhaseInc));

				UpdatePhase();
			}
			break;

			case EOsc::Noise:
				Output = Noise.Generate();
				break;
		}

		// Update the LFO phase after computing LFO values
		UpdatePhase();

		// Apply the final matrix-mod gain
		return Output * Gain * ExternalGainMod;
	}

	float FOsc::PolySmooth(const float InPhase, const float InPhaseInc)
	{
		// Smooth out the edges of the saw based on its current frequency
		// using a polynomial to smooth it at the discontinuity. This 
		// limits aliasing by avoiding the infinite frequency at the discontinuity.

		float Output = 0.0f;

		// The current phase is on the left side of discontinuity
		if (InPhase > 1.0f - InPhaseInc)
		{
			const float Dist = (InPhase - 1.0f) / InPhaseInc;
			Output = -Dist*Dist - 2.0f * Dist - 1.0f;
		}
		// The current phase is on the right side of the discontinuity
		else if (InPhase < InPhaseInc)
		{
			// Distance into polynomial
			const float Dist = InPhase / InPhaseInc;
			Output = Dist*Dist - 2.0f * Dist + 1.0f;
		}

		return Output;
	}

}
