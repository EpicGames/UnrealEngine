// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/PerlinNoise.h"
#include "Math/UnrealMathUtility.h"

namespace Audio
{
	float PerlinValueNoise1D(const float InX, const int32 InNumOctaves)
	{
		check(InNumOctaves > 0);
		
		float Frequency = 1.f;
		float NoiseSum = 0.f;
		float NormalizeRange = 0;

		// To normalize we add up the max range of each octave, to keep everything in the range [-1,1]
		for (int32 Octave = 0; Octave < InNumOctaves; Octave++, Frequency *= 2.f)
		{
			const float FreqReciprocal = 1.f / Frequency;
			NoiseSum += (FMath::PerlinNoise1D(InX * Frequency) * FreqReciprocal);
			NormalizeRange += FreqReciprocal;
		}
	
		return NoiseSum / NormalizeRange;
	}

	void PerlinValueNoise1DBuffer(TArrayView<const float> InXBuffer, const float InOffset, const int32 InNumOctaves, TArrayView<float> OutNoiseBuffer)
	{
		float* OutNoisePtr = OutNoiseBuffer.GetData();
		const float* XPtr = InXBuffer.GetData();
		int32 Size = OutNoiseBuffer.Num();
		
		check(InXBuffer.Num() <= Size)

		for (int32 i = 0; i < Size; ++i)
		{
			const float X = *XPtr++;
			*OutNoisePtr++ = PerlinValueNoise1D(X + InOffset, InNumOctaves);
		}
	}
}