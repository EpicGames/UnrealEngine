// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Phaser.h"

namespace Audio
{
	FPhaser::FPhaser()
		: ControlSampleCount(0)
		, ControlRate(256)
		, Frequency(0.2f)
		, WetLevel(0.4f)
		, Feedback(0.2f)
		, LFOType(ELFO::Sine)
		, NumChannels(0)
		, bIsBiquadPhase(true)
	{
	}

	FPhaser::~FPhaser()
	{

	}

	void FPhaser::Init(const float SampleRate, const int32 InNumChannels)
	{
		check(NumChannels <= MaxNumChannels);
		NumChannels = InNumChannels;

		// Initialize all the APFs
		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			for (int32 ApfIndex = 0; ApfIndex < NumApfs; ++ApfIndex)
			{
				APFs[Channel][ApfIndex].Init(SampleRate, 1, EBiquadFilter::AllPass);
			}
		}

		const float ControlSampleRate = SampleRate / ControlRate;

		LFO.Init(ControlSampleRate);
		LFO.SetFrequency(Frequency);
		LFO.SetType(LFOType);
		LFO.Update();
		LFO.Start();

		// Setup the LFO oscillation ranges for APF cutoff frequencies
		// Use ratios of current sample rate
		const float SampleRateRatio = SampleRate / 48000.0f;
		APFFrequencyRanges[0].X = 16.0f * SampleRateRatio;
		APFFrequencyRanges[0].Y = 1600.0f * SampleRateRatio;

		APFFrequencyRanges[1].X = 33.0f * SampleRateRatio;
		APFFrequencyRanges[1].Y = 3300.0f * SampleRateRatio;

		APFFrequencyRanges[2].X = 48.0f * SampleRateRatio;
		APFFrequencyRanges[2].Y = 4800.0f * SampleRateRatio;

		APFFrequencyRanges[3].X = 98.0f * SampleRateRatio;
		APFFrequencyRanges[3].Y = 9800.0f * SampleRateRatio;

		APFFrequencyRanges[4].X = 160.0f * SampleRateRatio;
		APFFrequencyRanges[4].Y = 16000.0f * SampleRateRatio;

		// Make sure final APF frequency won't go above Nyquist
		APFFrequencyRanges[5].X = 220.0f * SampleRateRatio;
		APFFrequencyRanges[5].Y = 22000.0f * SampleRateRatio;

		FeedbackFrame[0] = 0.0f;
		FeedbackFrame[1] = 0.0f;
	}

	void FPhaser::SetFrequency(const float InFreqHz)
	{
		if (InFreqHz != Frequency)
		{
			Frequency = FMath::Max(InFreqHz, SMALL_NUMBER);
			LFO.SetFrequency(Frequency);
			LFO.Update();
		}
	}

	void FPhaser::SetWetLevel(const float InWetLevel)
	{
		if (InWetLevel != WetLevel)
		{
			WetLevel = FMath::Clamp(InWetLevel, 0.0f, 1.0f);
		}
	}

	void FPhaser::SetFeedback(const float InFeedback)
	{
		if (Feedback != InFeedback)
		{
			Feedback = FMath::Clamp(InFeedback, 0.0f, 1.0f);
		}
	}

	void FPhaser::SetLFOType(const ELFO::Type InLFOType)
	{
		if (InLFOType != LFOType)
		{
			LFOType = InLFOType;
			LFO.SetType(LFOType);
			LFO.Update();
		}
	}

	void FPhaser::SetQuadPhase(const bool bInQuadPhase)
	{
		bIsBiquadPhase = bInQuadPhase;
	}

	void FPhaser::ComputeNewCoefficients(const int32 ChannelIndex, const float LFOValue)
	{
		for (int32 APFIndex = 0; APFIndex < NumApfs; ++APFIndex)
		{
			// Get the APF biquad
			FBiquadFilter& BiquadFilter = APFs[ChannelIndex][APFIndex];

			// Compute a new cutoff frequency
			FVector2D& FreqRange = APFFrequencyRanges[APFIndex];

			const float NewFrequencyCutoff = FMath::Lerp(FreqRange.X, FreqRange.Y, LFOValue);
			BiquadFilter.SetFrequency(NewFrequencyCutoff);
		}
	}

	void FPhaser::ProcessAudioFrame(const float* InFrame, float* OutFrame)
	{
		ControlSampleCount = ControlSampleCount & (ControlRate - 1);
		if (ControlSampleCount == 0)
		{
			float LFOQuadOutput = 0.0f;
			float LFOOutput = LFO.Generate(&LFOQuadOutput);

			// Convert to unipolar
			LFOOutput = Audio::GetUnipolar(LFOOutput);
			LFOOutput = FMath::Clamp(LFOOutput, 0.0f, 1.0f);

			// Use the LFO to compute new filter coefficients
			ComputeNewCoefficients(0, LFOOutput);
			ComputeNewCoefficients(1, bIsBiquadPhase ? GetUnipolar(LFOQuadOutput) : LFOOutput);
		}

		++ControlSampleCount;

		float InputAudioFrame[2];

		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			InputAudioFrame[Channel] = InFrame[Channel] + FeedbackFrame[Channel] * Feedback;

			float InSample = InputAudioFrame[Channel];
			float OutSample = 0.0f;

			// Feed the audio through the APF chain
			for (int32 ApfIndex = 0; ApfIndex < NumApfs; ++ApfIndex)
			{
				APFs[Channel][ApfIndex].ProcessAudioFrame(&InSample, &OutSample);
				InSample = OutSample;
			}

			// Store the last output sample in the feedback frame
			FeedbackFrame[Channel] = OutSample;

			// Mix in the wet level for the output
			OutFrame[Channel] = WetLevel * OutSample + (1.0f - WetLevel) * InFrame[Channel];
		}
	}

	void FPhaser::ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer)
	{
		for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex += NumChannels)
		{
			ProcessAudioFrame(&InBuffer[SampleIndex], &OutBuffer[SampleIndex]);
		}
	}



}
