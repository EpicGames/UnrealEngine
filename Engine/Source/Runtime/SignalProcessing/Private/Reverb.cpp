// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Reverb.h"

#include "DSP/WaveTableOsc.h"

namespace Audio
{
	struct FReverbSettingsInternal
	{
		// Sample rate used for hard-coded delay line values
		static const int32 PresetSampleRate = 29761;

		// Ratio of real sample rate to the preset sample rate
		static float SampleRateRatio;

		// Actual sample rate of the output device
		static int32 SampleRate;

		static void Init(const int32 InSampleRate)
		{
			SampleRate = InSampleRate;
			SampleRateRatio = (float)SampleRate / PresetSampleRate;
		}
	};

	static inline float GetDelayMsec(int32 InSamples)
	{
		const float RealSamples = (float)InSamples * FReverbSettingsInternal::SampleRateRatio;
		return 1000.0f * RealSamples / FReverbSettingsInternal::SampleRate;
	}

	static FReverbSettingsInternal GReverbSettingsInternal;

	float FReverbSettingsInternal::SampleRateRatio = 1.0f;
	int32 FReverbSettingsInternal::SampleRate = 44100;

	FEarlyReflections::FEarlyReflections()
	{
	}

	FEarlyReflections::~FEarlyReflections()
	{
	}

	void FEarlyReflections::Init(const int32 InSampleRate)
	{
		FReverbSettingsInternal::Init(InSampleRate);

		const float SampleRate = (float)FReverbSettingsInternal::SampleRate;
		const int32 DefaultDelayLength = (int32)(0.2f * SampleRate);

		for (int32 Channel = 0; Channel < 2; ++Channel)
		{
			Data[Channel].PreDelay.Init(SampleRate, 1.0f);

			for (int32 i = 0; i < 4; ++i)
			{
				Data[Channel].APF[i].Init(SampleRate, 0.2f);
			}
			FMemory::Memzero(Data[Channel].DelayLineInputs, sizeof(float) * 4);
			FMemory::Memzero(Data[Channel].DelayLineOuputs, sizeof(float) * 4);
		}

		Data[0].APF[0].SetDelayMsec(GetDelayMsec(5 * 142));
		Data[0].APF[1].SetDelayMsec(GetDelayMsec(5 * 107));
		Data[0].APF[2].SetDelayMsec(GetDelayMsec(5 * 379));
		Data[0].APF[3].SetDelayMsec(GetDelayMsec(5 * 277));

		Data[1].APF[0].SetDelayMsec(GetDelayMsec(5 * 279));
		Data[1].APF[1].SetDelayMsec(GetDelayMsec(5 * 137));
		Data[1].APF[2].SetDelayMsec(GetDelayMsec(5 * 213));
		Data[1].APF[3].SetDelayMsec(GetDelayMsec(5 * 327));

		ApplySettings();
	}

	void FEarlyReflections::SetSettings(const FEarlyReflectionsSettings& InSettings)
	{
		Settings.Gain = FMath::Clamp(InSettings.Gain, 0.0f, 0.9999f);
		Settings.PreDelayMsec = FMath::Clamp(InSettings.PreDelayMsec, 0.0f, 1000.0f);
		Settings.Bandwidth = FMath::Clamp(InSettings.Bandwidth, 0.0f, 0.99999f);
		Settings.Decay = FMath::Clamp(InSettings.Decay, 0.0001f, 1.0f);
		Settings.Absorption = FMath::Clamp(InSettings.Absorption, 0.0f, 0.99999f);

		ApplySettings();
	}

	void FEarlyReflections::ApplySettings()
	{
		for (int32 Channel = 0; Channel < 2; ++Channel)
		{
			Data[Channel].PreDelay.SetDelayMsec(Settings.PreDelayMsec);
			Data[Channel].InputLPF.SetG(Settings.Bandwidth);
		}

		Data[0].LPF[0].SetG(FMath::Min(Settings.Absorption + 0.1f, 0.9999f));
		Data[0].LPF[1].SetG(FMath::Min(Settings.Absorption - 0.12f, 0.9999f));
		Data[0].LPF[2].SetG(FMath::Min(Settings.Absorption + 0.08f, 0.9999f));
		Data[0].LPF[3].SetG(FMath::Min(Settings.Absorption - 0.07f, 0.9999f));

		Data[1].LPF[0].SetG(FMath::Min(Settings.Absorption + 0.17f, 0.999f));
		Data[1].LPF[1].SetG(FMath::Min(Settings.Absorption - 0.07f, 0.999f));
		Data[1].LPF[2].SetG(FMath::Min(Settings.Absorption + 0.05f, 0.999f));
		Data[1].LPF[3].SetG(FMath::Min(Settings.Absorption - 0.11f, 0.999f));

		// 1/sqrt(2) * Decay
		MatrixScaleFactor = (1.0f - Settings.Decay) * 0.707f;
	}

	float FEarlyReflections::ProcessDelayLine(const float InSample, FDelayAPF& InAPF, FOnePoleLPF& InLPF)
	{
		float APFOut = InAPF.ProcessAudioSample(InSample);
		float LPFOut = InLPF.ProcessAudioSample(APFOut);
		return LPFOut;
	}

	void FEarlyReflections::ProcessAudioFrame(const float* InBuffer, const int32 InChannels, float* OutBuffer, const int32 OutChannels)
	{
		// don't deal with mono input/output here
		if (InChannels != 2 && OutChannels != 2)
		{
			return;
		}

		// Sum left and right channels as input into reverberator
		for (int32 Channel = 0; Channel < 2; ++Channel)
		{
			float InputSample = InBuffer[Channel];

			// Input -> PreDelay
			float PreDelayOut = Data[Channel].PreDelay.ProcessAudioSample(InputSample);

			// PreDelay -> InputLPF
			PreDelayOut *= Settings.Bandwidth;

			float InputLPFOut = Data[Channel].InputLPF.ProcessAudioSample(PreDelayOut);

			// Process each delay line input

			Data[Channel].DelayLineInputs[0] = 0.25f * InputLPFOut + MatrixScaleFactor * (-Data[Channel].DelayLineOuputs[1] + Data[Channel].DelayLineOuputs[2]);
			Data[Channel].DelayLineInputs[1] = 0.25f * InputLPFOut + MatrixScaleFactor * ( Data[Channel].DelayLineOuputs[0] + Data[Channel].DelayLineOuputs[3]);
			Data[Channel].DelayLineInputs[2] = 0.25f * InputLPFOut + MatrixScaleFactor * ( Data[Channel].DelayLineOuputs[0] - Data[Channel].DelayLineOuputs[3]);
			Data[Channel].DelayLineInputs[3] = 0.25f * InputLPFOut + MatrixScaleFactor * (-Data[Channel].DelayLineOuputs[1] - Data[Channel].DelayLineOuputs[2]);

			OutBuffer[Channel] = 0.0f;
			for (int32 i = 0; i < 4; ++i)
			{
				Data[Channel].DelayLineOuputs[i] = ProcessDelayLine(Data[Channel].DelayLineInputs[i], Data[Channel].APF[i], Data[Channel].LPF[i]);
				OutBuffer[Channel] += Data[Channel].DelayLineOuputs[i];
			}

			// Apply the early reflections output gain setting
			OutBuffer[Channel] = (1.0f - Settings.Gain) * InBuffer[Channel] + OutBuffer[Channel] * Settings.Gain;
		}
	}

	FPlateReverb::FPlateReverb()
		: bEnableLateReflections(true)
		, bEnableEarlyReflections(true)
	{
		FMemory::Memzero(LeftTaps, sizeof(float)*NumTaps);
		FMemory::Memzero(RightTaps, sizeof(float)*NumTaps);
	}

	FPlateReverb::~FPlateReverb()
	{
	}

	void FPlateReverb::Init(const int32 InSampleRate)
	{
		FReverbSettingsInternal::Init(InSampleRate);

		EarlyReflections.Init(InSampleRate);

		const float SampleRate = (float)FReverbSettingsInternal::SampleRate;
		PreDelay.Init(SampleRate, 2.0f);

		const float DefaultDelayLength = 0.2f;

		APF1.Init(SampleRate, DefaultDelayLength);
		APF1.SetDelayMsec(GetDelayMsec(142));

		APF2.Init(SampleRate, DefaultDelayLength);
		APF2.SetDelayMsec(GetDelayMsec(107));

		APF3.Init(SampleRate, DefaultDelayLength);
		APF3.SetDelayMsec(GetDelayMsec(379));

		APF4.Init(SampleRate, DefaultDelayLength);
		APF4.SetDelayMsec(GetDelayMsec(277));

		if (!LFO.IsValid())
		{
			LFO = FWaveTableOsc::CreateWaveTable(EWaveTable::SineWaveTable);
		}
		LFO->Init(SampleRate, 1.0f);
		LFO->SetScaleAdd(0.5f, 0.5f);

		LeftPlate.ModulatedAPF.Init(SampleRate, DefaultDelayLength);
		LeftPlate.ModulatedBaseDelayMsec = GetDelayMsec(908);
		LeftPlate.ModulatedDeltaDelayMsec = GetDelayMsec(16);

		LeftPlate.Delay1.Init(SampleRate, DefaultDelayLength);
		LeftPlate.Delay1.SetDelayMsec(GetDelayMsec(4217));

		LeftPlate.APF.Init(SampleRate, DefaultDelayLength);
		LeftPlate.APF.SetDelayMsec(GetDelayMsec(2656));

		LeftPlate.Delay2.Init(SampleRate, DefaultDelayLength);
		LeftPlate.Delay2.SetDelayMsec(GetDelayMsec(3136));

		RightPlate.ModulatedAPF.Init(SampleRate, DefaultDelayLength);
		RightPlate.ModulatedBaseDelayMsec = GetDelayMsec(672);
		RightPlate.ModulatedDeltaDelayMsec = GetDelayMsec(16);

		RightPlate.Delay1.Init(SampleRate, DefaultDelayLength);
		RightPlate.Delay1.SetDelayMsec(GetDelayMsec(4453));

		RightPlate.APF.Init(SampleRate, DefaultDelayLength);
		RightPlate.APF.SetDelayMsec(GetDelayMsec(1800));

		RightPlate.Delay2.Init(SampleRate, DefaultDelayLength);
		RightPlate.Delay2.SetDelayMsec(GetDelayMsec(3720));

		LeftTaps[0] = GetDelayMsec(266);
		LeftTaps[1] = GetDelayMsec(2974);
		LeftTaps[2] = GetDelayMsec(1913);
		LeftTaps[3] = GetDelayMsec(1996);
		LeftTaps[4] = GetDelayMsec(1990);
		LeftTaps[5] = GetDelayMsec(187);
		LeftTaps[6] = GetDelayMsec(1066);

		RightTaps[0] = GetDelayMsec(353);
		RightTaps[1] = GetDelayMsec(3627);
		RightTaps[2] = GetDelayMsec(1228);
		RightTaps[3] = GetDelayMsec(2673);
		RightTaps[4] = GetDelayMsec(2111);
		RightTaps[5] = GetDelayMsec(335);
		RightTaps[6] = GetDelayMsec(121);

		ApplySettings();
	}

	void FPlateReverb::EnableLateReflections(const bool bInEnableLateReflections)
	{
		bEnableLateReflections = bInEnableLateReflections;
	}

	void FPlateReverb::EnableEarlyReflections(const bool bInEnableEarlyReflections)
	{
		bEnableEarlyReflections = bInEnableEarlyReflections;
	}

	void FPlateReverb::SetSettings(const FPlateReverbSettings& InSettings)
	{
		// Apply early reflections settings
		EarlyReflections.SetSettings(InSettings.EarlyReflections);

		Settings.LateDelayMsec = FMath::Clamp(InSettings.LateDelayMsec, 0.0f, 2000.0f);
		Settings.LateGain = FMath::Min(InSettings.LateGain, 0.0f);
		Settings.Bandwidth = FMath::Clamp(InSettings.Bandwidth, 0.0f, 0.99999f);
		Settings.Dampening = FMath::Clamp(InSettings.Dampening, 0.0f, 0.999999f);
		Settings.Diffusion = FMath::Clamp(InSettings.Diffusion, 0.0f, 1.0f);
		Settings.Decay = FMath::Clamp(InSettings.Decay, 0.0001f, 1.0f);
		Settings.Density = FMath::Clamp(InSettings.Density, 0.0f, 1.0f);
		Settings.Wetness = FMath::Clamp(InSettings.Wetness, 0.0f, 10.0f);

		ApplySettings();
	}

	void FPlateReverb::ApplySettings()
	{
		PreDelay.SetDelayMsec(Settings.LateDelayMsec);
		PreDelay.SetOutputAttenuationDB(Settings.LateGain);

		InputLPF.SetG(1.0f - Settings.Bandwidth);

		APF1.SetG(Settings.Diffusion);
		APF2.SetG(Settings.Diffusion);
		APF3.SetG(Settings.Diffusion - 0.125f);
		APF4.SetG(Settings.Diffusion - 0.125f);

		LeftPlate.ModulatedAPF.SetG(-Settings.Density);
		LeftPlate.LPF.SetG(Settings.Dampening);
		LeftPlate.APF.SetG(Settings.Density - 0.15f);

		RightPlate.ModulatedAPF.SetG(-Settings.Density);
		RightPlate.LPF.SetG(Settings.Dampening);
		RightPlate.APF.SetG(Settings.Density - 0.15f);
	}

	void FPlateReverb::ProcessAudioFrame(const float* InBuffer, const int32 InChannels, float* OutBuffer, const int32 OutChannels)
	{
		// TODO: mix for mono channels
		if (InChannels == 1 && OutChannels == 1)
		{
			return;
		}

		// If neither reflections are enabled, just do a pass through
		if (!bEnableLateReflections && !bEnableEarlyReflections)
		{
			OutBuffer[0] = InBuffer[0];
			OutBuffer[1] = InBuffer[1];
			return;
		}

		float InputSample = 0.0f;
		float EarlyReflectionsOutput[2];

		if (bEnableEarlyReflections)
		{
			EarlyReflections.ProcessAudioFrame(InBuffer, InChannels, EarlyReflectionsOutput, OutChannels);
		}
		else
		{
			// Pass input buffer to early reflections output if we ahve early reflections disabled
			EarlyReflectionsOutput[0] = InBuffer[0];
			EarlyReflectionsOutput[1] = InBuffer[1];
		}

		InputSample = 0.5f * (EarlyReflectionsOutput[0] + EarlyReflectionsOutput[1]);

		if (bEnableLateReflections)
		{
			// -------------------
			// INPUT DIFFUSION

			// InputSample -> PreDelay
			float PreDelayOut = PreDelay.ProcessAudioSample(InputSample);

			// Scale by bandwidth
			PreDelayOut *= Settings.Bandwidth;

			// PreDelay -> InputLPFOut
			float InputLPFOut = InputLPF.ProcessAudioSample(PreDelayOut);

			// InputLPFOut -> APF1
			float APF1Out = APF1.ProcessAudioSample(InputLPFOut);

			// APF1 -> APF2
			float APF2Out = APF2.ProcessAudioSample(APF1Out);;

			// APF2 -> APF3
			float APF3Out = APF3.ProcessAudioSample(APF2Out);

			// APF3 -> APF4
			float APF4Out = APF4.ProcessAudioSample(APF3Out);

			// -------------------
			// Modulation

			// Update LFO modulator and apply results to modulated APFs in plates
			float NormalPhaseOut = 0.0f;
			float QuadPhaseOut = 0.0f;

			LFO->Generate(&NormalPhaseOut, &QuadPhaseOut);

			float LeftPlateDelayMsec = LeftPlate.ModulatedBaseDelayMsec + NormalPhaseOut * LeftPlate.ModulatedDeltaDelayMsec;
			float RightPlateDelayMsec = RightPlate.ModulatedBaseDelayMsec + QuadPhaseOut * RightPlate.ModulatedDeltaDelayMsec;

			LeftPlate.ModulatedAPF.SetDelayMsec(LeftPlateDelayMsec);
			RightPlate.ModulatedAPF.SetDelayMsec(RightPlateDelayMsec);

			// -------------------
			// RIGHT PLATE

			// Get input into right plate by adding the left plate's previous sample (feedback path)
			float RightPlateInput = APF4Out + LeftPlate.PreviousSample;

			// Input -> ModulatedAPF
			float RightPlateModulatedAPFOut = RightPlate.ModulatedAPF.ProcessAudioSample(RightPlateInput);

			// ModulatedAPF -> Delay1
			float RightPlateDelay1Out = RightPlate.Delay1.ProcessAudioSample(RightPlateModulatedAPFOut);

			// Apply dampening
			RightPlateDelay1Out *= (1.0f - Settings.Dampening);

			// Delay1 -> LPF
			float RightPlateLPFOut = RightPlate.LPF.ProcessAudioSample(RightPlateDelay1Out);

			// Apply decay
			RightPlateLPFOut *= (1.0f - Settings.Decay);

			// LPF -> APF
			float RightPlateAPFOut = RightPlate.APF.ProcessAudioSample(RightPlateLPFOut);

			// APF-> Delay2
			float RightPlateDelay2Out = RightPlate.Delay2.ProcessAudioSample(RightPlateAPFOut);

			// Write to PreviousSample for feedback
			RightPlate.PreviousSample = (1.0f - Settings.Decay) * RightPlateDelay2Out;

			// -------------------
			// LEFT PLATE

			// Get input into right plate by adding the left plate's previous sample (feedback path)
			float LeftPlateInput = APF4Out + RightPlate.PreviousSample;

			// Input -> ModulatedAPF
			float LeftPlateModulatedAPFOut = LeftPlate.ModulatedAPF.ProcessAudioSample(LeftPlateInput);

			// ModulatedAPF -> Delay1
			float LeftPlateDelay1Out = LeftPlate.Delay1.ProcessAudioSample(LeftPlateModulatedAPFOut);

			// Apply dampening
			LeftPlateDelay1Out *= (1.0f - Settings.Dampening);

			// Delay1 -> LPF
			float LeftPlateLPFOut = LeftPlate.LPF.ProcessAudioSample(LeftPlateDelay1Out);

			// Apply decay
			LeftPlateLPFOut *= (1.0f - Settings.Decay);

			// LPF -> APF
			float LeftPlateAPFOut = LeftPlate.APF.ProcessAudioSample(LeftPlateLPFOut);
			
			// APF-> Delay2
			float LeftPlateDelay2Out = LeftPlate.Delay2.ProcessAudioSample(LeftPlateAPFOut);

			// Write to PreviousSample for feedback
			LeftPlate.PreviousSample = (1.0f - Settings.Decay) * LeftPlateDelay2Out;

			// --------------------
			// TAPOUTS

			// Now compute the outputs by reading taps from various delay lines
			float LeftOut = RightPlate.Delay1.ReadDelayAt(LeftTaps[0])  // a[266]
				+ RightPlate.Delay1.ReadDelayAt(LeftTaps[1])			// a[2974]
				- RightPlate.APF.ReadDelayAt(LeftTaps[2])				// b[1913]
				+ RightPlate.Delay2.ReadDelayAt(LeftTaps[3])			// c[1996]
				- LeftPlate.Delay1.ReadDelayAt(LeftTaps[4])				// d[1990]
				- LeftPlate.APF.ReadDelayAt(LeftTaps[5])				// e[187]
				- LeftPlate.Delay2.ReadDelayAt(LeftTaps[6]);			// f[1066]

			float RightOut = LeftPlate.Delay1.ReadDelayAt(RightTaps[0]) // d[353]
				+ LeftPlate.Delay1.ReadDelayAt(RightTaps[1])			// d[3627]
				- LeftPlate.APF.ReadDelayAt(RightTaps[2])				// e[1228]
				+ LeftPlate.Delay2.ReadDelayAt(RightTaps[3])			// f[2673]
				- RightPlate.Delay1.ReadDelayAt(RightTaps[4])			// a[2111]
				- RightPlate.APF.ReadDelayAt(RightTaps[5])				// b[335]
				- RightPlate.Delay2.ReadDelayAt(RightTaps[6]);			// c[121]

			OutBuffer[0] = Settings.Wetness * LeftOut;
			OutBuffer[1] = Settings.Wetness * RightOut;

		}
		else
		{
			OutBuffer[0] = Settings.Wetness * EarlyReflectionsOutput[0];
			OutBuffer[1] = Settings.Wetness * EarlyReflectionsOutput[1];
		}

	}
}
