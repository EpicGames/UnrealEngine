// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/ReverbFast.h"
#include "DSP/FloatArrayMath.h"
#include "AudioMixer.h"

namespace Audio {
	FPlateReverbFastSettings::FPlateReverbFastSettings()
		: bEnableEarlyReflections(true)
		, bEnableLateReflections(true)
		, QuadBehavior(EQuadBehavior::StereoOnly)
	{}

	bool FPlateReverbFastSettings::operator==(const FPlateReverbFastSettings& Other) const
	{
		bool bIsEqual = (
			(Other.EarlyReflections == EarlyReflections) &&
			(Other.LateReflections == LateReflections) &&
			(Other.bEnableEarlyReflections == bEnableEarlyReflections) &&
			(Other.bEnableLateReflections == bEnableLateReflections) &&
			(Other.QuadBehavior == QuadBehavior));
		
		return bIsEqual;
	}

	bool FPlateReverbFastSettings::operator!=(const FPlateReverbFastSettings& Other) const
	{
		return !(*this == Other);
	}

	const FPlateReverbFastSettings FPlateReverbFast::DefaultSettings;

	FPlateReverbFast::FPlateReverbFast(float InSampleRate, int32 InMaxInternalBufferSamples, const FPlateReverbFastSettings& InSettings)
		: SampleRate(InSampleRate)
		, EarlyReflections(InSampleRate, InMaxInternalBufferSamples)
		, LateReflections(InSampleRate, InMaxInternalBufferSamples, InSettings.LateReflections)
	{
		SetSettings(InSettings);
	}

	FPlateReverbFast::~FPlateReverbFast()
	{}

	void FPlateReverbFast::SetSettings(const FPlateReverbFastSettings& InSettings)
	{
		// If the early reflections are getting disabled, need to flush the audio.
		// So that the audio history is empty when they are re-enabled.
		if (Settings.bEnableEarlyReflections && !InSettings.bEnableEarlyReflections)
		{
			EarlyReflections.FlushAudio();
		}

		// If the late reflections are getting disabled, need to flush the audio.
		// So that the audio history is empty when they are re-enabled.
		if (Settings.bEnableLateReflections && !InSettings.bEnableLateReflections)
		{
			LateReflections.FlushAudio();
		}

		Settings = InSettings;
		
		ApplySettings();
	}

	const FPlateReverbFastSettings& FPlateReverbFast::GetSettings() const
	{
		return Settings;
	}
	// Process a buffer of input audio samples.
	void FPlateReverbFast::ProcessAudio(const FAlignedFloatBuffer& InSamples, const int32 InNumChannels, FAlignedFloatBuffer& OutSamples, const int32 OutNumChannels)
	{
		if(InSamples.Num() == 0)
		{
			OutSamples.Reset(0);
			return;
		}
		
		ScaledInputBuffer.Reset(InSamples.Num());
		ScaledInputBuffer.AddUninitialized(InSamples.Num());
		check(ScaledInputBuffer.Num() == InSamples.Num());

		FMemory::Memcpy(ScaledInputBuffer.GetData(), InSamples.GetData(), InSamples.Num() * sizeof(float));

		checkf((1 == InNumChannels) || (2 == InNumChannels), TEXT("FPlateReverbFast only supports 1 or 2 channel inputs."))
		checkf(OutNumChannels >= 2, TEXT("FPlateReverbFast requires at least 2 output channels."))

		// Determine number of frames and output size.
		const int32 InNum = InSamples.Num();
		const int32 InNumFrames = InNum / InNumChannels;

		if (0 == InNum)
		{
			// If not input samples, then reset output samples and return
			OutSamples.Reset(0);
			return;
		}

		if (!Settings.bEnableEarlyReflections && !Settings.bEnableLateReflections)
		{
			// Zero output buffers if all reverb is disabled. 
			const int32 OutNum = InNumFrames * OutNumChannels;

			OutSamples.Reset(OutNum);
			OutSamples.AddZeroed(OutNum);
			return;
		}

		// Resize internal buffers	
		FrontLeftReverbSamples.Reset(InNumFrames);
		FrontRightReverbSamples.Reset(InNumFrames);

		FrontLeftReverbSamples.AddUninitialized(InNumFrames);
		FrontRightReverbSamples.AddUninitialized(InNumFrames);

		if (Settings.bEnableEarlyReflections && !Settings.bEnableLateReflections)
		{
			// Only generate early reflections.
			EarlyReflections.ProcessAudio(ScaledInputBuffer, InNumChannels, FrontLeftReverbSamples, FrontRightReverbSamples);
		}
		else if (!Settings.bEnableEarlyReflections && Settings.bEnableLateReflections)
		{
			// Only generate late reflections.
			LateReflections.ProcessAudio(ScaledInputBuffer, InNumChannels, FrontLeftReverbSamples, FrontRightReverbSamples);
		}
		else if (Settings.bEnableEarlyReflections && Settings.bEnableLateReflections)
		{
			// Resize internal buffers
			FrontLeftLateReflectionsSamples.Reset(InNumFrames);
			FrontRightLateReflectionsSamples.Reset(InNumFrames);
			FrontLeftEarlyReflectionsSamples.Reset(InNumFrames);
			FrontRightEarlyReflectionsSamples.Reset(InNumFrames);

			FrontLeftLateReflectionsSamples.AddUninitialized(InNumFrames);
			FrontRightLateReflectionsSamples.AddUninitialized(InNumFrames);
			FrontLeftEarlyReflectionsSamples.AddUninitialized(InNumFrames);
			FrontRightEarlyReflectionsSamples.AddUninitialized(InNumFrames);

			// Generate both early reflections and late reflections 
			EarlyReflections.ProcessAudio(ScaledInputBuffer, InNumChannels, FrontLeftEarlyReflectionsSamples, FrontRightEarlyReflectionsSamples);
			LateReflections.ProcessAudio(ScaledInputBuffer, InNumChannels, FrontLeftLateReflectionsSamples, FrontRightLateReflectionsSamples);
			// Add early and late reflections together.
			ArraySum(FrontLeftEarlyReflectionsSamples, FrontLeftLateReflectionsSamples, FrontLeftReverbSamples);
			ArraySum(FrontRightEarlyReflectionsSamples, FrontRightLateReflectionsSamples, FrontRightReverbSamples);
		}


		// Interleave and upmix
		InterleaveAndMixOutput(FrontLeftReverbSamples, FrontRightReverbSamples, OutSamples, OutNumChannels);
	}

	void FPlateReverbFast::ClampSettings(FPlateReverbFastSettings& InOutSettings)
	{
		// Clamp settings for this object and member objects.
		FLateReflectionsFast::ClampSettings(InOutSettings.LateReflections);
		FEarlyReflectionsFast::ClampSettings(InOutSettings.EarlyReflections);
	}

	// Copy reverberated samples to interleaved output samples. Map channels according to internal settings.
	// InFrontLeftSamples and InFrontRightSamples may be modified in-place.
	void FPlateReverbFast::InterleaveAndMixOutput(const FAlignedFloatBuffer& InFrontLeftSamples, const FAlignedFloatBuffer& InFrontRightSamples, FAlignedFloatBuffer& OutSamples, const int32 OutNumChannels)
	{
		check(InFrontLeftSamples.Num() == InFrontRightSamples.Num())

		const int32 InNumFrames = InFrontLeftSamples.Num();
		const int32 OutNum = OutNumChannels * InNumFrames;

		// Resize output buffer
		OutSamples.Reset(OutNum);
		OutSamples.AddUninitialized(OutNum);
		FMemory::Memset(OutSamples.GetData(), 0, sizeof(float) * OutNum);

		// Interleave / mix reverb audio into output buffer
		if (2 == OutNumChannels)
		{
			// Stereo interleaved output
			BufferInterleave2ChannelFast(FrontLeftReverbSamples, FrontRightReverbSamples, OutSamples);
		}
		else
		{
			if ((OutNumChannels < 5) || (FPlateReverbFastSettings::EQuadBehavior::StereoOnly == Settings.QuadBehavior))
			{
				// We do not handle any quad reverb mapping when OutNumChannels is less than 5

				float* LeftSampleData = FrontLeftReverbSamples.GetData();
				float* RightSampleData = FrontRightReverbSamples.GetData();
				float* OutSampleData = OutSamples.GetData();
				
				int32 OutPos = 0;
				for (int32 i = 0; i < InNumFrames; i++, OutPos += OutNumChannels)
				{
					OutSampleData[OutPos + EAudioMixerChannel::FrontLeft] = LeftSampleData[i];
					OutSampleData[OutPos + EAudioMixerChannel::FrontRight] = RightSampleData[i];
				}
			}
			else
			{
				// There are 5 or more output channels and quad mapping is enabled.

				LeftAttenuatedSamples.Reset(InNumFrames);
				RightAttenuatedSamples.Reset(InNumFrames);

				LeftAttenuatedSamples.AddUninitialized(InNumFrames);
				RightAttenuatedSamples.AddUninitialized(InNumFrames);

				// Reduce volume of output reverbs.
				ArrayMultiplyByConstant(FrontLeftReverbSamples, 0.5f, LeftAttenuatedSamples);
				ArrayMultiplyByConstant(FrontRightReverbSamples, 0.5f, RightAttenuatedSamples);

				const float* FrontLeftSampleData = LeftAttenuatedSamples.GetData();
				const float* FrontRightSampleData = RightAttenuatedSamples.GetData();
				// WARNING: this pointer will alias other pointers in this scope. Be conscious of RESTRICT keyword in any called functions.
				const float* BackLeftSampleData = nullptr;
				// WARNING: this pointer will alias other pointers in this scope. Be conscious of RESTRICT keyword in any called functions.
				const float* BackRightSampleData = nullptr;

				// Map quads by asigning pointers.
				switch (Settings.QuadBehavior)
				{
					case FPlateReverbFastSettings::EQuadBehavior::QuadFlipped:
						// Left and right are flipped.
						BackLeftSampleData = FrontRightSampleData;
						BackRightSampleData = FrontLeftSampleData;
						break;
					case FPlateReverbFastSettings::EQuadBehavior::QuadMatched:
					default:
						// Left and right are matched.
						BackLeftSampleData = FrontLeftSampleData;
						BackRightSampleData = FrontRightSampleData;
				}

				// Interleave to output
				float* OutSampleData = OutSamples.GetData();
				int32 OutPos = 0;
				for (int32 i = 0; i < InNumFrames; i++, OutPos += OutNumChannels)
				{
					OutSampleData[OutPos + EAudioMixerChannel::FrontLeft] = FrontLeftSampleData[i];
					OutSampleData[OutPos + EAudioMixerChannel::FrontRight] = FrontRightSampleData[i];
					OutSampleData[OutPos + EAudioMixerChannel::BackLeft] = BackLeftSampleData[i];
					OutSampleData[OutPos + EAudioMixerChannel::BackRight] = BackRightSampleData[i];
				}
			}
		}
	}

	void FPlateReverbFast::ApplySettings()
	{
		EarlyReflections.SetSettings(Settings.EarlyReflections);
		LateReflections.SetSettings(Settings.LateReflections);
	}
}
