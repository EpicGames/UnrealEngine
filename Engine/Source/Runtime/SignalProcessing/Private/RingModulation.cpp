// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/RingModulation.h"
#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FRingModulation::FRingModulation()
		: ModulationFrequency(800.0f)
		, ModulationDepth(0.5f)
		, DryLevel(0.0f)
		, WetLevel(1.0f)
		, Scale(1.0f)
		, NumChannels(0)
	{
		UpdateScale();
	}

	FRingModulation::~FRingModulation()
	{

	}

	void FRingModulation::Init(const float InSampleRate, const int32 InNumChannels)
	{
		Osc.Init(InSampleRate);
		Osc.SetFrequency(ModulationFrequency);
		Osc.Update();
		Osc.Start();

		NumChannels = InNumChannels;
	}

	void FRingModulation::UpdateScale()
	{
		Scale = WetLevel * ModulationDepth;
	}

	void FRingModulation::SetExternalPatchSource(Audio::FPatchOutputStrongPtr InPatch)
	{
		Patch = InPatch;
	}

	void FRingModulation::SetWetLevel(const float InWetLevel)
	{
		WetLevel = InWetLevel;
		UpdateScale();
	}

	void FRingModulation::SetModulatorWaveType(const EOsc::Type InType)
	{
		Osc.SetType(InType);
	}

	void FRingModulation::SetModulationFrequency(const float InModulationFrequency)
	{
		Osc.SetFrequency(FMath::Clamp(InModulationFrequency, 10.0f, 10000.0f));
		Osc.Update();
	}

	void FRingModulation::SetModulationDepth(const float InModulationDepth)
	{
		ModulationDepth = FMath::Clamp(InModulationDepth, -1.0f, 1.0f);
		UpdateScale();
	}

	void FRingModulation::ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer)
	{
		if (ModulationBuffer.Num() != InNumSamples * NumChannels)
		{
			ModulationBuffer.Reset();
			ModulationBuffer.AddZeroed(InNumSamples * NumChannels);
		}

		// If we have an external patch source to modulate against, copy that data into the modulation buffer
		if (Patch.IsValid())
		{
			Patch->PopAudio(ModulationBuffer.GetData(), ModulationBuffer.Num(), true);
		}
		else
		{
			// Write the oscillator data into the modulation buffer
			float* ModulationBufferPtr = ModulationBuffer.GetData();
			for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
			{
				ModulationBufferPtr[SampleIndex] = Osc.Generate();
			}
		}

		TArrayView<const float> InBufferView(InBuffer, InNumSamples);
		TArrayView<float> ModulationBufferView(ModulationBuffer.GetData(), InNumSamples);
		TArrayView<float> OutBufferView(OutBuffer, InNumSamples);

		// Multiply the input buffer by the modulation buffer in-place
		Audio::ArrayMultiplyInPlace(InBufferView, ModulationBufferView);

		// Perform a buffer weighted sum of the modulation buffer with the Scale value and the dry level as weights
		Audio::ArrayWeightedSum(InBufferView, DryLevel, ModulationBufferView, Scale, OutBufferView);
	}


}
