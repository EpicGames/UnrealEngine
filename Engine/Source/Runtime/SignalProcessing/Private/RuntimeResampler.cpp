// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/RuntimeResampler.h"

#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/MultichannelBuffer.h"
#include "HAL/PlatformMath.h"
#include "Math/UnrealMathUtility.h"

// Throughout this file, all variables with the FP suffix are in the 16.16 fixed point format.

namespace Audio
{
	const float FRuntimeResampler::MaxFrameRatio = 100.f;
	const float FRuntimeResampler::MinFrameRatio = 0.01f;
	const int32 FRuntimeResampler::FPScale = 65536;
	const float FRuntimeResampler::FPScaleFloat = 65536;

	FRuntimeResampler::FRuntimeResampler(int32 InNumChannels)
	{
		Reset(InNumChannels);
	}

	void FRuntimeResampler::Reset(const int32 InNumChannels)
	{
		check(InNumChannels > 0);
		CurrentInputFrameIndexFP = 0;
		CurrentFrameRatioFP = FPScale;
		TargetFrameRatioFP = FPScale;
		FrameRatioFrameDeltaFP = 0;
		NumFramesToInterpolate = 0;

		PreviousFrame.Empty();
		PreviousFrame.AddZeroed(InNumChannels);

		TempInputPointers.Empty();
		TempInputPointers.AddZeroed(InNumChannels);

		TempOutputPointers.Empty();
		TempOutputPointers.AddZeroed(InNumChannels);
	}

	void FRuntimeResampler::SetFrameRatio(float InRatio, int32 InDesiredNumFramesToInterpolate)
	{
		if (ensureMsgf((InRatio >= MinFrameRatio) && (InRatio <= MaxFrameRatio), TEXT("The frame ratio (%f) must be between %f and %f."), InRatio, MinFrameRatio, MaxFrameRatio))
		{
			const uint32 RatioFP = FMath::RoundToInt32(InRatio * FPScale);

			if (InDesiredNumFramesToInterpolate <= 0 || RatioFP == CurrentFrameRatioFP)
			{
				// Set frame ratio immediately. 
				CurrentFrameRatioFP = RatioFP;
				TargetFrameRatioFP = RatioFP;
				FrameRatioFrameDeltaFP = 0;
				NumFramesToInterpolate = 0;
			}
			else
			{
				// Interpolate frame ratio over output frames. 
				TargetFrameRatioFP = RatioFP;
				const uint32 RatioChangeAbsolute = uint32(FMath::Abs(int32(TargetFrameRatioFP - CurrentFrameRatioFP)));

				// Round up the frame delta so it's always at least 1
				const uint32 FrameDeltaAbsolute = FMath::DivideAndRoundUp<uint32>(RatioChangeAbsolute, InDesiredNumFramesToInterpolate);

				// Fudge the number of frames over which we do the interpolation to make sure we land close to the target
				check(FrameDeltaAbsolute > 0);
				NumFramesToInterpolate = FMath::DivideAndRoundDown(RatioChangeAbsolute, FrameDeltaAbsolute);

				if (NumFramesToInterpolate > 0)
				{
					FrameRatioFrameDeltaFP = int32(FrameDeltaAbsolute);
					if (TargetFrameRatioFP < CurrentFrameRatioFP)
					{
						FrameRatioFrameDeltaFP = -FrameRatioFrameDeltaFP;
					}
				}
				else
				{
					CurrentFrameRatioFP = RatioFP;
					TargetFrameRatioFP = RatioFP;
					FrameRatioFrameDeltaFP = 0;
					check(NumFramesToInterpolate == 0);
				}
			}
		}
	}

	int64 FRuntimeResampler::MapOutputFrameToInputFrameFP(const int32 InOutputFrameIndex) const
	{
		checkf(InOutputFrameIndex >= 0, TEXT("Frame index mapping function is only value for outputs frames greater than or equal to 0"));

		int64 InputFrameIndex = 0;
		if (NumFramesToInterpolate > 0)
		{
			if (InOutputFrameIndex < NumFramesToInterpolate)
			{
				// Frame ratio interpolation is still occurring at the output frame index.
				const int64 PreviousIndex = InOutputFrameIndex;
				const int64 AccumulationOfFrameDeltasFP = FrameRatioFrameDeltaFP * ((PreviousIndex * (PreviousIndex + 1)) / 2);
				InputFrameIndex = PreviousIndex * CurrentFrameRatioFP + AccumulationOfFrameDeltasFP;
			}
			else
			{
				// Frame ratio interpolation occurred, but has reached the target frame ratio by the output frame index.
				const int64 AccumulationOfFrameDeltasFP = FrameRatioFrameDeltaFP * (int64(NumFramesToInterpolate) * (NumFramesToInterpolate + 1) / 2);
				InputFrameIndex = int64(NumFramesToInterpolate) * CurrentFrameRatioFP + AccumulationOfFrameDeltasFP + int64(InOutputFrameIndex - NumFramesToInterpolate) * TargetFrameRatioFP;
			}
		}
		else
		{
			// No interpolation is happening. The math is quite a bit simpler. 
			InputFrameIndex = int64(TargetFrameRatioFP) * InOutputFrameIndex;
		}

		// Apply current internal offset. 
		InputFrameIndex += CurrentInputFrameIndexFP;
		return InputFrameIndex;
	}

	int32 FRuntimeResampler::GetNumInputFramesNeededToProduceOutputFrames(const int32 InNumOutputFrames) const
	{
		check(InNumOutputFrames >= 0);
		if (InNumOutputFrames > 0)
		{
			const int64 InputFrameFP = MapOutputFrameToInputFrameFP(InNumOutputFrames - 1);

			// We need the frame after InputFrameFP, so we take the floor and add 1. Because we're returning
			//  the number of frames needed rather than the index of the last frame, we add another 1. We add these
			//  frames before dividing to make sure the numerator is positive and the division gives us the floor.
			return int32((InputFrameFP + 2 * FPScale) / FPScale);
		}
		return 0;
	}

	int32 FRuntimeResampler::GetNumOutputFramesProducedByInputFrames(const int32 InNumInputFrames) const
	{
		check(InNumInputFrames >= 0);
		if (InNumInputFrames <= 0)
		{
			return 0;
		}

		const int64 InputFrameIndexFP = int64(InNumInputFrames - 1) * FPScale;
		const int64 RelativeInputFrameIndexFP = InputFrameIndexFP - CurrentInputFrameIndexFP;
		int32 OutputFrameIndex;
		if (RelativeInputFrameIndexFP <= 0)
		{
			// We don't have enough input to produce output frame 0.
			OutputFrameIndex = -1;
		}
		else if (NumFramesToInterpolate > 0)
		{
			const int64 InputSampleAtBreakpointFP = MapOutputFrameToInputFrameFP(NumFramesToInterpolate);
			if (InputFrameIndexFP > InputSampleAtBreakpointFP)
			{
				OutputFrameIndex = NumFramesToInterpolate + (InputFrameIndexFP - InputSampleAtBreakpointFP - 1) / TargetFrameRatioFP;
			}
			else
			{
				// Do a binary search for the previous output frame
				int32 LowerFrameIndex = 0;
				int32 UpperFrameIndex = NumFramesToInterpolate;
				while (UpperFrameIndex > LowerFrameIndex + 1)
				{
					const int32 MidFrameIndex = (LowerFrameIndex + UpperFrameIndex) / 2;
					const int64 InputAtMidOutputFP = MapOutputFrameToInputFrameFP(MidFrameIndex);
					if (InputAtMidOutputFP < InputFrameIndexFP)
					{
						LowerFrameIndex = MidFrameIndex;
					}
					else
					{
						UpperFrameIndex = MidFrameIndex;
					}
				}

				OutputFrameIndex = LowerFrameIndex;
			}
		}
		else
		{
			OutputFrameIndex = int32((RelativeInputFrameIndexFP - 1) / TargetFrameRatioFP);
		}

		return OutputFrameIndex + 1;
	}

	template<typename OutputBufferType, typename OutputChannelType>
	int32 FRuntimeResampler::ProcessCircularBufferGeneric(FMultichannelCircularBuffer& InAudio, OutputBufferType& OutAudio)
	{
		const int32 NumChannels = PreviousFrame.Num();
		check(InAudio.Num() == NumChannels && OutAudio.Num() == NumChannels);

		ResamplingParameters Parameters;
		Parameters.InputAudio = TempInputPointers;
		Parameters.InputStride = 1;
		Parameters.OutputAudio = TempOutputPointers;
		Parameters.OutputStride = 1;

		int32 NumFramesConsumed = 0;
		int32 NumFramesProduced = 0;

		// The input can be in two disjoint segments, so we run the resampling core up to two times.
		for (int32 SegmentIdx = 0; SegmentIdx < 2; ++SegmentIdx)
		{
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
			{
				// Set up input pointers
				DisjointedArrayView<const float> InputView = InAudio[ChannelIdx].PeekInPlace(InAudio[ChannelIdx].Num());
				if (ChannelIdx == 0)
				{
					Parameters.NumInputFrames = InputView.FirstBuffer.Num();
				}
				else
				{
					// We require that all of the circular buffers have the same layout
					check(Parameters.NumInputFrames == InputView.FirstBuffer.Num());
				}
				TempInputPointers[ChannelIdx] = InputView.FirstBuffer.GetData();

				// Set up the output pointers
				OutputChannelType& OutputChannel = OutAudio[ChannelIdx];
				TempOutputPointers[ChannelIdx] = OutputChannel.GetData() + NumFramesProduced;
				if (ChannelIdx == 0)
				{
					Parameters.NumOutputFrames = OutputChannel.Num() - NumFramesProduced;
				}
				else
				{
					check(Parameters.NumOutputFrames == OutputChannel.Num() - NumFramesProduced); // Every output channel must have the same size
				}
			}

			// Run the resampler, pop data from the input
			ProcessAudioInternal(Parameters);
			NumFramesConsumed += Parameters.OutNumInputFramesConsumed;
			NumFramesProduced += Parameters.OutNumOutputFramesProduced;
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
			{
				InAudio[ChannelIdx].Pop(Parameters.OutNumInputFramesConsumed);
			}

			// Skip the second pass if there's nothing left to do
			if (NumFramesProduced == OutAudio[0].Num() || InAudio[0].Num() == 0)
			{
				break;
			}
		}

		return NumFramesProduced;
	}

	int32 FRuntimeResampler::ProcessCircularBuffer(FMultichannelCircularBuffer& InAudio, FMultichannelBuffer& OutAudio)
	{
		return ProcessCircularBufferGeneric<FMultichannelBuffer, FAlignedFloatBuffer>(InAudio, OutAudio);
	}

	int32 FRuntimeResampler::ProcessCircularBuffer(FMultichannelCircularBuffer& InAudio, FMultichannelBufferView& OutAudio)
	{
		return ProcessCircularBufferGeneric<FMultichannelBufferView, TArrayView<float>>(InAudio, OutAudio);
	}

	void FRuntimeResampler::ProcessInterleaved(const TArrayView<const float> Input, const TArrayView<float> Output, int32& OutNumInputFramesConsumed, int32& OutNumOutputFramesProduced)
	{
		const int32 NumChannels = PreviousFrame.Num();
		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			TempInputPointers[Channel] = Input.GetData() + Channel;
			TempOutputPointers[Channel] = Output.GetData() + Channel;
		}

		ResamplingParameters Params{};
		Params.InputAudio = TempInputPointers;
		Params.NumInputFrames = Input.Num() / NumChannels;
		Params.InputStride = NumChannels;
		Params.OutputAudio = TempOutputPointers;
		Params.NumOutputFrames = Output.Num() / NumChannels;
		Params.OutputStride = NumChannels;

		ProcessAudioInternal(Params);

		OutNumInputFramesConsumed = Params.OutNumInputFramesConsumed;
		OutNumOutputFramesProduced = Params.OutNumOutputFramesProduced;
	}

	void FRuntimeResampler::ProcessAudioInternal(ResamplingParameters& Parameters)
	{
		check(Parameters.InputStride > 0 && Parameters.NumInputFrames >= 0 &&
			Parameters.OutputStride > 0 && Parameters.NumOutputFrames >= 0);

		const int32 NumChannels = PreviousFrame.Num();
		check(NumChannels > 0 && NumChannels == Parameters.InputAudio.Num() && NumChannels == Parameters.OutputAudio.Num());

		// Prevent overflowing an int32 in our fixed-point input index by dividing into batches
		const int32 MaxSamplesPerRun = 30000;
		if (Parameters.NumInputFrames > MaxSamplesPerRun)
		{
			ProcessAudioInBatches(Parameters, MaxSamplesPerRun);
			return;
		}

		// This will be reduced later if we don't have enough input.
		int32 NumOutputFrames = Parameters.NumOutputFrames;

		if (NumOutputFrames < 1 || Parameters.NumInputFrames < 1)
		{
			Parameters.OutNumInputFramesConsumed = 0;
			Parameters.OutNumOutputFramesProduced = 0;
			return;
		}

		float const* const* const InputPointers = Parameters.InputAudio.GetData();
		float* const* const OutputPointers = Parameters.OutputAudio.GetData();

		uint32 InputFrameRatioFP = CurrentFrameRatioFP;
		int32 InputFrameIndexFP = CurrentInputFrameIndexFP;
		const int32 OutputStride = Parameters.OutputStride;
		const int32 InputStride = Parameters.InputStride;

		// See if we can quickly do Memcpy's instead of resampling
		if (CurrentInputFrameIndexFP == 0 && TargetFrameRatioFP == FPScale && NumFramesToInterpolate == 0)
		{
			if (DoDirectCopy(Parameters))
			{
				return;
			}
		}

		// Check that InputFrameIndexFP is >= -1
		checkf(InputFrameIndexFP >= -FPScale, TEXT("Input frame index references discarded data"));

		// Figure out how many samples we can safely generate in the core loop
		const int32 NumFramesNeeded = GetNumInputFramesNeededToProduceOutputFrames(NumOutputFrames);
		if (NumFramesNeeded > Parameters.NumInputFrames)
		{
			NumOutputFrames = GetNumOutputFramesProducedByInputFrames(Parameters.NumInputFrames);
		}

		const int32 FinalNumFramesToInterpolate = FMath::Max(NumFramesToInterpolate - NumOutputFrames, 0);
		const int32 OutputEndIndex = NumOutputFrames * Parameters.OutputStride;
		int32 OutputSampleIndex = 0;

		// Output any samples that involve PreviousFrame
		while (InputFrameIndexFP < 0 && OutputSampleIndex < OutputEndIndex)
		{
			const float Alpha = (InputFrameIndexFP + FPScale) / FPScaleFloat;
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
			{
				OutputPointers[ChannelIdx][OutputSampleIndex] = FMath::Lerp(PreviousFrame[ChannelIdx], InputPointers[ChannelIdx][0], Alpha);
			}

			OutputSampleIndex += OutputStride;

			if (NumFramesToInterpolate > 0)
			{
				if (--NumFramesToInterpolate == 0)
				{
					InputFrameRatioFP = TargetFrameRatioFP;
					FrameRatioFrameDeltaFP = 0;
				}
				else
				{
					InputFrameRatioFP += FrameRatioFrameDeltaFP;
				}
			}

			InputFrameIndexFP += InputFrameRatioFP;
		}

		// Do the bulk of the processing in an optimized core
		if (NumChannels == 1 && InputStride == 1 && OutputStride == 1)
		{
			MonoResamplingCore(InputPointers[0], OutputPointers[0], InputFrameIndexFP, InputFrameRatioFP, OutputSampleIndex, OutputEndIndex);
		}
		else if (NumChannels == 2 && InputStride == 2 && OutputStride == 2)
		{
			StereoInterleavedResamplingCore(InputPointers[0], OutputPointers[0], InputFrameIndexFP, InputFrameRatioFP, OutputSampleIndex, OutputEndIndex);
		}
		else if (NumChannels == 2 && InputStride == 1 && OutputStride == 1)
		{
			StereoDeinterleavedResamplingCore(InputPointers[0], InputPointers[1], OutputPointers[0], OutputPointers[1], InputFrameIndexFP, InputFrameRatioFP, OutputSampleIndex, OutputEndIndex);
		}
		else
		{
			GenericResamplingCore(Parameters, InputFrameIndexFP, InputFrameRatioFP, OutputSampleIndex, OutputEndIndex);
		}

		NumFramesToInterpolate = FinalNumFramesToInterpolate;

		// Set output values
		const uint32 NumInputFramesConsumed = InputFrameIndexFP < 0 ? 0 : FMath::Min(InputFrameIndexFP / FPScale + 1, Parameters.NumInputFrames);
		Parameters.OutNumInputFramesConsumed = NumInputFramesConsumed;
		Parameters.OutNumOutputFramesProduced = NumOutputFrames;

		// Make sure we don't fail to make any progress 
		check(NumOutputFrames > 0 || NumInputFramesConsumed > 0);

		// Update saved state for next run
		if (NumInputFramesConsumed > 0)
		{
			InputFrameIndexFP -= NumInputFramesConsumed * FPScale;

			// Save final consumed input frame if we'll need it next run
			if (InputFrameIndexFP < 0)
			{
				const int32 SampleIndexToSave = (NumInputFramesConsumed - 1) * InputStride;
				for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
				{
					PreviousFrame[ChannelIdx] = InputPointers[ChannelIdx][SampleIndexToSave];
				}
			}
		}

		if (NumFramesToInterpolate <= 0)
		{
			check(NumFramesToInterpolate == 0);
			CurrentFrameRatioFP = TargetFrameRatioFP;
			FrameRatioFrameDeltaFP = 0;
		}
		else
		{
			CurrentFrameRatioFP = InputFrameRatioFP;
		}

		CurrentInputFrameIndexFP = InputFrameIndexFP;
	}

	void FRuntimeResampler::ProcessAudioInBatches(ResamplingParameters& Parameters, const int32 MaxSamplesPerBatch)
	{
		const int32 NumChannels = PreviousFrame.Num();

		int32 InputFramesRemaining = Parameters.NumInputFrames;
		Parameters.OutNumInputFramesConsumed = 0;
		Parameters.OutNumOutputFramesProduced = 0;

		// Create temporary copies of the pointer arrrays and resampling parameters
		ResamplingParameters BatchParameters(Parameters);
		TArray<const float*, TInlineAllocator<8>> BatchInputAudio(Parameters.InputAudio);
		TArray<float*, TInlineAllocator<8>> BatchOutputAudio(Parameters.OutputAudio);
		BatchParameters.InputAudio = BatchInputAudio;
		BatchParameters.OutputAudio = BatchOutputAudio;

		while (InputFramesRemaining > 0 && BatchParameters.NumOutputFrames > 0)
		{
			// Process a batch
			BatchParameters.NumInputFrames = FMath::Min(MaxSamplesPerBatch, InputFramesRemaining);
			ProcessAudioInternal(BatchParameters);

			// Reduce frame counts and advance pointers for the next batch
			Parameters.OutNumInputFramesConsumed += BatchParameters.OutNumInputFramesConsumed;
			Parameters.OutNumOutputFramesProduced += BatchParameters.OutNumOutputFramesProduced;
			InputFramesRemaining -= BatchParameters.OutNumInputFramesConsumed;
			BatchParameters.NumOutputFrames -= BatchParameters.OutNumOutputFramesProduced;
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
			{
				BatchInputAudio[ChannelIdx] += BatchParameters.OutNumInputFramesConsumed * Parameters.InputStride;
				BatchOutputAudio[ChannelIdx] += BatchParameters.OutNumOutputFramesProduced * Parameters.OutputStride;
			}
		}
	}

	bool FRuntimeResampler::DoDirectCopy(ResamplingParameters& Parameters)
	{
		const int32 NumChannels = PreviousFrame.Num();
		float const* const* const InputPointers = Parameters.InputAudio.GetData();
		float* const* const OutputPointers = Parameters.OutputAudio.GetData();

		if (Parameters.InputStride == 1 && Parameters.OutputStride == 1)
		{
			// Copy multi-mono buffers
			const int32 CopyFrames = FMath::Min(Parameters.NumInputFrames, Parameters.NumOutputFrames);
			const int32 CopyBytes = CopyFrames * sizeof(float);
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
			{
				FMemory::Memcpy(OutputPointers[ChannelIdx], InputPointers[ChannelIdx], CopyBytes);
			}
			Parameters.OutNumInputFramesConsumed = CopyFrames;
			Parameters.OutNumOutputFramesProduced = CopyFrames;
			return true;
		}
		else if (Parameters.InputStride == NumChannels && Parameters.OutputStride == NumChannels)
		{
			// Check if this is a complete interleaved buffer
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
			{
				if (InputPointers[ChannelIdx] != InputPointers[0] + ChannelIdx ||
					OutputPointers[ChannelIdx] != OutputPointers[0] + ChannelIdx)
				{
					return false; // Not appropriately interleaved.
				}
			}

			// Do one big copy
			const int32 CopyFrames = FMath::Min(Parameters.NumInputFrames, Parameters.NumOutputFrames);
			const int32 CopyBytes = CopyFrames * NumChannels * sizeof(float);
			FMemory::Memcpy(OutputPointers[0], InputPointers[0], CopyBytes);
			Parameters.OutNumInputFramesConsumed = CopyFrames;
			Parameters.OutNumOutputFramesProduced = CopyFrames;
			return true;
		}

		return false;
	}

	void FRuntimeResampler::GenericResamplingCore(const ResamplingParameters& Parameters, int32& OutInputFrameIndexFP, uint32& OutInputFrameRatioFP, int32 OutputSampleIndex, const int32 OutputEndIndex)
	{
		const int32 NumChannels = PreviousFrame.Num();
		const int32 OutputStride = Parameters.OutputStride;
		const int32 InputStride = Parameters.InputStride;
		float const* const* const InputPointers = Parameters.InputAudio.GetData();
		float* const* const OutputPointers = Parameters.OutputAudio.GetData();
		int32 InputFrameIndexFP = OutInputFrameIndexFP;
		int32 InputFrameRatioFP = OutInputFrameRatioFP;

		// Handle frames where samples are interpolated. 
		const int32 BatchEndIndex = FMath::Min(OutputEndIndex, OutputSampleIndex + NumFramesToInterpolate * OutputStride);
		for (; OutputSampleIndex < BatchEndIndex; OutputSampleIndex += OutputStride)
		{
			const int32 LowerSampleIndex = (InputFrameIndexFP / FPScale) * InputStride;
			const int32 UpperSampleIndex = LowerSampleIndex + InputStride;
			const float Alpha = (InputFrameIndexFP & (FPScale - 1)) / FPScaleFloat;
			checkSlow(UpperSampleIndex < Parameters.NumInputFrames * InputStride);
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
			{
				OutputPointers[ChannelIdx][OutputSampleIndex] = FMath::Lerp(InputPointers[ChannelIdx][LowerSampleIndex], InputPointers[ChannelIdx][UpperSampleIndex], Alpha);
			}

			InputFrameRatioFP += FrameRatioFrameDeltaFP;
			InputFrameIndexFP += InputFrameRatioFP;
		}

		// Handle frames where sample rate is constant.
		for (; OutputSampleIndex < OutputEndIndex; OutputSampleIndex += OutputStride)
		{
			const int32 LowerSampleIndex = (InputFrameIndexFP / FPScale) * InputStride;
			const int32 UpperSampleIndex = LowerSampleIndex + InputStride;
			const float Alpha = (InputFrameIndexFP & (FPScale - 1)) / FPScaleFloat;
			checkSlow(UpperSampleIndex < Parameters.NumInputFrames * InputStride);
			for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
			{
				OutputPointers[ChannelIdx][OutputSampleIndex] = FMath::Lerp(InputPointers[ChannelIdx][LowerSampleIndex], InputPointers[ChannelIdx][UpperSampleIndex], Alpha);
			}

			InputFrameIndexFP += TargetFrameRatioFP;
		}

		OutInputFrameIndexFP = InputFrameIndexFP;
		OutInputFrameRatioFP = InputFrameRatioFP;
	}

	void FRuntimeResampler::MonoResamplingCore(const float* Input, float* Output, int32& OutInputFrameIndexFP, uint32& OutInputFrameRatioFP, int32 OutputSampleIndex, const int32 OutputEndIndex)
	{
		int32 InputFrameIndexFP = OutInputFrameIndexFP;
		uint32 InputFrameRatioFP = OutInputFrameRatioFP;

		// Handle frames where samples are interpolated. 
		const int32 BatchEndIndex = FMath::Min(OutputEndIndex, OutputSampleIndex + NumFramesToInterpolate);
		for (; OutputSampleIndex < BatchEndIndex; OutputSampleIndex++)
		{
			const int32 LowerSampleIndex = InputFrameIndexFP / FPScale;
			const int32 UpperSampleIndex = LowerSampleIndex + 1;
			const float Alpha = (InputFrameIndexFP & (FPScale - 1)) / FPScaleFloat;
			Output[OutputSampleIndex] = FMath::Lerp(Input[LowerSampleIndex], Input[UpperSampleIndex], Alpha);

			InputFrameRatioFP += FrameRatioFrameDeltaFP;
			InputFrameIndexFP += InputFrameRatioFP;
		}

		// Handle frames where sample rate is constant.
		for (; OutputSampleIndex < OutputEndIndex; OutputSampleIndex++)
		{
			const int32 LowerSampleIndex = InputFrameIndexFP / FPScale;
			const int32 UpperSampleIndex = LowerSampleIndex + 1;
			const float Alpha = (InputFrameIndexFP & (FPScale - 1)) / FPScaleFloat;
			Output[OutputSampleIndex] = FMath::Lerp(Input[LowerSampleIndex], Input[UpperSampleIndex], Alpha);

			InputFrameIndexFP += TargetFrameRatioFP;
		}

		OutInputFrameIndexFP = InputFrameIndexFP;
		OutInputFrameRatioFP = InputFrameRatioFP;
	}

	void FRuntimeResampler::StereoInterleavedResamplingCore(const float* Input, float* Output, int32& OutInputFrameIndexFP, uint32& OutInputFrameRatioFP, int32 OutputSampleIndex, const int32 OutputEndIndex)
	{
		int32 InputFrameIndexFP = OutInputFrameIndexFP;
		int32 InputFrameRatioFP = OutInputFrameRatioFP;

		// Handle frames where samples are interpolated. 
		const int32 BatchEndIndex = FMath::Min(OutputEndIndex, OutputSampleIndex + NumFramesToInterpolate * 2);
		for (; OutputSampleIndex < BatchEndIndex; OutputSampleIndex += 2)
		{
			const int32 LowerSampleIndex = (InputFrameIndexFP / FPScale) * 2;
			const float Alpha = (InputFrameIndexFP & (FPScale - 1)) / FPScaleFloat;
			Output[OutputSampleIndex] = FMath::Lerp(Input[LowerSampleIndex], Input[LowerSampleIndex + 2], Alpha);
			Output[OutputSampleIndex + 1] = FMath::Lerp(Input[LowerSampleIndex + 1], Input[LowerSampleIndex + 3], Alpha);

			InputFrameRatioFP += FrameRatioFrameDeltaFP;
			InputFrameIndexFP += InputFrameRatioFP;
		}

		// Handle frames where sample rate is constant.
		for (; OutputSampleIndex < OutputEndIndex; OutputSampleIndex += 2)
		{
			const int32 LowerSampleIndex = (InputFrameIndexFP / FPScale) * 2;
			const float Alpha = (InputFrameIndexFP & (FPScale - 1)) / FPScaleFloat;
			Output[OutputSampleIndex] = FMath::Lerp(Input[LowerSampleIndex], Input[LowerSampleIndex + 2], Alpha);
			Output[OutputSampleIndex + 1] = FMath::Lerp(Input[LowerSampleIndex + 1], Input[LowerSampleIndex + 3], Alpha);

			InputFrameIndexFP += TargetFrameRatioFP;
		}

		OutInputFrameIndexFP = InputFrameIndexFP;
		OutInputFrameRatioFP = InputFrameRatioFP;
	}

	void FRuntimeResampler::StereoDeinterleavedResamplingCore(const float* InputLeft, const float* InputRight, float* OutputLeft, float* OutputRight, int32& OutInputFrameIndexFP, uint32& OutInputFrameRatioFP, int32 OutputSampleIndex, const int32 OutputEndIndex)
	{
		int32 InputFrameIndexFP = OutInputFrameIndexFP;
		int32 InputFrameRatioFP = OutInputFrameRatioFP;

		// Handle frames where samples are interpolated. 
		const int32 BatchEndIndex = FMath::Min(OutputEndIndex, OutputSampleIndex + NumFramesToInterpolate);
		for (; OutputSampleIndex < BatchEndIndex; OutputSampleIndex++)
		{
			const int32 LowerSampleIndex = InputFrameIndexFP / FPScale;
			const int32 UpperSampleIndex = LowerSampleIndex + 1;
			const float Alpha = (InputFrameIndexFP & (FPScale - 1)) / FPScaleFloat;
			OutputLeft[OutputSampleIndex] = FMath::Lerp(InputLeft[LowerSampleIndex], InputLeft[UpperSampleIndex], Alpha);
			OutputRight[OutputSampleIndex] = FMath::Lerp(InputRight[LowerSampleIndex], InputRight[UpperSampleIndex], Alpha);

			InputFrameRatioFP += FrameRatioFrameDeltaFP;
			InputFrameIndexFP += InputFrameRatioFP;
		}

		// Handle frames where sample rate is constant.
		for (; OutputSampleIndex < OutputEndIndex; OutputSampleIndex++)
		{
			const int32 LowerSampleIndex = InputFrameIndexFP / FPScale;
			const int32 UpperSampleIndex = LowerSampleIndex + 1;
			const float Alpha = (InputFrameIndexFP & (FPScale - 1)) / FPScaleFloat;
			OutputLeft[OutputSampleIndex] = FMath::Lerp(InputLeft[LowerSampleIndex], InputLeft[UpperSampleIndex], Alpha);
			OutputRight[OutputSampleIndex] = FMath::Lerp(InputRight[LowerSampleIndex], InputRight[UpperSampleIndex], Alpha);

			InputFrameIndexFP += TargetFrameRatioFP;
		}

		OutInputFrameIndexFP = InputFrameIndexFP;
		OutInputFrameRatioFP = InputFrameRatioFP;
	}
}

