// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/SampleBufferReader.h"
#include "DSP/SinOsc.h"

namespace Audio
{
	FSampleBufferReader::FSampleBufferReader()
		: BufferPtr(nullptr)
		, BufferNumSamples(0)
		, BufferNumFrames(0)
		, BufferSampleRate(0)
		, BufferNumChannels(0)
		, FadeFrames(512)
		, FadeValue(0.0f)
		, FadeIncrement(1.0f / (float)FadeFrames)
		, DeviceSampleRate(0.0f)
		, BasePitch(1.0f)
		, PitchScale(1.0f)
		, CurrentFrameIndex(0)
		, NextFrameIndex(0)
		, AlphaLerp(0.0f)
		, CurrentBufferFrameIndexInterpolated(0.0)
		, PlaybackProgress(0.0f)
		, ScrubAnchorFrame(0.0)
		, ScrubMinFrame(0.0)
		, ScrubMaxFrame(0.0)
		, ScrubWidthFrames(0.0)
		, CurrentSeekTime(0.0f)
		, CurrentScrubWidthSec(0.0f)
		, CurrentSeekType(ESeekType::FromBeginning)
		, bWrap(false)
		, bIsScrubMode(false)
		, bIsFinished(false)
	{
	}

	FSampleBufferReader::~FSampleBufferReader()
	{
	}

	void FSampleBufferReader::Init(const int32 InSampleRate)
	{
		DeviceSampleRate = InSampleRate;

		BufferPtr = nullptr;
		BufferNumSamples = 0;
		BufferNumFrames = 0;
		BufferSampleRate = 0;
		BufferNumChannels = 0;

		CurrentFrameIndex = 0;
		NextFrameIndex = 0;
		AlphaLerp = 0.0f;

		Pitch.Init(DeviceSampleRate);
		Pitch.SetValue(1.0, 0.0f);

		BasePitch = 1.0f;

		bIsFinished = false;
		CurrentBufferFrameIndexInterpolated = 0.0;
		ScrubAnchorFrame = 0.0;
		ScrubMinFrame = 0.0;
		ScrubMaxFrame = 0.0;

		// Default the scrub width to 0.1 seconds
		bIsScrubMode = false;
		ScrubWidthFrames = 0.1 * DeviceSampleRate;
		PlaybackProgress = 0.0f;
	}

	void FSampleBufferReader::SetBuffer(const int16* InBufferPtr, const int32 InNumBufferSamples, const int32 InNumChannels, const int32 InBufferSampleRate)
	{
		// Re-init on setting a new buffer
		Init(InBufferSampleRate);

		BufferPtr = InBufferPtr;
		BufferNumSamples = InNumBufferSamples;
		BufferNumChannels = InNumChannels;
		BufferSampleRate = InBufferSampleRate;
		BufferNumFrames = BufferNumSamples / BufferNumChannels;

		// This is the base pitch to use play at the "correct" sample rate for the buffer to sound correct on the output device sample rate
		BasePitch = BufferSampleRate / DeviceSampleRate;

		// Set the pitch to the previous pitch scale.
		Pitch.SetValueInterrupt(PitchScale * BasePitch);

		bIsFinished = false;

		UpdateScrubMinAndMax();
	}

	void FSampleBufferReader::ClearBuffer()
	{
		BufferPtr = nullptr;
		BufferNumSamples = 0;
		BufferNumChannels = 0;
		BufferSampleRate = 0;
		BufferNumFrames = 0;
	}

	void FSampleBufferReader::UpdateSeekFrame()
	{
		if (BufferPtr)
		{
			check(BufferNumChannels > 0);
			const float CurrentSeekFrame = ((float)BufferSampleRate * CurrentSeekTime);

			if (CurrentSeekType == ESeekType::FromBeginning)
			{
				CurrentBufferFrameIndexInterpolated = (double)CurrentSeekFrame;
			}
			else if (CurrentSeekType == ESeekType::FromEnd)
			{
				CurrentBufferFrameIndexInterpolated = (double)(BufferNumFrames - CurrentSeekFrame - 1);
			}
			else
			{
				CurrentBufferFrameIndexInterpolated += (double)CurrentSeekFrame;
			}

			if (bWrap)
			{
				while (CurrentBufferFrameIndexInterpolated >= (double)BufferNumFrames)
				{
					CurrentBufferFrameIndexInterpolated -= (double)BufferNumFrames;
				}

				while (CurrentBufferFrameIndexInterpolated < 0.0)
				{
					CurrentBufferFrameIndexInterpolated += (double)BufferNumFrames;
				}

				check(CurrentBufferFrameIndexInterpolated >= 0.0 && CurrentBufferFrameIndexInterpolated < (double)BufferNumFrames);
			}
			else
			{
				CurrentBufferFrameIndexInterpolated = FMath::Clamp(CurrentBufferFrameIndexInterpolated, 0.0, (double)BufferNumFrames);
			}
		}

		ScrubAnchorFrame = CurrentBufferFrameIndexInterpolated;
	}

	void FSampleBufferReader::SeekTime(const float InTimeSec, const ESeekType::Type InSeekType, const bool bInWrap)
	{
		CurrentSeekTime = InTimeSec;
		CurrentSeekType = InSeekType;
		bWrap = bInWrap;

		if (bIsScrubMode)
		{
			UpdateSeekFrame();
			UpdateScrubMinAndMax();
		}
		else
		{
			UpdateSeekFrame();
		}
	}

	void FSampleBufferReader::SetScrubTimeWidth(const float InScrubTimeWidthSec)
	{
		CurrentScrubWidthSec = InScrubTimeWidthSec;

		UpdateScrubMinAndMax();
	}

	void FSampleBufferReader::SetPitch(const float InPitch, const float InterpolationTimeSec)
	{
		PitchScale = InPitch;
		Pitch.SetValue(PitchScale * BasePitch, InterpolationTimeSec);
	}

	void FSampleBufferReader::SetScrubMode(const bool bInIsScrubMode)
	{
		bIsScrubMode = bInIsScrubMode;
		
		// Anchor the current frame index as the scrub anchor
		ScrubAnchorFrame = CurrentBufferFrameIndexInterpolated;
		UpdateSeekFrame();
		UpdateScrubMinAndMax();
	}

	void FSampleBufferReader::UpdateScrubMinAndMax()
	{
		UpdateSeekFrame();

		if (BufferNumFrames > 0)
		{
			ScrubWidthFrames = (double)(DeviceSampleRate * FMath::Max(CurrentScrubWidthSec, 0.001f));
			ScrubWidthFrames = FMath::Min((double)(BufferNumFrames - 1), ScrubWidthFrames);

			// Don't allow the scrub width to be less than 2 times the scrubwidth frames
			ScrubWidthFrames = FMath::Max(ScrubWidthFrames, (double)2*FadeFrames);

			ScrubMinFrame = ScrubAnchorFrame - 0.5 * ScrubWidthFrames;
			ScrubMaxFrame = ScrubAnchorFrame + 0.5 * ScrubWidthFrames;
		}
	}

	float FSampleBufferReader::GetSampleValue(const int16* InBuffer, const int32 SampleIndex)
	{
		int16 PCMSampleValue = InBuffer[SampleIndex];
		return (float)PCMSampleValue / 32767.0f;
	}

	bool FSampleBufferReader::Generate(float* OutAudioBuffer, const int32 NumFrames, const int32 OutChannels, const bool bInWrap)
	{
		// Don't have a buffer yet, so fill in zeros, say we're not done
		const int32 NumSamples = NumFrames * OutChannels;
		if (!HasBuffer() || bIsFinished)
		{
			FMemory::Memzero(OutAudioBuffer, NumSamples * sizeof(float));
			return false;
		}	

#if 0
		static FSineOsc SineOsc(48000.0f, 440.0f, 0.2f);

		int32 SampleIndex = 0;
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			const float Value = 0.1f * SineOsc.ProcessAudio();
			for (int32 Channel = 0; Channel < OutChannels; ++Channel)
			{
				OutAudioBuffer[SampleIndex++] = Value;
			}
		}
#else

		// We always want to wrap if we're in scrub mode
		const bool bDoWrap = bInWrap || bIsScrubMode;

		int32 OutSampleIndex = 0;
		for (int32 i = 0; i < NumFrames && !bIsFinished; ++i)
		{
			float CurrentPitch = Pitch.GetNextValue();

			// Don't let the pitch go to 0.
			if (FMath::IsNearlyZero(CurrentPitch))
			{
				CurrentPitch = SMALL_NUMBER;
			}

			// We're going forward in the buffer
			if (CurrentPitch > 0.0f)
			{
				CurrentFrameIndex = FMath::FloorToInt(CurrentBufferFrameIndexInterpolated);
				NextFrameIndex = CurrentFrameIndex + 1;
				AlphaLerp = CurrentBufferFrameIndexInterpolated - (double)CurrentFrameIndex;
				if (!bIsScrubMode && !bWrap)
				{
					if (NextFrameIndex >= BufferNumFrames)
					{
						bIsFinished = true;
					}
				}
			}
			else
			{
				CurrentFrameIndex = FMath::CeilToInt(CurrentBufferFrameIndexInterpolated);
				NextFrameIndex = CurrentFrameIndex - 1;
				AlphaLerp = (double)CurrentFrameIndex - CurrentBufferFrameIndexInterpolated;
				if (!bIsScrubMode && !bWrap)
				{
					if (NextFrameIndex < 0)
					{
						bIsFinished = true;
					}
				}
			}

			if (!bIsFinished)
			{
				// Check for scrub boundaries and wrap. Note that we've already wrapped on the buffer boundary at this point.
				if (bWrap || bIsScrubMode)
				{
					int32 MinWrapFrame = 0;
					int32 MaxWrapFrame = BufferNumFrames;
					if (bIsScrubMode)
					{
						MinWrapFrame = ScrubMinFrame;
						MaxWrapFrame = ScrubMaxFrame;
					}

					if (CurrentPitch > 0.0f && NextFrameIndex >= MaxWrapFrame)
					{
						NextFrameIndex = MinWrapFrame;
						CurrentFrameIndex = (int32)(MaxWrapFrame - 1.0f);
						CurrentBufferFrameIndexInterpolated = FMath::Fmod(CurrentBufferFrameIndexInterpolated, 1.0) + (double)(NextFrameIndex);
					}
					else if (NextFrameIndex < MinWrapFrame)
					{
						NextFrameIndex = (int32)(MaxWrapFrame - 1);
						CurrentFrameIndex = (int32)MinWrapFrame;
						CurrentBufferFrameIndexInterpolated = FMath::Fmod(CurrentBufferFrameIndexInterpolated, 1.0) + (double)NextFrameIndex;
					}

					CurrentFrameIndex = FMath::CeilToInt(CurrentBufferFrameIndexInterpolated);
					NextFrameIndex = CurrentFrameIndex - 1;
					AlphaLerp = (double)CurrentFrameIndex - CurrentBufferFrameIndexInterpolated;

					FadeValue = 1.0f;
					int32 MaxFadeInFrame = MinWrapFrame + FadeFrames;
					if (CurrentFrameIndex >= MinWrapFrame && CurrentFrameIndex < MaxFadeInFrame)
					{
						FadeValue = (float)(CurrentFrameIndex - MinWrapFrame) / FadeFrames;
					}
					else if (CurrentFrameIndex >= MaxWrapFrame - FadeFrames && CurrentFrameIndex < MaxWrapFrame)
					{
						FadeValue = 1.0f - (float)(CurrentFrameIndex - (MaxWrapFrame - FadeFrames)) / FadeFrames;
					}
				}
				else
				{
					FadeValue = 1.0f;
				}

				if (OutChannels == BufferNumChannels)
				{
					for (int32 Channel = 0; Channel < BufferNumChannels; ++Channel)
					{
						OutAudioBuffer[OutSampleIndex++] = FadeValue * GetSampleValueForChannel(Channel);
					}
				}
				else if (OutChannels == 1 && BufferNumChannels == 2)
				{
					float LeftChannel = FadeValue * GetSampleValueForChannel(0);
					float RightChannel = FadeValue * GetSampleValueForChannel(1);
					OutAudioBuffer[OutSampleIndex++] = 0.5f * (LeftChannel + RightChannel);
				}
				else if (OutChannels == 2 && BufferNumChannels == 1)
				{
					float Sample = FadeValue * GetSampleValueForChannel(0);
					OutAudioBuffer[OutSampleIndex++] = 0.5f * Sample;
					OutAudioBuffer[OutSampleIndex++] = 0.5f * Sample;
				}

				CurrentBufferFrameIndexInterpolated += CurrentPitch;
			}
		}
#endif
		if (OutSampleIndex < NumSamples)
		{
			FMemory::Memzero(&OutAudioBuffer[OutSampleIndex], (NumSamples - OutSampleIndex) * sizeof(float));
		}

		return bIsFinished;
	}

	static int32 WrapIndex(int32 Value, int32 Max)
	{
		if (Value < 0)
		{
			Value += Max;
		}
		else if (Value >= Max)
		{
			Value -= Max;
		}
		return Value;
	}

	float FSampleBufferReader::GetSampleValueForChannel(const int32 Channel)
	{
		if (BufferPtr)
		{
			// Wrap the current frame index
			int32 WrappedCurrentFrameIndex = WrapIndex(CurrentFrameIndex, BufferNumFrames);
			int32 WrappedNextFrameIndex = WrapIndex(NextFrameIndex, BufferNumFrames);

			// Update the current playback time
			PlaybackProgress = ((float)WrappedCurrentFrameIndex) / BufferSampleRate;

			const int32 CurrentBufferSampleIndex = BufferNumChannels * WrappedCurrentFrameIndex + Channel;
			const int32 NextBufferSampleIndex = BufferNumChannels * WrappedNextFrameIndex + Channel;

			const float CurrentSampleValue = GetSampleValue(BufferPtr, CurrentBufferSampleIndex);
			const float NextSampleValue = GetSampleValue(BufferPtr, NextBufferSampleIndex);
			return FMath::Lerp(CurrentSampleValue, NextSampleValue, AlphaLerp);
		}
		return 0.0f;
	}

}


