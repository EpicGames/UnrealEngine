// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/SampleRateConverter.h"

namespace Audio
{
	static float GetFloatSampleValue(const float InSampleValue)
	{
		return InSampleValue;
	}

	static float GetFloatSampleValue(const int16 InSampleValue)
	{
		return (float)InSampleValue / 32767.0f;
	}

	class FSampleRateConverter : public ISampleRateConverter
	{
	public:
		FSampleRateConverter()
			: CurrentFrameIndex(0)
			, NextFrameIndex(1)
			, FrameAlpha(0.0f)
			, NumChannels(0)
			, SampleRateRatio(1.0f)
			, TargetSampleRateRatio(1.0f)
			, SampleRatioDelta(0.0f)
			, NumSampleRatioFrameTicks(0)
			, CurrentSampleRatioFrameTick(0)
			, bUsePreviousChunkFrame(false)
		{
		}

		virtual ~FSampleRateConverter() {}
		
		virtual void Init(const float InSampleRateRatio, const int32 InNumChannels) override
		{
			CurrentFrameIndex = 0;
			NextFrameIndex = 1;
			FrameAlpha = 0.0f;

			SetSampleRateRatio(InSampleRateRatio, 0);

			NumChannels = InNumChannels;

			bUsePreviousChunkFrame = false;
			PreviousChunkFrame.Reset();
			PreviousChunkFrame.AddZeroed(NumChannels);
		}

		virtual void SetSampleRateRatio(const float InSampleRateRatio, const int32 NumInterpolationFrames) override
		{
			NumSampleRatioFrameTicks = FMath::Max(NumInterpolationFrames, 0);
			CurrentSampleRatioFrameTick = 0;

			if (NumSampleRatioFrameTicks == 0)
			{
				SampleRateRatio = InSampleRateRatio;
				TargetSampleRateRatio = SampleRateRatio;
				SampleRatioDelta = 0.0f;
			}
			else
			{
				SampleRatioDelta = (InSampleRateRatio - SampleRateRatio) / NumSampleRatioFrameTicks;
				TargetSampleRateRatio = InSampleRateRatio;
			}
		}

		template<typename T>
		int32 ProcessChunkImpl(const T* InBufferChunk, const int32 InNumSamples, const int32 RequestedFrames, TArray<float>& OutBuffer)
		{
			// Reset the output buffer. Can reuse the buffer and avoid reallocs.
			OutBuffer.Reset();

			// We don't have any data (or less than one frame of data) in the input buffer chunk
			// Or we didn't initialize the sample rate converter
			if (NumChannels == 0 || InNumSamples < NumChannels)
			{
				// TODO: Error?
				return 0;
			}

			const int32 NumInputFrames = InNumSamples / NumChannels;

			// TODO: error if requested frames is greater than input frames?
			const int32 FramesToProcess = FMath::Min(NumInputFrames, RequestedFrames);

			int32 NumFramesGenerated = 0;

			// Process this buffer until we've processed all the frames we can
			while (NextFrameIndex < NumInputFrames)
			{
				// Perform the linear interpolation to the next frame
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					float CurrentSample = 0.0f;

					// If we're using the frame from a previous chunk for the "current" frame, then we
					// read from that chunk
					if (bUsePreviousChunkFrame)
					{
						CurrentSample = PreviousChunkFrame[Channel];
					}
					else
					{
						const int32 CurrentSampleIndex = NumChannels * CurrentFrameIndex + Channel;
						CurrentSample = GetFloatSampleValue(InBufferChunk[CurrentSampleIndex]);
					}

					const int32 NextSampleIndex = NumChannels * NextFrameIndex + Channel;
					float NextSample = GetFloatSampleValue(InBufferChunk[NextSampleIndex]);

					float OutSample = FMath::Lerp(CurrentSample, NextSample, FrameAlpha);
					OutBuffer.Add(OutSample);
				}

				++NumFramesGenerated;

				// Increment the frame alpha based on the current sample rate ratio
				FrameAlpha += SampleRateRatio;

				// Update the sample ratio to the new interpolated value
				if (CurrentSampleRatioFrameTick < NumSampleRatioFrameTicks)
				{
					SampleRateRatio += SampleRatioDelta;
					CurrentSampleRatioFrameTick++;
				}
				else
				{
					// Make sure we're exactly at the target sample rate ratio
					SampleRateRatio = TargetSampleRateRatio;
				}

				// Now increment the current and next frame based on the new alpha value.
				// If alpha is greater than 1.0 then we need to increment the frame counts based
				int32 FrameAlphaRounded = (int32)FrameAlpha;
				if (FrameAlphaRounded > 0)
				{
					// No longer using the previous chunk frame if this hits
					bUsePreviousChunkFrame = false;

					CurrentFrameIndex += FrameAlphaRounded;
					NextFrameIndex = CurrentFrameIndex + 1;

					// Subtract the frame alpha back to < 1.0 region
					FrameAlpha -= (float)FrameAlphaRounded;
					check(FrameAlpha < 1.0f && FrameAlpha >= 0.0f);
				}
			}

			// If the current frame index is within this buffer, but the next frame index is 
			// in the next buffer, then we need to store the value of the current frame index
			// so we can use it in the next chunk
			if (CurrentFrameIndex < NumInputFrames && NextFrameIndex >= NumInputFrames)
			{
				bUsePreviousChunkFrame = true;
				for (int32 Channel = 0; Channel < NumChannels; ++Channel)
				{
					const int32 CurrentSampleIndex = NumChannels * CurrentFrameIndex + Channel;
					PreviousChunkFrame[Channel] = GetFloatSampleValue(InBufferChunk[CurrentFrameIndex]);
				}
			}

			// Now wrap the current frame index if we've finished this buffer
			// The current and next frame indicies will now be relative to the next input buffer
			if (CurrentFrameIndex >= NumInputFrames)
			{
				CurrentFrameIndex -= NumInputFrames;
				NextFrameIndex = CurrentFrameIndex + 1;
			}

			return NumFramesGenerated;
		}

		template <typename T>
		int32 ProcessFullbufferImpl(const T* InBuffer, const int32 InNumSamples, TArray<float>& OutBuffer)
		{
			int32 NumFrames = InNumSamples / NumChannels;
			return ProcessChunkImpl(InBuffer, InNumSamples, NumFrames, OutBuffer);
		}

		virtual int32 ProcessChunk(const int16* BufferChunk, const int32 NumSamples, const int32 RequestedFrames, TArray<float>& OutBuffer) override
		{
			return ProcessChunkImpl(BufferChunk, NumSamples, RequestedFrames, OutBuffer);
		}

		virtual int32 ProcessChunk(const float* BufferChunk, const int32 NumSamples, const int32 RequestedFrames, TArray<float>& OutBuffer) override
		{
			return ProcessChunkImpl(BufferChunk, NumSamples, RequestedFrames, OutBuffer);
		}

		virtual int32 ProcessFullbuffer(const int16* InBuffer, const int32 InNumSamples, TArray<float>& OutBuffer) override
		{
			return ProcessFullbufferImpl(InBuffer, InNumSamples, OutBuffer);
		}

		virtual int32 ProcessFullbuffer(const float* InBuffer, const int32 InNumSamples, TArray<float>& OutBuffer) override
		{
			return ProcessFullbufferImpl(InBuffer, InNumSamples, OutBuffer);
		}

	protected:
		
		int CurrentFrameIndex;
		int NextFrameIndex;
		float FrameAlpha;

		int32 NumChannels;
		float SampleRateRatio;
		float TargetSampleRateRatio;
		float SampleRatioDelta;
		int32 NumSampleRatioFrameTicks;
		int32 CurrentSampleRatioFrameTick;

		bool bUsePreviousChunkFrame;
		TArray<float> PreviousChunkFrame;
	};


	ISampleRateConverter* ISampleRateConverter::CreateSampleRateConverter()
	{
		return new Audio::FSampleRateConverter();
	}
}


