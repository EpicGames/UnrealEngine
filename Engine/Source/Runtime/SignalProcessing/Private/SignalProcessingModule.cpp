// Copyright Epic Games, Inc. All Rights Reserved.

#include "SignalProcessingModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "DSP/FFTAlgorithm.h"
#include "DSP/ConvolutionAlgorithm.h"
#include "DSP/AudioFFT.h"
#include "VectorFFT.h"
#include "UniformPartitionConvolution.h"

namespace Audio
{
	class FSignalProcessingModule : public IModuleInterface
	{
		FVectorFFTFactory VectorFFTAlgorithmFactory;

		FUniformPartitionConvolutionFactory UniformPartitionConvolutionFactory;

	public:

		virtual void StartupModule() override
		{
			// FFT factories to register
			IModularFeatures::Get().RegisterModularFeature(IFFTAlgorithmFactory::GetModularFeatureName(), &VectorFFTAlgorithmFactory);

			// Convolution factories to register
			IModularFeatures::Get().RegisterModularFeature(IConvolutionAlgorithmFactory::GetModularFeatureName(), &UniformPartitionConvolutionFactory);
		}

		virtual void ShutdownModule() override
		{
			// FFT Factories to unregister
			IModularFeatures::Get().UnregisterModularFeature(IFFTAlgorithmFactory::GetModularFeatureName(), &VectorFFTAlgorithmFactory);

			// Convolution factories to unregister
			IModularFeatures::Get().UnregisterModularFeature(IConvolutionAlgorithmFactory::GetModularFeatureName(), &UniformPartitionConvolutionFactory);
		}
	};
}

DEFINE_LOG_CATEGORY(LogSignalProcessing);

IMPLEMENT_MODULE(Audio::FSignalProcessingModule, SignalProcessing);
