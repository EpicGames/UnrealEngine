// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/SinOsc.h"

namespace Audio
{
	FSineOsc::FSineOsc()
		: SampleRate(0)
		, FrequencyHz(440.0f)
		, B1(0.0f)
		, B2(0.0f)
		, Yn_1(0.0f)
		, Yn_2(0.0f)
		, Scale(1.0f)
		, Add(0.0f)
	{
	}

	FSineOsc::FSineOsc(const int32 InSampleRate, const float InFrequencyHz, const float InScale, const float InAdd)
		: SampleRate(0)
		, FrequencyHz(440.0f)
		, B1(0.0f)
		, B2(0.0f)
		, Yn_1(0.0f)
		, Yn_2(0.0f)
		, Scale(InScale)
		, Add(InAdd)
	{
		Init(InSampleRate, InFrequencyHz);
	}

	FSineOsc::~FSineOsc()
	{
	}

	void FSineOsc::Init(const int32 InSampleRate, const float InFrequencyHz, const float InScale, const float InAdd)
	{
		check(InSampleRate > 0);
		check(InFrequencyHz > 0.0f);

		Scale = InScale;
		Add = InAdd;

		SampleRate = InSampleRate;

		FrequencyHz = -1.0f;
		SetFrequency(InFrequencyHz);
	}

	void FSineOsc::SetScale(const float InScale)
	{
		Scale = InScale;
	}

	void FSineOsc::SetAdd(const float InAdd)
	{
		Add = InAdd;
	}

	void FSineOsc::SetFrequency(const float InFrequencyHz)
	{
		check(SampleRate > 0);

		if (FrequencyHz == InFrequencyHz)
		{
			return;
		}

		FrequencyHz = InFrequencyHz;

		// Find new wT value
		const float OmegaT = 2.0f * PI * FrequencyHz / SampleRate;

		// set the biquad feedback coefficients using filter design equations
		B1 = -2.0f * FMath::Cos(OmegaT);
		B2 = 1.0f;

		// Set up initial conditions based on current state of the oscillator to avoid pops when dynamically changing frequencies

		Yn_1 = FMath::Clamp(Yn_1, -1.0f, 1.0f);

		// Get previous outputs phase
		const float OmegaTPrev = FMath::Asin(Yn_1);

		// Get N by dividing prev phase over new current phase
		float N = OmegaTPrev / FMath::Max(OmegaT, SMALL_NUMBER);

		// If currently on rising edge (newer value is higher)
		if (Yn_1 > Yn_2)
		{
			// new y(n-2) value will be Sin((N - 1) * wT)
			N -= 1.0f;
		}
		// If on falling edge (new value is lower)
		else
		{
			// new y(n-2) value will be Sin((N + 1) * wT)
			N += 1.0f;
		}

		Yn_2 = FMath::Sin(N * OmegaT);
	}

	float FSineOsc::GetFrequency() const
	{ 
		return FrequencyHz; 
	}

	float FSineOsc::ProcessAudio()
	{
		// using direct-form difference equation
		// y(n) = -b1 * y(n - 1) - b2 * y(n - 2)
		const float Yn = -B1 * Yn_1 - B2 * Yn_2;

		// Move outputs down
		Yn_2 = Yn_1;
		Yn_1 = Yn;
		return Scale * Yn + Add;
	}
}
