// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/SineWaveTableOsc.h"

namespace Audio
{
	static const int32 SINE_WAVE_TABLE_SIZE = 4096;

	FSineWaveTableOsc::FSineWaveTableOsc()
	{
		SetFrequencyHz(FrequencyHz);
	}

	FSineWaveTableOsc::~FSineWaveTableOsc()
	{
	}

	void FSineWaveTableOsc::Init(const float InSampleRate, const float InFrequencyHz, const float InPhase)
	{
		SampleRate = FMath::Clamp(InSampleRate, 0.0f, InSampleRate);
		FrequencyHz = FMath::Clamp(InFrequencyHz, 0.0f, InFrequencyHz);
		InitialPhase = FMath::Clamp(InPhase, 0.0f, 1.0f);
		InstantaneousPhase = InitialPhase;

		Reset();
		UpdatePhaseIncrement();
	}

	void FSineWaveTableOsc::SetSampleRate(const float InSampleRate)
	{
		SampleRate = FMath::Clamp(InSampleRate, 0.0f, InSampleRate);;
		UpdatePhaseIncrement();
	}

	void FSineWaveTableOsc::Reset()
	{
		ReadIndex = InitialPhase * WaveTableBuffer.Num();
		while (ReadIndex >= WaveTableBuffer.Num())
		{
			ReadIndex -= WaveTableBuffer.Num();
		}
	}

	void FSineWaveTableOsc::SetFrequencyHz(const float InFrequencyHz)
	{
		FrequencyHz = FMath::Clamp(InFrequencyHz, 0.0f, InFrequencyHz);;
		UpdatePhaseIncrement();
	}

	void FSineWaveTableOsc::SetPhase(const float InPhase)
	{
		float ClampedInPhase = FMath::Clamp(InPhase, 0.0f, 1.0f);
		if (!FMath::IsNearlyEqual(ClampedInPhase, InitialPhase))
		{
			float PhaseDiff = ClampedInPhase - InitialPhase;
			// InitialPhase will be set to ClampedInPhase once we interpolate the phase in Generate
			InstantaneousPhase = ClampedInPhase;
			// Increment ReadIndex by phase difference and wrap around if necessary
			ReadIndex += PhaseDiff * WaveTableBuffer.Num();
			while (ReadIndex >= WaveTableBuffer.Num())
			{
				ReadIndex -= WaveTableBuffer.Num();
			}
			while (ReadIndex < 0.0f)
			{
				ReadIndex += WaveTableBuffer.Num();
			}
		}
	}

	void FSineWaveTableOsc::UpdatePhaseIncrement()
	{
		PhaseIncrement = (float)WaveTableBuffer.Num() * FrequencyHz / (float)SampleRate;
	}

	void FSineWaveTableOsc::Generate(float* OutBuffer, const int32 NumSamples)
	{
		// Interpolate phase shift over the block 
		const float HalfNumSamples = FMath::Max(1.f, static_cast<float>(NumSamples / 2));
		float PhaseShiftTrianglePeak = 0.0f;
		bool InterpolatePhase = !FMath::IsNearlyEqual(InitialPhase, InstantaneousPhase);
		if (InterpolatePhase)
		{
			float PhaseDiff = InstantaneousPhase - InitialPhase;
			PhaseShiftTrianglePeak = PhaseDiff * 2 / NumSamples;
			InitialPhase = InstantaneousPhase;
		}

		for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
		{
			// Interpolate between two samples
			const int32 ReadIndexPrev = FMath::Clamp((int32)ReadIndex, 0, WaveTableBuffer.Num() - 1);
			const float Alpha = FMath::Clamp(ReadIndex - (float)ReadIndexPrev, 0.f, 1.f);

			const int32 ReadIndexNext = (ReadIndexPrev + 1) % WaveTableBuffer.Num();
			OutBuffer[SampleIndex] = FMath::Lerp(WaveTableBuffer[ReadIndexPrev], WaveTableBuffer[ReadIndexNext], Alpha);
			
			// Increment ReadIndex and wrap around if necessary
			if (InterpolatePhase)
			{
				// Interpolate phase over the block using a triangle 
				float BlockFraction = static_cast<float>(SampleIndex);
				float PhaseShift = PhaseShiftTrianglePeak * FMath::Abs((FMath::Abs(1.f - BlockFraction / HalfNumSamples) - 1.f));
				ReadIndex += PhaseIncrement + PhaseShift;
			}
			else
			{
				ReadIndex += PhaseIncrement;
			}

			while (ReadIndex >= WaveTableBuffer.Num())
			{
				ReadIndex -= WaveTableBuffer.Num();
			}
		}
	}

	const TArray<float>& FSineWaveTableOsc::GetWaveTable()
	{
		auto MakeSineTable = []() -> const TArray<float>
		{
			// Generate the table
			TArray<float> WaveTable;
			WaveTable.AddUninitialized(SINE_WAVE_TABLE_SIZE);
			float* WaveTableData = WaveTable.GetData();
			for (int32 i = 0; i < SINE_WAVE_TABLE_SIZE; ++i)
			{
				float Phase = (float)i / SINE_WAVE_TABLE_SIZE;
				WaveTableData[i] = FMath::Sin(Phase * 2.f * PI);
			}
			return WaveTable;
		};

		static const TArray<float> SineWaveTable = MakeSineTable();
		return SineWaveTable;
	}
}
