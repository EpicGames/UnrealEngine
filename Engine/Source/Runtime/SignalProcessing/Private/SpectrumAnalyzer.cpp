// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/SpectrumAnalyzer.h"

#include "Algo/MaxElement.h"
#include "Algo/MinElement.h"
#include "DSP/ConstantQ.h"
#include "DSP/FFTAlgorithm.h"
#include "DSP/FloatArrayMath.h"
#include "SignalProcessingModule.h"

namespace Audio
{
	namespace SpectrumAnalyzerIntrinsics
	{
		// Bit mask for returning even numbers of int32
		const int32 EvenNumberMask = 0xFFFFFFFE;

		// Constant useful for calculating log10
		const float Loge10 = FMath::Loge(10.f);
	}


	// Implementation of spectrum band extractor
	class FSpectrumBandExtractor : public ISpectrumBandExtractor
	{
		using FBandSettings = ISpectrumBandExtractor::FBandSettings;
			// FBandSpec describes specifications for a single band.
			struct FBandSpec : public FBandSettings
			{
				// Location in output array where band value should be stored.
				int32 OutIndex;

				// The scaling parameter to apply to the power spectrum.
				float PowerSpectrumScale;

				FBandSpec(const FBandSettings& InBandSettings, const FSpectrumBandExtractorSettings& InSettings, const FSpectrumBandExtractorSpectrumSettings& InSpectrumSettings, int32 InOutIndex)
				:	FBandSettings(InBandSettings)
				,	OutIndex(InOutIndex)
				,	PowerSpectrumScale(1.f)
				{
					FBandSpec::Update(InSettings, InSpectrumSettings);
				}

				virtual ~FBandSpec() {}

				// Update calculates parameters that are specific to FFT implementation
				// and sample rate.
				virtual void Update(const FSpectrumBandExtractorSettings& InSettings, const FSpectrumBandExtractorSpectrumSettings& InSpectrumSettings)
				{
					PowerSpectrumScale = 1.f;
					float FloatFFTSize = FMath::Max(static_cast<float>(InSpectrumSettings.FFTSize), 1.f);

					switch (InSpectrumSettings.FFTScaling)
					{
						case EFFTScaling::MultipliedByFFTSize:
							PowerSpectrumScale = 1.f / (FloatFFTSize * FloatFFTSize);
							break;

						case EFFTScaling::MultipliedBySqrtFFTSize:
							PowerSpectrumScale = 1.f / FloatFFTSize;
							break;

						case EFFTScaling::DividedByFFTSize:
							PowerSpectrumScale = FloatFFTSize * FloatFFTSize;
							break;

						case EFFTScaling::DividedBySqrtFFTSize:
							PowerSpectrumScale = FloatFFTSize;
							break;

						case EFFTScaling::None:
						default:
							PowerSpectrumScale = 1.f;
							break;
					}
				}

				
				void UpdateOutputFromPowerSpectrum(const float* InputBuffer, int32 InNum, float* OutputBuffer, int32 OutNum) const
				{
					check(OutIndex >= 0);
					check(OutIndex < OutNum);

					OutputBuffer[OutIndex] = ExtractFromPowerSpectrum(InputBuffer, InNum) * PowerSpectrumScale;
				}

				virtual float ExtractFromPowerSpectrum(const float* InputBuffer, int32 InNum) const = 0;
			};

			// Specification for a nearest neighbor band.
			struct FNNBandSpec : public FBandSpec
			{
				// Use parent constructor
				using FBandSpec::FBandSpec;

				// Index in power spectrum to lookup band.
				int32 Index;

				virtual void Update(const FSpectrumBandExtractorSettings& InSettings, const FSpectrumBandExtractorSpectrumSettings& InSpectrumSettings) override
				{
					// Call parent class update.
					FBandSpec::Update(InSettings, InSpectrumSettings);

					// Update the index
					const int32 MaxSpectrumIndex = InSpectrumSettings.FFTSize / 2;
					const float Position = CenterFrequency / FMath::Max(InSpectrumSettings.SampleRate, 1.f) * InSpectrumSettings.FFTSize;
					Index = FMath::RoundToInt(Position);
					Index = FMath::Clamp(Index, 0, MaxSpectrumIndex);
				}

				virtual float ExtractFromPowerSpectrum(const float* InputBuffer, int32 InNum) const override
				{
					int32 SafeIndex = FMath::Clamp(Index, 0, InNum);

					return InputBuffer[SafeIndex];
				}
			};

			// Specification for a linearly interpolated band.
			struct FLerpBandSpec : public FBandSpec
			{
				// Use parent constructor
				using FBandSpec::FBandSpec;

				// Lower index power spectrum.
				int32 LowerIndex;

				// Upper index of power spectrum.
				int32 UpperIndex;

				// Value used for lerping between lower and upper band values.
				float Alpha;

				virtual void Update(const FSpectrumBandExtractorSettings& InSettings, const FSpectrumBandExtractorSpectrumSettings& InSpectrumSettings) override
				{
					// Call parent class update.
					FBandSpec::Update(InSettings, InSpectrumSettings);

					// Update lower index, upper index and alpha.
					const int32 MaxSpectrumIndex = InSpectrumSettings.FFTSize / 2;
					const float Position = CenterFrequency / FMath::Max(InSpectrumSettings.SampleRate, 1.f) * InSpectrumSettings.FFTSize;

					LowerIndex = FMath::FloorToInt(Position);
					UpperIndex = LowerIndex + 1;
					Alpha = Position - LowerIndex;

					LowerIndex = FMath::Clamp(LowerIndex, 0, MaxSpectrumIndex);
					UpperIndex = FMath::Clamp(UpperIndex, 0, MaxSpectrumIndex);
					Alpha = FMath::Clamp(Alpha, 0.f, 1.f);
				}

				virtual float ExtractFromPowerSpectrum(const float* InputBuffer, int32 InNum) const override
				{
					int32 SafeLowerIndex = FMath::Clamp(LowerIndex, 0, InNum);
					int32 SafeUpperIndex = FMath::Clamp(UpperIndex, 0, InNum);

					return FMath::Lerp<float>(InputBuffer[SafeLowerIndex], InputBuffer[SafeUpperIndex], Alpha);
				}
			};

			// Specification for band using quadratic interpolation.
			struct FQuadraticBandSpec : public FBandSpec
			{
				// Use parent constructor
				using FBandSpec::FBandSpec;

				// Lower index of power spectrum used for interpolation.
				int32 LowerIndex;

				// Middle index of power spectrum used for interpolation.
				int32 MidIndex;

				// Upper index of power spectrum used for interpolation.
				int32 UpperIndex;

				// Weight for lower value.
				float LowerWeight;

				// Weight for middle value.
				float MidWeight;

				// Weight for upper value.
				float UpperWeight;

				virtual void Update(const FSpectrumBandExtractorSettings& InSettings, const FSpectrumBandExtractorSpectrumSettings& InSpectrumSettings) override
				{
					// Call parent class update.
					FBandSpec::Update(InSettings, InSpectrumSettings);

					QFactor = FMath::Clamp(QFactor, 0.0001f, 10000.f);

					// Update indices and weights.
					const int32 MaxSpectrumIndex = InSpectrumSettings.FFTSize / 2;
					const float Position = CenterFrequency / FMath::Max(InSpectrumSettings.SampleRate, 1.f) * InSpectrumSettings.FFTSize;

					MidIndex = FMath::RoundToInt(Position);
					LowerIndex = MidIndex - 1;
					UpperIndex = MidIndex + 1;

					// Calculate polynomail weights 
					float RelativePosition = Position - LowerIndex;

					LowerWeight = ((RelativePosition - 1.f) * (RelativePosition - 2.f)) / 2.f;
					MidWeight = (RelativePosition * (RelativePosition - 2.f)) / -1.f;
					UpperWeight = (RelativePosition * (RelativePosition - 1.f)) / 2.f;

					LowerIndex = FMath::Clamp(LowerIndex, 0, MaxSpectrumIndex);
					MidIndex = FMath::Clamp(MidIndex, 0, MaxSpectrumIndex);
					UpperIndex = FMath::Clamp(UpperIndex, 0, MaxSpectrumIndex);
				}

				virtual float ExtractFromPowerSpectrum(const float* InputBuffer, int32 InNum) const override
				{
					int32 SafeLowerIndex = FMath::Clamp(LowerIndex, 0, InNum);
					int32 SafeMidIndex = FMath::Clamp(MidIndex, 0, InNum);
					int32 SafeUpperIndex = FMath::Clamp(UpperIndex, 0, InNum);

					const float LowerValue = InputBuffer[SafeLowerIndex];
					const float MidValue = InputBuffer[SafeMidIndex];
					const float UpperValue = InputBuffer[SafeUpperIndex];
					
					return (LowerValue * LowerWeight) + (MidValue * MidWeight) + (UpperValue * UpperWeight);
				}
			};

			// Specification for band using CQT band.
			struct FCQTBandSpec : public FBandSpec
			{
				// Use parent constructor
				using FBandSpec::FBandSpec;

				// Start index in power spectrum 
				int32 StartIndex;

				// Weights (offset by start index) to apply to power spectrum
				FAlignedFloatBuffer Weights;

				// Internal buffer used when calculating band.
				mutable FAlignedFloatBuffer WorkBuffer;

				virtual void Update(const FSpectrumBandExtractorSettings& InSettings, const FSpectrumBandExtractorSpectrumSettings& InSpectrumSettings) override
				{
					// Call parent class update.
					FBandSpec::Update(InSettings, InSpectrumSettings);

					// Update band weights and offset index.
					const int32 MaxSpectrumIndex = InSpectrumSettings.FFTSize / 2;
					const float Position = CenterFrequency / FMath::Max(InSpectrumSettings.SampleRate, 1.f) * InSpectrumSettings.FFTSize;
					FPseudoConstantQBandSettings CQTBandSettings;
					CQTBandSettings.CenterFreq = CenterFrequency;
					CQTBandSettings.BandWidth = FMath::Max(SMALL_NUMBER, CenterFrequency / FMath::Max(SMALL_NUMBER, QFactor));
					CQTBandSettings.FFTSize = InSpectrumSettings.FFTSize;
					CQTBandSettings.SampleRate = FMath::Max(1.f, InSpectrumSettings.SampleRate);
					CQTBandSettings.Normalization = EPseudoConstantQNormalization::EqualEnergy;

					StartIndex = 0;
					Weights.Reset();
					WorkBuffer.Reset();

					FPseudoConstantQ::FillArrayWithConstantQBand(CQTBandSettings, Weights, StartIndex);

					if (Weights.Num() > 0)
					{
						WorkBuffer.AddUninitialized(Weights.Num());
					}
				}

				virtual float ExtractFromPowerSpectrum(const float* InputBuffer, int32 InNum) const override
				{
					int32 SafeStartIndex = FMath::Clamp(StartIndex, 0, InNum);

					if (ensure((SafeStartIndex + Weights.Num()) <= InNum))
					{

						float Value = 0.f;
						int32 NumWeights = Weights.Num();

						if (NumWeights > 0)
						{
							check(NumWeights == WorkBuffer.Num());
							FMemory::Memcpy(WorkBuffer.GetData(), &InputBuffer[SafeStartIndex], NumWeights * sizeof(float));

							ArrayMultiplyInPlace(Weights, WorkBuffer);
							ArraySum(WorkBuffer, Value);
						}

						return Value;
					}

					return 0.f;
				}
			};

			// Tracks minimum/maximum values given an attack/release time.
			class FAutoRange 
			{
				float CurrentMinimum;
				float CurrentMaximum;
				float AttackTimeConstant;
				float ReleaseTimeConstant;

				bool bAreCurrentValuesValid;

				public:

					FAutoRange(float InAttackTime = 0.5f, float InReleaseTime = 30.f)
					:	CurrentMinimum(0.f)
					,	CurrentMaximum(0.f)
					,	AttackTimeConstant(1.f)
					,	ReleaseTimeConstant(1.f)
					,	bAreCurrentValuesValid(false)
					{
						SetAttackTime(InAttackTime);
						SetReleaseTime(InReleaseTime);
					}	

					void Set(float InMinimum, float InMaximum)
					{
						CurrentMinimum = InMinimum;
						CurrentMaximum = InMaximum;
						bAreCurrentValuesValid = true;
					}

					
					void Update(float InMinimumValue, float InMaximumValue, float InTimeDelta)
					{
						if (bAreCurrentValuesValid)
						{
							InTimeDelta = FMath::Max(0.f, InTimeDelta);

							// The time delta isn't constant between updates, so we need to determine
							// how much the value will change.
							const float AttackCoef = InTimeDelta * AttackTimeConstant;
							const float ReleaseCoef = InTimeDelta * ReleaseTimeConstant;

							const float MaxDiff = InMaximumValue - CurrentMaximum;

							if (MaxDiff > 0.f)
							{
								CurrentMaximum += MaxDiff * AttackCoef;
							}
							else
							{
								CurrentMaximum += MaxDiff * ReleaseCoef;
							}

							const float MinDiff = InMinimumValue - CurrentMinimum;

							if (MinDiff < 0.f)
							{
								CurrentMinimum += MinDiff * AttackCoef;
							}
							else
							{
								CurrentMinimum += MinDiff * ReleaseCoef;
							}

						}
						else
						{
							// On first time around, initialize to given min/max values.
							Set(InMinimumValue, InMaximumValue);

						}
					}

					// Normalize input values to the min/max range. 
					void Normalize(TArrayView<float> InValues) const
					{
						
						ArrayClampInPlace(InValues, CurrentMinimum, CurrentMaximum);

						ArraySubtractByConstantInPlace(InValues, CurrentMinimum);

						float Range = FMath::Max(SMALL_NUMBER, CurrentMaximum - CurrentMinimum);

						ArrayMultiplyByConstantInPlace(InValues, 1.f / Range);
					}

					void SetAttackTime(float InAttackTime)
					{
						// Factor set to achieve 90% of value by attack time.
						AttackTimeConstant = 0.9f / FMath::Max(0.001f, InAttackTime);
					}
					
					void SetReleaseTime(float InReleaseTime)
					{
						// Factor set to achieve 90% of value by release time.
						ReleaseTimeConstant = 0.9f / FMath::Max(0.001f, InReleaseTime);
					}

			};


		public:
			FSpectrumBandExtractor(const FSpectrumBandExtractorSettings& InSettings)
			:	Settings(InSettings)
			,	LastTimestamp(0)
			{
				AutoRange.SetAttackTime(Settings.AutoRangeAttackTimeInSeconds);
				AutoRange.SetReleaseTime(Settings.AutoRangeReleaseTimeInSeconds);
			}

			virtual void SetSettings(const FSpectrumBandExtractorSettings& InSettings) override
			{
				Settings = InSettings;
				AutoRange.SetAttackTime(Settings.AutoRangeAttackTimeInSeconds);
				AutoRange.SetReleaseTime(Settings.AutoRangeReleaseTimeInSeconds);
			}

			virtual void SetSpectrumSettings(const FSpectrumBandExtractorSpectrumSettings& InSpectrumSettings) override
			{
				bool bSpectrumSettingsChanged = (SpectrumSettings != InSpectrumSettings);
				SpectrumSettings = InSpectrumSettings;

				if (bSpectrumSettingsChanged)
				{

					// If the settings have changed from the previous call, the band specs
					// need to be updated with the new information.
					UpdateBandSpecs();
				}
			}
			
			// Clear out all added bands.
			virtual void RemoveAllBands() override
			{
				NNBandSpecs.Reset();
				LerpBandSpecs.Reset();
				QuadraticBandSpecs.Reset();
				CQTBandSpecs.Reset();
			}

			// Return total number of bands.
			virtual int32 GetNumBands() const override
			{
				int32 Num = NNBandSpecs.Num();
				Num += LerpBandSpecs.Num();
				Num += QuadraticBandSpecs.Num();
				Num += CQTBandSpecs.Num();

				return Num;
			}

			virtual void AddBand(const FBandSettings& InBandSettings) override
			{
				switch (InBandSettings.Type)
				{
					case EBandType::NearestNeighbor:
						AddBand<FNNBandSpec>(NNBandSpecs, InBandSettings);
						break;

					case EBandType::Lerp:
						AddBand<FLerpBandSpec>(LerpBandSpecs, InBandSettings);
						break;

					case EBandType::Quadratic:
						AddBand<FQuadraticBandSpec>(QuadraticBandSpecs, InBandSettings);
						break;

					case EBandType::ConstantQ:
					default:
						AddBand<FCQTBandSpec>(CQTBandSpecs, InBandSettings);
						break;
				}
			}


			// Extract band from input.
			virtual void ExtractBands(const FAlignedFloatBuffer& InComplexBuffer, double InTimestamp, TArray<float>& OutValues) override
			{
				const int32 NumComplex = InComplexBuffer.Num();

				float TimeDelta = FMath::Max(0.0f, static_cast<float>(InTimestamp - LastTimestamp));
				LastTimestamp = InTimestamp;

				check(NumComplex == (SpectrumSettings.FFTSize + 2));

				OutValues.Reset();
				OutValues.AddZeroed(GetNumBands());

				PowerSpectrum.Reset();
				if (NumComplex > 1)
				{
					PowerSpectrum.AddUninitialized(NumComplex / 2);
				}

				// All band extractors expect a power spectrum
				ArrayComplexToPower(InComplexBuffer, PowerSpectrum);

				ExtractBands(PowerSpectrum, NNBandSpecs, OutValues);
				ExtractBands(PowerSpectrum, LerpBandSpecs, OutValues);
				ExtractBands(PowerSpectrum, QuadraticBandSpecs, OutValues);
				ExtractBands(PowerSpectrum, CQTBandSpecs, OutValues);

				ApplyMetric(OutValues);

				if (Settings.bDoAutoRange)
				{
					Normalize(OutValues);

					float* MinimumValuePtr = Algo::MinElement(OutValues);
					float* MaximumValuePtr = Algo::MaxElement(OutValues);

					float MinimumValue = MinimumValuePtr == nullptr ? 0.f : *MinimumValuePtr;
					float MaximumValue = MaximumValuePtr == nullptr ? 0.f : *MaximumValuePtr;

					AutoRange.Update(MinimumValue, MaximumValue, TimeDelta);

					AutoRange.Normalize(OutValues);
				}
				else if (Settings.bDoNormalize)
				{
					Normalize(OutValues);

					ArrayClampInPlace(OutValues, 0.f, 1.f);
				}
			}

		private:

			// Adds a band spec and returns a reference to the added spec.
			template<typename T> 
			T& AddBand(TArray<T>& InBandSpecs, const FBandSettings& InBandSettings)
			{
				int32 OutIndex = GetNumBands();

				T BandSpec(InBandSettings, Settings, SpectrumSettings, OutIndex);
				BandSpec.Update(Settings, SpectrumSettings);

				return InBandSpecs.Add_GetRef(MoveTemp(BandSpec));
			}

			// Calls update on all band specs in the array.
			template<typename T>
			void UpdateBandSpecs(TArray<T>& InSpecs)
			{
				for (T& BandSpec : InSpecs)
				{
					BandSpec.Update(Settings, SpectrumSettings);
				}
			}

			// Updates all band specs.
			void UpdateBandSpecs()
			{
				UpdateBandSpecs<FNNBandSpec>(NNBandSpecs);
				UpdateBandSpecs<FLerpBandSpec>(LerpBandSpecs);
				UpdateBandSpecs<FQuadraticBandSpec>(QuadraticBandSpecs);
				UpdateBandSpecs<FCQTBandSpec>(CQTBandSpecs);
			}

			template<typename T>
			void ExtractBands(const FAlignedFloatBuffer& InPowerSpectrum, const TArray<T>& InBandSpecs, TArray<float>& OutValues) const
			{
				float* OutData = OutValues.GetData();
				int32 OutNum = OutValues.Num();
				const float* InData = InPowerSpectrum.GetData();
				int32 InNum = InPowerSpectrum.Num();

				for (const T& Spec : InBandSpecs)
				{
					Spec.UpdateOutputFromPowerSpectrum(InData, InNum, OutData, OutNum);
				}
			}

			// apply metric to a band value
			void ApplyMetric(TArray<float>& InValues) const
			{

				switch (Settings.Metric)
				{
					case FSpectrumBandExtractorSettings::EMetric::Magnitude:
						ArraySqrtInPlace(InValues);
						break;

					case FSpectrumBandExtractorSettings::EMetric::Decibel:
						ArrayPowerToDecibelInPlace(InValues, Settings.DecibelNoiseFloor);
						break;

					case FSpectrumBandExtractorSettings::EMetric::Power:
					default:
						break;
				}
			}

			void Normalize(TArray<float>& InValues) const
			{
				// Only decibel values need to be normalized. 
				if (FSpectrumBandExtractorSettings::EMetric::Decibel == Settings.Metric)
				{
					for (float& Value : InValues)
					{
						if (!FMath::IsFinite(Value))
						{
							Value = Settings.DecibelNoiseFloor;
						}
					}

					ArrayClampMinInPlace(InValues, Settings.DecibelNoiseFloor);

					ArraySubtractByConstantInPlace(InValues, Settings.DecibelNoiseFloor);

					float Range = FMath::Max(0.01f, -Settings.DecibelNoiseFloor);

					ArrayMultiplyByConstantInPlace(InValues, 1.f / Range);
				}
			}

			FSpectrumBandExtractorSettings Settings;
			FSpectrumBandExtractorSpectrumSettings SpectrumSettings;
			double LastTimestamp;

			FAlignedFloatBuffer PowerSpectrum;

			TArray<FNNBandSpec> NNBandSpecs;
			TArray<FLerpBandSpec> LerpBandSpecs;
			TArray<FQuadraticBandSpec> QuadraticBandSpecs;
			TArray<FCQTBandSpec> CQTBandSpecs;

			FAutoRange AutoRange;
	};


	// Creates a concreted implementation of the ISpectrumBandExtractor interface.
	TUniquePtr<ISpectrumBandExtractor> ISpectrumBandExtractor::CreateSpectrumBandExtractor(const FSpectrumBandExtractorSettings& InSettings)
	{
		return MakeUnique<FSpectrumBandExtractor>(InSettings);
	}

	FSpectrumAnalyzer::FSpectrumAnalyzer()
		: CurrentSettings(FSpectrumAnalyzerSettings())
		, bSettingsWereUpdated(false)
		, bIsInitialized(false)
		, SampleRate(0.0f)
		, Window(CurrentSettings.WindowType, (int32)CurrentSettings.FFTSize, 1, false)
		, SampleCounter(0)
		, InputQueue(FMath::Max((int32)CurrentSettings.FFTSize * 4, 4096))
		, FrequencyBuffer((int32)CurrentSettings.FFTSize)
		, LockedBufferTimestamp(0)
		, LockedFrequencyVector(nullptr)
	{
	}

	FSpectrumAnalyzer::FSpectrumAnalyzer(const FSpectrumAnalyzerSettings& InSettings, float InSampleRate)
		: CurrentSettings(InSettings)
		, bSettingsWereUpdated(false)
		, bIsInitialized(true)
		, SampleRate(InSampleRate)
		, Window(InSettings.WindowType, (int32)InSettings.FFTSize, 1, false)
		, SampleCounter(0)
		, InputQueue(FMath::Max((int32)CurrentSettings.FFTSize * 4, 4096))
		, FrequencyBuffer((int32)InSettings.FFTSize)
		, LockedBufferTimestamp(0)
		, LockedFrequencyVector(nullptr)
	{
		ResetSettings();
	}

	FSpectrumAnalyzer::FSpectrumAnalyzer(float InSampleRate)
		: CurrentSettings(FSpectrumAnalyzerSettings())
		, bSettingsWereUpdated(false)
		, bIsInitialized(true)
		, SampleRate(InSampleRate)
		, Window(CurrentSettings.WindowType, (int32)CurrentSettings.FFTSize, 1, false)
		, SampleCounter(0)
		, InputQueue(FMath::Max((int32)CurrentSettings.FFTSize * 4, 4096))
		, FrequencyBuffer((int32)CurrentSettings.FFTSize)
		, LockedBufferTimestamp(0)
		, LockedFrequencyVector(nullptr)
	{
		ResetSettings();
	}

	void FSpectrumAnalyzer::Init(float InSampleRate)
	{
		FSpectrumAnalyzerSettings DefaultSettings = FSpectrumAnalyzerSettings();
		Init(DefaultSettings, InSampleRate);
	}

	void FSpectrumAnalyzer::Init(const FSpectrumAnalyzerSettings& InSettings, float InSampleRate)
	{
		CurrentSettings = InSettings;
		bSettingsWereUpdated = false;
		SampleRate = InSampleRate;
		SampleCounter.Set(0);
		InputQueue.SetCapacity(FMath::Max((int32)CurrentSettings.FFTSize * 4, 4096));
		FrequencyBuffer.Reset((int32)CurrentSettings.FFTSize);
		ResetSettings();

		bIsInitialized = true;
	}

	void FSpectrumAnalyzer::ResetSettings()
	{
		// If the game thread has locked a frequency vector, we can't resize our buffers under it.
		// Thus, wait until it's unlocked.
		if (LockedFrequencyVector != nullptr)
		{
			return;
		}

		Window = FWindow(CurrentSettings.WindowType, (int32)CurrentSettings.FFTSize, 1, false);
		FFTSize = (int32) CurrentSettings.FFTSize;
		int32 Log2FFTSize = 9;
		if (FFTSize > 0)
		{
			// FFTSize must be log2
			check(FMath::CountBits(FFTSize) == 1);
			Log2FFTSize = FMath::CountTrailingZeros(FFTSize);
		}

		AnalysisTimeDomainBuffer.Reset();
		
		if (FMath::IsNearlyZero(CurrentSettings.HopSize))
		{
			HopInSamples = GetCOLAHopSizeForWindow(CurrentSettings.WindowType, (uint32)CurrentSettings.FFTSize);
		}
		else
		{
			HopInSamples = FMath::FloorToInt((float)CurrentSettings.FFTSize * CurrentSettings.HopSize);
		}

		// Create a new FFT
		FFFTSettings FFTSettings;
		FFTSettings.Log2Size = Log2FFTSize;
		FFTSettings.bArrays128BitAligned = true;
		FFTSettings.bEnableHardwareAcceleration = true;

		FFT = FFFTFactory::NewFFTAlgorithm(FFTSettings);

		if (!FFT.IsValid())
		{
			if (FFFTFactory::AreFFTSettingsSupported(FFTSettings))
			{
				UE_LOGF(LogSignalProcessing, Error, "Failed to create fft for supported settings.")
			}
			else
			{
				UE_LOGF(LogSignalProcessing, Warning, "FFT Settings are unsupported.")
			}
			FFTScaling = EFFTScaling::None;

			if (FFTSize > 0)
			{
				AnalysisTimeDomainBuffer.AddZeroed(FFTSize);
				FrequencyBuffer.Reset(FFTSize);
			}
		}
		else
		{
			int32 NumFFTInput = FFT->NumInputFloats();
			int32 NumFFTOutput = FFT->NumOutputFloats();
			FFTScaling = FFT->ForwardScaling();

			if (NumFFTInput > 0)
			{
				AnalysisTimeDomainBuffer.AddUninitialized(NumFFTInput);
			}

			FrequencyBuffer.Reset(NumFFTOutput);
		}
		
		bSettingsWereUpdated = false;
	}

	void FSpectrumAnalyzer::PerformInterpolation(const FAlignedFloatBuffer& InComplexBuffer, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod, const float InFreq, float& OutReal, float& OutImag)
	{
		const float* InComplexData = InComplexBuffer.GetData();
		const int32 VectorLength = InComplexBuffer.Num();
		const int32 NyquistPosition = VectorLength - 2;
		
		const float Nyquist = SampleRate / 2.f;

		// Fractional position in the frequency vector in terms of indices.
		// float Position = NyquistPosition + (InFreq / Nyquist);
		const float NormalizedFreq = (InFreq / Nyquist);
		float Position = InFreq >= 0 ? (NormalizedFreq * VectorLength) : 0.f;
		
		switch (InMethod)
		{
		case Audio::FSpectrumAnalyzer::EPeakInterpolationMethod::NearestNeighbor:
		{
			int32 Index = FMath::RoundToInt(Position) & SpectrumAnalyzerIntrinsics::EvenNumberMask;

			Index = FMath::Clamp(Index, 0, NyquistPosition);

			OutReal = InComplexData[Index];
			OutImag = InComplexData[Index + 1];

			break;
		}
			
		case Audio::FSpectrumAnalyzer::EPeakInterpolationMethod::Linear:
		{
			int32 LowerIndex = FMath::FloorToInt(Position) & SpectrumAnalyzerIntrinsics::EvenNumberMask;
			int32 UpperIndex = LowerIndex + 2;

			LowerIndex = FMath::Clamp(LowerIndex, 0, NyquistPosition);
			UpperIndex = FMath::Clamp(UpperIndex, 0, NyquistPosition);

			const float PositionFraction = Position - LowerIndex;

			const float y1Real = InComplexData[LowerIndex];
			const float y2Real = InComplexData[UpperIndex];

			OutReal = FMath::Lerp<float>(y1Real, y1Real, PositionFraction);

			const float y1Imag = InComplexData[LowerIndex + 1];
			const float y2Imag = InComplexData[UpperIndex + 1];

			OutImag = FMath::Lerp<float>(y1Imag, y2Imag, PositionFraction);
			break;
		}	
		case Audio::FSpectrumAnalyzer::EPeakInterpolationMethod::Quadratic:
		{
			// Note: math here does not interpolate quadratically. 
			const int32 MidIndex = FMath::Clamp(FMath::RoundToInt(Position) & SpectrumAnalyzerIntrinsics::EvenNumberMask, 0, NyquistPosition);
			const int32 LowerIndex = FMath::Max(0, MidIndex - 2);
			const int32 UpperIndex = FMath::Min(NyquistPosition, MidIndex + 2);

			const float y1Real = InComplexData[LowerIndex];
			const float y2Real = InComplexData[MidIndex];
			const float y3Real = InComplexData[UpperIndex];

			const float InterpReal = (y3Real - y1Real) / (2.f * (2.f * y2Real - y1Real - y3Real));
			
			OutReal = InterpReal;

			const float y1Imag = InComplexData[LowerIndex + 1];
			const float y2Imag = InComplexData[MidIndex + 1];
			const float y3Imag = InComplexData[UpperIndex + 1];
			const float InterpImag = (y3Imag - y1Imag) / (2.f * (2.f * y2Imag - y1Imag - y3Imag));

			OutImag = InterpImag;
			break;
		}
			
		default:
			break;
		}
	}

	void FSpectrumAnalyzer::SetSettings(const FSpectrumAnalyzerSettings& InSettings)
	{
		CurrentSettings = InSettings;
		bSettingsWereUpdated = true;
	}

	void FSpectrumAnalyzer::GetSettings(FSpectrumAnalyzerSettings& OutSettings)
	{
		OutSettings = CurrentSettings;
	}

	float FSpectrumAnalyzer::GetMagnitudeForFrequency(float InFrequency, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod)
	{
		if (!bIsInitialized)
		{
			return 0.f;
		}

		const FAlignedFloatBuffer* OutVector = nullptr;
		bool bShouldUnlockBuffer = true;

		if (LockedFrequencyVector)
		{
			OutVector = LockedFrequencyVector;
			bShouldUnlockBuffer = false;
		}
		else
		{
			OutVector = &FrequencyBuffer.LockMostRecentBuffer();
		}

		// Perform work.
		if (OutVector)
		{
			float OutMagnitude = 0.0f;

			float InterpolatedReal, InterpolatedImag;
			PerformInterpolation(*OutVector, InMethod, InFrequency, InterpolatedReal, InterpolatedImag);

			OutMagnitude = FMath::Sqrt((InterpolatedReal * InterpolatedReal) + (InterpolatedImag * InterpolatedImag));

			if (bShouldUnlockBuffer)
			{
				FrequencyBuffer.UnlockBuffer();
				LockedBufferTimestamp = 0.0;
			}

			return OutMagnitude;
		}

		// If we got here, something went wrong, so just output zero.
		return 0.0f;
	}

	float FSpectrumAnalyzer::GetNormalizedMagnitudeForFrequency(float InFrequency, EPeakInterpolationMethod InMethod)
	{
		float Norm = static_cast<uint16>(CurrentSettings.FFTSize) * 0.5f;

		if (Norm > 0.0f)
		{
			return GetMagnitudeForFrequency(InFrequency, InMethod) / Norm;
		}

		return 0.f;
	}

	float FSpectrumAnalyzer::GetPhaseForFrequency(float InFrequency, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod)
	{
		if (!bIsInitialized)
		{
			return 0.f;
		}

		const FAlignedFloatBuffer* OutVector = nullptr;
		bool bShouldUnlockBuffer = true;

		if (LockedFrequencyVector)
		{
			OutVector = LockedFrequencyVector;
			bShouldUnlockBuffer = false;
		}
		else
		{
			OutVector = &FrequencyBuffer.LockMostRecentBuffer();
		}

		// Perform work.
		if (OutVector)
		{
			float OutPhase = 0.0f;

			float InterpolatedReal, InterpolatedImag;
			PerformInterpolation(*OutVector, InMethod, InFrequency, InterpolatedReal, InterpolatedImag);

			OutPhase = FMath::Atan2(InterpolatedImag, InterpolatedReal);

			if (bShouldUnlockBuffer)
			{
				FrequencyBuffer.UnlockBuffer();
				LockedBufferTimestamp = 0.0;
			}

			return OutPhase;
		}

		// If we got here, something went wrong, so just output zero.
		return 0.0f;
	}

	// Return bands extracted by band extractor.
	void FSpectrumAnalyzer::GetBands(ISpectrumBandExtractor& InExtractor, TArray<float>& OutValues) 
	{
		OutValues.Reset();

		if (!bIsInitialized)
		{
			return;
		}

		const FAlignedFloatBuffer* AnalysisBuffer = nullptr;
		bool bShouldUnlockBuffer = true;

		FSpectrumBandExtractorSpectrumSettings ExtractorSettings;
		ExtractorSettings.SampleRate = SampleRate;
		ExtractorSettings.FFTSize = FFTSize;
		ExtractorSettings.FFTScaling = FFTScaling;
		ExtractorSettings.WindowType = Window.GetWindowType();

		// This should have minimal cost if settings have not changed between calls.
		InExtractor.SetSpectrumSettings(ExtractorSettings);

		if (LockedFrequencyVector)
		{
			AnalysisBuffer = LockedFrequencyVector;
			bShouldUnlockBuffer = false;
		}
		else
		{
			AnalysisBuffer = &FrequencyBuffer.LockMostRecentBuffer(LockedBufferTimestamp);
		}

		// Perform work.
		if (AnalysisBuffer)
		{
			InExtractor.ExtractBands(*AnalysisBuffer, LockedBufferTimestamp, OutValues);

			if (bShouldUnlockBuffer)
			{
				FrequencyBuffer.UnlockBuffer();
				LockedBufferTimestamp = 0.0;
			}
		}
	}

	void FSpectrumAnalyzer::LockOutputBuffer()
	{
		if (!bIsInitialized)
		{
			return;
		}

		if (LockedFrequencyVector != nullptr)
		{
			FrequencyBuffer.UnlockBuffer();
			LockedBufferTimestamp = 0.0;
		}

		LockedFrequencyVector = &FrequencyBuffer.LockMostRecentBuffer(LockedBufferTimestamp);
	}

	void FSpectrumAnalyzer::UnlockOutputBuffer()
	{
		if (!bIsInitialized)
		{
			return;
		}

		if (LockedFrequencyVector != nullptr)
		{
			FrequencyBuffer.UnlockBuffer();
			LockedFrequencyVector = nullptr;
			LockedBufferTimestamp = 0.0;
		}
	}

	bool FSpectrumAnalyzer::PushAudio(const TSampleBuffer<float>& InBuffer)
	{
		check(InBuffer.GetNumChannels() == 1);
		return PushAudio(InBuffer.GetData(), InBuffer.GetNumSamples());
	}

	bool FSpectrumAnalyzer::PushAudio(const float* InBuffer, int32 NumSamples)
	{
		SampleCounter.Add(NumSamples);

		return InputQueue.Push(InBuffer, NumSamples) == NumSamples;
	}

	bool FSpectrumAnalyzer::PerformAnalysisIfPossible(bool bUseLatestAudio)
	{
		if (!IsInitialized())
		{
			return false;
		}		

		// If settings were updated, perform resizing and parameter updates here:
		if (bSettingsWereUpdated)
		{
			ResetSettings();
		}


		FAlignedFloatBuffer& FFTOutput = FrequencyBuffer.StartWorkOnBuffer();

		// If we have enough audio pushed to the spectrum analyzer and we have an available buffer to work in,
		// we can start analyzing.
		uint32 RequiredSize = FMath::Max(FFTSize, HopInSamples);
		if (InputQueue.Num() >= RequiredSize)
		{
			int64 WindowSampleCenterIndex = 0;

			float* TimeDomainBuffer = AnalysisTimeDomainBuffer.GetData();

			if (bUseLatestAudio)
			{
				WindowSampleCenterIndex = SampleCounter.GetValue() - (FFTSize / 2);

				// If we are only using the latest audio, scrap the oldest audio in the InputQueue:
				InputQueue.SetNum((uint32)FFTSize);
				InputQueue.Pop(TimeDomainBuffer, FFTSize);
			}
			else
			{
				WindowSampleCenterIndex = SampleCounter.GetValue() - InputQueue.Num() + (FFTSize / 2);
				
				// Perform pop/peek here based on FFT size and hop amount.
				InputQueue.Peek(TimeDomainBuffer, FFTSize);
				InputQueue.Pop(HopInSamples);
			}

			double Timestamp = static_cast<double>(WindowSampleCenterIndex) / FMath::Max(SampleRate, 1.f);

			// apply window if necessary.
			Window.ApplyToBuffer(TimeDomainBuffer);

			// Perform FFT.
			if (FFT.IsValid())
			{
				check(AnalysisTimeDomainBuffer.Num() == FFT->NumInputFloats());
				check(FFTOutput.Num() == FFT->NumOutputFloats());

				FFT->ForwardRealToComplex(TimeDomainBuffer, FFTOutput.GetData());
			}
			else
			{
				if (FFTOutput.Num() > 0)
				{
					FMemory::Memset(FFTOutput.GetData(), 0, sizeof(float) * FFTOutput.Num());
				}
			}

			// We're done, so unlock this vector.
			FrequencyBuffer.StopWorkOnBuffer(Timestamp);

			return true;
		}
		else
		{
			return false;
		}
	}

	bool FSpectrumAnalyzer::IsInitialized()
	{
		return bIsInitialized;
	}

	static const int32 SpectrumAnalyzerBufferSize = 4;

	FSpectrumAnalyzerBuffer::FSpectrumAnalyzerBuffer()
		: OutputIndex(0)
		, InputIndex(0)
	{
	}

	FSpectrumAnalyzerBuffer::FSpectrumAnalyzerBuffer(int32 InNum)
	{
		Reset(InNum);
	}

	void FSpectrumAnalyzerBuffer::Reset(int32 InNum)
	{
		FScopeLock ScopeLock(&BufferIndicesCriticalSection);

		static_assert(SpectrumAnalyzerBufferSize > 2, "Please ensure that SpectrumAnalyzerBufferSize is greater than 2.");
		
		ComplexBuffers.Reset();

		for (int32 Index = 0; Index < SpectrumAnalyzerBufferSize; Index++)
		{
			FAlignedFloatBuffer& Buffer = ComplexBuffers.Emplace_GetRef();

			if (InNum > 0)
			{
				Buffer.AddZeroed(InNum);
			}
		}

		Timestamps.Reset();
		Timestamps.AddZeroed(SpectrumAnalyzerBufferSize);

		InputIndex = 0;
		OutputIndex = 0;
	}

	void FSpectrumAnalyzerBuffer::IncrementInputIndex()
	{
		FScopeLock ScopeLock(&BufferIndicesCriticalSection);

		InputIndex = (InputIndex + 1) % SpectrumAnalyzerBufferSize;
		if (InputIndex == OutputIndex)
		{
			InputIndex = (InputIndex + 1) % SpectrumAnalyzerBufferSize;
		}

		check(InputIndex != OutputIndex);
	}

	void FSpectrumAnalyzerBuffer::IncrementOutputIndex()
	{
		FScopeLock ScopeLock(&BufferIndicesCriticalSection);

		OutputIndex = (OutputIndex + 1) % SpectrumAnalyzerBufferSize;
		if (InputIndex == OutputIndex)
		{
			OutputIndex = (OutputIndex + 1) % SpectrumAnalyzerBufferSize;
		}

		check(InputIndex != OutputIndex);
	}

	FAlignedFloatBuffer& FSpectrumAnalyzerBuffer::StartWorkOnBuffer()
	{
		FScopeLock ScopeLock(&BufferIndicesCriticalSection);
		return ComplexBuffers[InputIndex];
	}

	void FSpectrumAnalyzerBuffer::StopWorkOnBuffer(double InTimestamp)
	{
		FScopeLock ScopeLock(&BufferIndicesCriticalSection);
		Timestamps[InputIndex] = InTimestamp;
		IncrementInputIndex();
	}

	const FAlignedFloatBuffer& FSpectrumAnalyzerBuffer::LockMostRecentBuffer(double& OutTimestamp) const
	{
		FScopeLock ScopeLock(&BufferIndicesCriticalSection);
		OutTimestamp = Timestamps[OutputIndex];
		return ComplexBuffers[OutputIndex];
	}

	const FAlignedFloatBuffer& FSpectrumAnalyzerBuffer::LockMostRecentBuffer() const
	{
		FScopeLock ScopeLock(&BufferIndicesCriticalSection);
		return ComplexBuffers[OutputIndex];
	}

	void FSpectrumAnalyzerBuffer::UnlockBuffer()
	{
		IncrementOutputIndex();
	}


	void FSpectrumAnalysisAsyncWorker::DoWork()
	{
		FScopeLock AbandonLock(&NonAbandonableSection);
		
		if (!bIsAbandoned)
		{
			TSharedPtr<FSpectrumAnalyzer, ESPMode::ThreadSafe> AnalyzerSharedPtr = AnalyzerWeakPtr.Pin();
			if (AnalyzerSharedPtr.IsValid())
			{
				AnalyzerSharedPtr->PerformAnalysisIfPossible(bUseLatestAudio);
			}
		}
	}

	void FSpectrumAnalysisAsyncWorker::Abandon()
	{
		FScopeLock AbandonLock(&NonAbandonableSection);
		bIsAbandoned = true;
	}

	FAsyncSpectrumAnalyzer::FAsyncSpectrumAnalyzer()
		: Analyzer(MakeShared<FSpectrumAnalyzer, ESPMode::ThreadSafe>())
	{
	}
	FAsyncSpectrumAnalyzer::FAsyncSpectrumAnalyzer(float InSampleRate)
	:	Analyzer(MakeShared<FSpectrumAnalyzer, ESPMode::ThreadSafe>(InSampleRate))
	{
	}

	FAsyncSpectrumAnalyzer::FAsyncSpectrumAnalyzer(const FSpectrumAnalyzerSettings& InSettings, float InSampleRate)
	:	Analyzer(MakeShared<FSpectrumAnalyzer, ESPMode::ThreadSafe>(InSettings, InSampleRate))
	{
	}

	FAsyncSpectrumAnalyzer::~FAsyncSpectrumAnalyzer()
	{
		if (AsyncAnalysisTask.IsValid())
		{
			if (!AsyncAnalysisTask->IsDone())
			{
				if (!AsyncAnalysisTask->Cancel())
				{
					const bool bDoWorkOnThisThreadIfNotStarted = true;
					AsyncAnalysisTask->EnsureCompletion(bDoWorkOnThisThreadIfNotStarted);
				}
			}
			
		}
	}

	void FAsyncSpectrumAnalyzer::Init(float InSampleRate)
	{
		Analyzer->Init(InSampleRate);
	}

	void FAsyncSpectrumAnalyzer::Init(const FSpectrumAnalyzerSettings& InSettings, float InSampleRate)
	{
		Analyzer->Init(InSettings, InSampleRate);
	}

	bool FAsyncSpectrumAnalyzer::IsInitialized()
	{
		return Analyzer->IsInitialized();
	}

	void FAsyncSpectrumAnalyzer::SetSettings(const FSpectrumAnalyzerSettings& InSettings)
	{
		Analyzer->SetSettings(InSettings);
	}

	void FAsyncSpectrumAnalyzer::GetSettings(FSpectrumAnalyzerSettings& OutSettings)
	{
		Analyzer->GetSettings(OutSettings);
	}

	float FAsyncSpectrumAnalyzer::GetMagnitudeForFrequency(float InFrequency, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod)
	{
		return Analyzer->GetMagnitudeForFrequency(InFrequency, InMethod);
	}

	float FAsyncSpectrumAnalyzer::GetNormalizedMagnitudeForFrequency(float InFrequency, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod)
	{
		return Analyzer->GetNormalizedMagnitudeForFrequency(InFrequency, InMethod);
	}

	float FAsyncSpectrumAnalyzer::GetPhaseForFrequency(float InFrequency, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod)
	{
		return Analyzer->GetPhaseForFrequency(InFrequency, InMethod);
	}

	void FAsyncSpectrumAnalyzer::GetBands(ISpectrumBandExtractor& InExtractor, TArray<float>& OutValues)
	{
		Analyzer->GetBands(InExtractor, OutValues);
	}

	void FAsyncSpectrumAnalyzer::LockOutputBuffer()
	{
		Analyzer->LockOutputBuffer();
	}

	void FAsyncSpectrumAnalyzer::UnlockOutputBuffer()
	{
		Analyzer->UnlockOutputBuffer();
	}
	
	bool FAsyncSpectrumAnalyzer::PushAudio(const TSampleBuffer<float>& InBuffer)
	{
		return Analyzer->PushAudio(InBuffer);
	}

	bool FAsyncSpectrumAnalyzer::PushAudio(const float* InBuffer, int32 NumSamples)
	{
		return Analyzer->PushAudio(InBuffer, NumSamples);
	}

	bool FAsyncSpectrumAnalyzer::PerformAnalysisIfPossible(bool bUseLatestAudio)
	{
		return Analyzer->PerformAnalysisIfPossible(bUseLatestAudio);
	}

	bool FAsyncSpectrumAnalyzer::PerformAsyncAnalysisIfPossible(bool bUseLatestAudio)
	{
		if (!IsInitialized())
		{
			return false;
		}		

		// if bAsync is true, kick off a new task if one isn't in flight already, and return.
		if (!AsyncAnalysisTask.IsValid())
		{
			AsyncAnalysisTask.Reset(new FSpectrumAnalyzerTask(Analyzer, bUseLatestAudio));
			AsyncAnalysisTask->StartBackgroundTask();
		}
		else if (AsyncAnalysisTask->IsDone())
		{
			AsyncAnalysisTask->StartBackgroundTask();
		}

		return true;
	}

}

