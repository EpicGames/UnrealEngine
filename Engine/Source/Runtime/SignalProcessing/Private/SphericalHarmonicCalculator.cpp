// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/SphericalHarmonicCalculator.h"

namespace Audio
{
	namespace SphericalHarmonicsPrivate
	{
		// Common constants used in basis functions.
		constexpr float SQRT_3 = 1.73205080757f;
		constexpr float SQRT_3_DIV_2  = 1.224744871391589f;     
		constexpr float SQRT_5 = 2.2360679775f;
		constexpr float SQRT_6 = 2.44948974278f;
		constexpr float SQRT_7 = 2.64575131106f;
		constexpr float SQRT_10 = 3.16227766017f;
		constexpr float SQRT_15 = 3.87298334621f;
		constexpr float SQRT_15_DIV_2 = SQRT_15 / 2.0f;
		constexpr float SQRT_21_DIV_2 = 3.24037034920f;
		constexpr float SQRT_35_DIV_2 = 4.18330013267f;
		constexpr float SQRT_105 = 10.24695076596f;
		constexpr float SQRT_35 = 5.9160797831f;
		constexpr float SQRT_70 = 8.3666002653f;
		constexpr float SQRT_5_DIV_4 = 1.1180339887f;      
		constexpr float SQRT_70_DIV_4 = 2.0916500663f;     
		constexpr float SQRT_105_DIV_4 = 2.5311320584f;    
		constexpr float SQRT_5_DIV_16 = 0.5590169944f;     
		constexpr float SQRT_70_DIV_8 = 1.0458250331f;     
		constexpr float SQRT_35_DIV_8 = 0.7395099729f;     
		constexpr float SQRT_14 = 3.74165738677f;
		constexpr float SQRT_13 = 3.60555127546f;
		constexpr float SQRT_91 = 9.53939201417f;
		constexpr float SQRT_210 = 14.4913767462f;
		constexpr float SQRT_273 = 16.5227116410f;
		constexpr float SQRT_429 = 20.7082789866f;
		constexpr float SQRT_858 = 29.2966064066f;
		constexpr float SQRT_13_DIV_32 = SQRT_13 / 32.f;
		constexpr float SQRT_91_DIV_8 = SQRT_91 / 8.f;
		constexpr float SQRT_273_DIV_16 = SQRT_273 / 16.f;
		constexpr float SQRT_429_DIV_8 = SQRT_429 / 8.f;
		constexpr float SQRT_14_DIV_16 = SQRT_14 * 3.0f / 16.0f;      
		constexpr float SQRT_70_DIV_32 = SQRT_70 / 32.0f;
		constexpr float SQRT_15_DIV_8 = SQRT_15 / 8.0f;               

		// Cache trig functions.
		struct FTrigCache
		{
			// We only support up to 7th order for now.
			static constexpr int32 MaxOrder = 7;
		
			float SinElevation = 0.f;
			float CosElevation = 0.f;
			float SinAzimuthOrder[MaxOrder+1];
			float CosAzimuthOrder[MaxOrder+1];
        
			FTrigCache(const float Azimuth, const float Elevation, const int32 Order)
				: SinElevation(FMath::Sin(Elevation))
				, CosElevation(FMath::Cos(Elevation))
			{
				check(Order >= 0 && Order <= MaxOrder);

				// Cache sin/cos * order
				SinAzimuthOrder[0] = 0.f;  
				CosAzimuthOrder[0] = 1.f;  
				for (int32 i = 1; i <= Order; ++i)
				{
					SinAzimuthOrder[i] = FMath::Sin(i * Azimuth);
					CosAzimuthOrder[i] = FMath::Cos(i * Azimuth);
				}
			}
		};

		// Order 0
		static float SHBasisFunction_ACN_0(const FTrigCache& /*Trig*/)
		{
			return 1.0f; // Omnidirectional
		}

		static float SHBasisFunction_ACN_1(const FTrigCache& T)
		{
			return -SQRT_3 * T.CosElevation * T.SinAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_2(const FTrigCache& T)
		{
			return SQRT_3 * T.SinElevation;
		}

		static float SHBasisFunction_ACN_3(const FTrigCache& T)
		{
			return -SQRT_3 * T.CosElevation * T.CosAzimuthOrder[1];
		}

		//------------------------------------------------------------------------------
		// Order 2 
		//------------------------------------------------------------------------------
  
		static float SHBasisFunction_ACN_4(const FTrigCache& T)
		{
			return 0.645497f * T.CosElevation * T.SinAzimuthOrder[2];
		}
	
		static float SHBasisFunction_ACN_5(const FTrigCache& T)
		{
			const float S= T.SinElevation;
			const float P21 = 3.f * S* T.CosElevation;
			constexpr float N2_1 = 1.290994f; 
			return -N2_1 * P21 * T.SinAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_6(const FTrigCache& T)
		{
			const float S= T.SinElevation;
			const float P20 = 0.5f * (3.f * S*S - 1.f);
			return 2.236068f * P20;
		}

		static float SHBasisFunction_ACN_7(const FTrigCache& T)
		{
			const float S= T.SinElevation;
			const float P21 = 3.f * S* T.CosElevation;
			constexpr float N2_1 = 1.290994f;
			return -N2_1 * P21 * T.CosAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_8(const FTrigCache& T)
		{
			const float C = T.CosElevation;
			const float P22 = 3.f * C * C;
			constexpr float N2_2 = 0.645497f;
			return N2_2 * P22 * T.CosAzimuthOrder[2];
		}

		//------------------------------------------------------------------------------
		// Order 3 
		//------------------------------------------------------------------------------

		static float SHBasisFunction_ACN_9(const FTrigCache& T)
		{
			const float C = T.CosElevation;
			const float P33 = C * C * C;        
			constexpr float N3_3 = 2.092034f;  
			return -N3_3 * P33 * T.SinAzimuthOrder[3];
		}

		static float SHBasisFunction_ACN_10(const FTrigCache& T)
		{
			constexpr float N3D_3_2 = 0.0963537f;		
			return  N3D_3_2 * T.CosElevation * T.SinAzimuthOrder[2];
		}

		static float SHBasisFunction_ACN_11(const FTrigCache& T)
		{
			const float S= T.SinElevation;
			const float P31 = 3.f * S* T.CosElevation;
			constexpr float N3_1 = 1.145644f;
			return -N3_1 * P31 * T.SinAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_12(const FTrigCache& T)
		{
			const float S= T.SinElevation;
			const float P30 = 0.5f * (5.f * S*S*S - 3.f * S);
			return 2.645751f * P30;
		}

		static float SHBasisFunction_ACN_13(const FTrigCache& T)
		{
			const float S= T.SinElevation;
			const float P31 = 3.f * S* T.CosElevation;  
			constexpr float N3_1 = 1.145644f;  
			return -N3_1 * P31 * T.CosAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_14(const FTrigCache& T)
		{
			const float C = T.CosElevation;
			const float P32 = 3.f * C * C;
			constexpr float N3_2 = 1.207615f;
			return N3_2 * P32 * T.CosAzimuthOrder[2];
		}

		static float SHBasisFunction_ACN_15(const FTrigCache& T)
		{
			const float C = T.CosElevation;
			const float P33 = C * C * C;
			constexpr float N3_3 = 2.092034f;
			return -N3_3 * P33 * T.CosAzimuthOrder[3];
		}

		//------------------------------------------------------------------------------
		// Order 4
		//------------------------------------------------------------------------------

		static float SHBasisFunction_ACN_16(const FTrigCache& T)
		{
			constexpr float N3D_4_4 = 0.0059603f;
			return N3D_4_4 * T.CosElevation * T.SinAzimuthOrder[4];
		}

		static float SHBasisFunction_ACN_17(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float P43 = 15.f * S* C * C * C;
			constexpr float N4_3 = 0.4183301f;
			return -N4_3 * P43 * T.SinAzimuthOrder[3];
		}

		static float SHBasisFunction_ACN_18(const FTrigCache& T)
		{
			constexpr float N3D_4_2 = 0.0630783f;		
			return N3D_4_2 * T.CosElevation * T.SinAzimuthOrder[2];
		}

		static float SHBasisFunction_ACN_19(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float P41 = 2.5f * S* (7.f * S*S - 3.f) * C;
			constexpr float N4_1 = 0.9486833f;  
			return -N4_1 * P41 * T.SinAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_20(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float s2  = S*S;
			const float P40 = 0.125f * (35.f * s2*s2 - 30.f * s2 + 3.f);
			return 3.0f * P40;
		}

		static float SHBasisFunction_ACN_21(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float P41  = 2.5f * S* (7.f * S*S - 3.f) * C;
			constexpr float N4_1 = 0.9486833f;
			return -N4_1 * P41 * T.CosAzimuthOrder[1];
		}
	
		static float SHBasisFunction_ACN_22(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float S2  = S*S;
			const float P42 = -0.5f * (105.f * S2*S2 - 120.f * S2 + 15.f);
			constexpr float N4_2 = 0.2236068f;
			return  N4_2 * P42 * T.CosAzimuthOrder[2];
		}

		static float SHBasisFunction_ACN_23(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float P43 = 15.f * S * C * C * C;
			constexpr float N4_3 = 0.4183301f;
			return -N4_3 * P43 * T.CosAzimuthOrder[3];
		}

		static float SHBasisFunction_ACN_24(const FTrigCache& T)
		{
			const float C = T.CosElevation;
			const float P44 = C * C * C * C;
			constexpr float N4_4 = 2.218528f;
			return N4_4 * P44 * T.CosAzimuthOrder[4];
		}

		static float SHBasisFunction_ACN_25(const FTrigCache& T)
		{
			const float C = T.CosElevation;
			const float P55  = -945.f * C*C*C*C*C;
			constexpr float N5_5 = 0.0024619f;
			return  N5_5 * P55 * T.SinAzimuthOrder[5];
		}

		static float SHBasisFunction_ACN_26(const FTrigCache& T)
		{
			constexpr float N3D_5_4 = 0.0012290f;
			return N3D_5_4 * T.CosElevation * T.SinAzimuthOrder[4];
		}

		static float SHBasisFunction_ACN_27(const FTrigCache& T)
		{
			const float S  = T.SinElevation;
			const float x2  = S*S;
			const float P53 = -FMath::Pow(1.f - x2, 1.5f) * (3780.f * x2 - 420.f) * 0.125f;
			constexpr float N5_3 = 0.03303437f;
			return N5_3 * P53 * T.SinAzimuthOrder[3];
		}

		static float SHBasisFunction_ACN_28(const FTrigCache& T)
		{
			constexpr float N3D_5_2 = 0.0413065f;		
			return N3D_5_2 * T.CosElevation * T.SinAzimuthOrder[2];
		}

		static float SHBasisFunction_ACN_29(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float x2 = S*S;
			const float P51 = -FMath::Sqrt(1.f - x2) * ((315.f * x2*x2 - 210.f * x2 + 15.f) * 0.125f);
			constexpr float N5_1 = 0.85634918f;
			return N5_1 * P51 * T.SinAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_30(const FTrigCache& T)
		{
			const float S  = T.SinElevation;
			const float P50 = 0.125f * (63.f * S*S*S*S*S - 70.f * S*S*S + 15.f * S);
			constexpr float N5_0 = 3.31662479f;
			return N5_0 * P50;
		}

		static float SHBasisFunction_ACN_31(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float x2 = S*S;
			const float P51  = -FMath::Sqrt(1.f - x2) * ((315.f * x2*x2 - 210.f*x2 + 15.f) * 0.125f);
			constexpr float N5_1 = 0.85634918f;
			return N5_1 * P51 * T.CosAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_32(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float x2 = S*S;
			const float P52 = 52.5f * S* (3.f * x2 - 1.f) * (1.f - x2);
			constexpr float N5_2 = 0.161835f;
			return N5_2 * P52 * T.CosAzimuthOrder[2];
		}	
	
		static float SHBasisFunction_ACN_33(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float x2 = S*S;
			const float P53 = -0.125f * (3780.f * x2 - 420.f) * (C * C * C);
			constexpr float N5_3 = 0.03303437f;
			return N5_3 * P53 * T.CosAzimuthOrder[3];;
		}

		static float SHBasisFunction_ACN_34(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float S2 = S*S;
			const float P54 = 945.f * S * (1.f - S2) * (1.f - S2);
			constexpr float N5_4 = 0.007786274f;
			return N5_4 * P54 * T.CosAzimuthOrder[4];
		}

		static float SHBasisFunction_ACN_35(const FTrigCache& T)
		{
			const float C = T.CosElevation;
			const float P55 = -945.f * C * C * C * C * C;
			constexpr float N5_5 = 0.00246216f;
			return N5_5 * P55 * T.CosAzimuthOrder[5];
		}

		//------------------------------------------------------------------------------
		// Order 6 
		//------------------------------------------------------------------------------

		static float SHBasisFunction_ACN_36(const FTrigCache& T)
		{
			constexpr float N3D_6_6 = 0.0000657f;
			return N3D_6_6 * T.CosElevation * T.SinAzimuthOrder[6];
		}
	
		static float SHBasisFunction_ACN_37(const FTrigCache& T)
		{
			const float S  = T.SinElevation;
			const float C   = T.CosElevation;
			const float P65 = -10395.f * S * C * C * C * C * C;
			constexpr float N6_5 = 0.00080696f;
			return N6_5 * P65 * T.SinAzimuthOrder[5];
		}
	
		static float SHBasisFunction_ACN_38(const FTrigCache& T)
		{
			constexpr float N3D_6_4 = 0.0010679f;
			return N3D_6_4 * T.CosElevation * T.SinAzimuthOrder[4];
		}

		static float SHBasisFunction_ACN_39(const FTrigCache& T)
		{
			const float S  = T.SinElevation;
			const float S2  = S*S;
			const float OneMinus_s2 = 1.f - S2;
			const float P63 = ((-1732.5f * S*S*S) + (472.5F * S)) * OneMinus_s2 * T.CosElevation;
			constexpr float N6_3 = 0.0207339f;
			return N6_3 * P63 * T.SinAzimuthOrder[3];
		}

		static float SHBasisFunction_ACN_40(const FTrigCache& T)
		{
			constexpr float N3D_6_2 = 0.0350935f;
			return N3D_6_2 * T.CosElevation * T.SinAzimuthOrder[2];
		}

		static float SHBasisFunction_ACN_41(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float X2  = S*S;
			const float P61 = -0.0625f * (1386.f * S*X2*X2 -1260.f * S*X2 + 210.f * S) * T.CosElevation;
			constexpr float N6_1 = 0.786800f;
			return N6_1 * P61 * T.SinAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_42(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float S2 = S*S;
			const float S4 = S2*S2;
			const float S6 = S4*S2;
			const float P60 = 0.0625F * (231.F * S6 - 315.F * S4 + 105.F * S2 - 5.F);
			constexpr float N6_0 = 3.605551f;
			return N6_0 * P60;
		}

		static float SHBasisFunction_ACN_43(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float X2 = S*S;
			const float P61 = -0.0625f * (1386.f * S*X2*X2 -1260.f * S*X2 + 210.f * S) * T.CosElevation;
			constexpr float N6_1 = 0.786800f;
			return N6_1 * P61 * T.CosAzimuthOrder[1];
		}
	
		static float SHBasisFunction_ACN_44(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float S2 = S * S;
			const float S4 = S2 * S2;
			const float S6 = S4 * S2;
			const float P62 = (-3465.f * S6 + 5355.f * S4 - 1995.f * S2 +  105.f) * 0.125f;
			constexpr float N6_2 = 0.1244032f;
			return N6_2 * P62 * T.CosAzimuthOrder[2];
		}

		static float SHBasisFunction_ACN_45(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float S2 = S * S;
			const float OneMinusS2 = 1.f - S2;
			const float P63 = ((-1732.5f * S* S * S) + (472.5f * S)) * OneMinusS2 * T.CosElevation;
			constexpr float N6_3 = 0.0207339f;
			return N6_3 * P63 * T.CosAzimuthOrder[3];;
		}

		static float SHBasisFunction_ACN_46(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float S2 = S * S;
			const float OneMinusS2 = 1.f - S2;
			const float P64 = 0.5f * (10395.f * S2 - 945.f) * OneMinusS2 * OneMinusS2;
			constexpr float N6_4 = 0.003785f;
			return N6_4 * P64 * T.CosAzimuthOrder[4];
		}
	
		static float SHBasisFunction_ACN_47(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float P65 = -10395.f * S * C * C * C * C * C;
			constexpr float N6_5 = 0.0008070657f;  
			return N6_5 * P65 * T.CosAzimuthOrder[5];
		}

		static float SHBasisFunction_ACN_48(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float P66  = 10395.f * FMath::Pow(1.f - S*S, 3);
			constexpr float N6_6 = 0.00023317f;
			return N6_6 * P66 * T.CosAzimuthOrder[6];
		}

		//------------------------------------------------------------------------------
		// Order 7 
		//----------------------------------------------------------------------------//
	
		static float SHBasisFunction_ACN_49(const FTrigCache& T)
		{
			const float C = T.CosElevation;
			const float P77 = 135135.f * C * C * C * C * C * C * C;
			constexpr float N7_7 = 1.85505e-05f;
			return -N7_7 * P77 * T.SinAzimuthOrder[7];
		}
	
		static float SHBasisFunction_ACN_50(const FTrigCache& T)
		{
			constexpr float N3D_7_6 = 0.0000196f;
			return N3D_7_6 * T.CosElevation * T.SinAzimuthOrder[6];
		}

		static float SHBasisFunction_ACN_51(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float P75  = -((67567.5f * S*S - 5197.5f) * C * C * C * C * C);
			constexpr float N7_5 = 0.0003539220f;  
			return N7_5 * P75 * T.SinAzimuthOrder[5];
		}

		static float SHBasisFunction_ACN_52(const FTrigCache& T)
		{
			constexpr float N3D_7_4 = 0.0005990f;
			return N3D_7_4 * T.CosElevation * T.SinAzimuthOrder[4];
		}

		static float SHBasisFunction_ACN_53(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float X2 = S * S;
			const float P7 = (90090.f * X2 * X2 - 41580.f * X2 +  1890.f) * 0.0625f;
			const float P73  = C * C * C * P7;
			constexpr float N7_3 = 0.0140849f;
			return -N7_3 * P73 * T.SinAzimuthOrder[3];;
		}

		static float SHBasisFunction_ACN_54(const FTrigCache& T)
		{
			constexpr float N3D_7_2 = 0.0280973f;
			return N3D_7_2 * T.CosElevation * T.SinAzimuthOrder[2];
		}

		static constexpr float N7_1_CAL = 0.731924f;  

		static float SHBasisFunction_ACN_55(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float X2 = S*S;
			const float P7d = (3003.f * X2*X2*X2 -3465.f * X2*X2 + 945.f * X2 - 35.f) * 0.0625f;
			const float P71 = -C * P7d;
			return N7_1_CAL * P71 * T.SinAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_56(const FTrigCache& T)
		{
			const float S  = T.SinElevation;
			const float S2  = S * S;
			const float S3  = S2 * S;
			const float S5  = S3 * S2;
			const float S7  = S5 * S2;
			const float P70 = 0.0625f * (429.f * S7 - 693.f * S5 + 315.f * S3 - 35.f * S);
			constexpr float N7_0 = 3.872983f;
			return N7_0 * P70;
		}

		static float SHBasisFunction_ACN_57(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float X2 = S*S;
			const float P7d = (3003.f * X2*X2*X2 -3465.f * X2*X2 + 945.f * X2 - 35.f) * 0.0625f;
			const float P71 = -C * P7d;
			return N7_1_CAL * P71 * T.CosAzimuthOrder[1];
		}

		static float SHBasisFunction_ACN_58(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float X2 = S*S;
			const float P72 = 0.125f * (945.f * S*X2*X2 -1575.f * S*X2 +  525.f * S) * (1.f - X2);
			constexpr float N7_2 = 1.0159443f;  
			return N7_2 * P72 * T.CosAzimuthOrder[2];
		}

		static float SHBasisFunction_ACN_59(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float X2  = S*S;
			const float P7 = (90090.f * X2 * X2 - 41580.f * X2 +  1890.f) * 0.0625f;
			const float P73 = C * C * C * P7;
			constexpr float N7_3 = 1.127530f / 80.05332f;  
			return -N7_3 * P73 * T.CosAzimuthOrder[3];
		}

		static float SHBasisFunction_ACN_60(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float X2 = S * S;
			const float OneMinusX2 = 1.f - X2;
			const float A = (360360.f * S * X2 - 83160.f * S) * 0.0625f;
			const float P74  = OneMinusX2 * OneMinusX2 * A;
			constexpr float N7_4 = 0.00212334f;  
			return N7_4 * P74 * T.CosAzimuthOrder[4];
		}

		static float SHBasisFunction_ACN_61(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float C = T.CosElevation;
			const float P75 = 5197.5f * (1.f - 13.f * S * S) * C * C * C * C * C;
			constexpr float N7_5 = 0.0003539217f;  
			return N7_5 * P75 * T.CosAzimuthOrder[5];
		}

		static float SHBasisFunction_ACN_62(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float OneMinusS2 = 1.f - S*S;
			const float P76 = 135135.f * S * OneMinusS2 * OneMinusS2 * OneMinusS2;
			constexpr float N7_6 = 0.829055f / 11944.359359455508f;  
			return N7_6 * P76 * T.CosAzimuthOrder[6];
		}

		static float SHBasisFunction_ACN_63(const FTrigCache& T)
		{
			const float S = T.SinElevation;
			const float OneMinusS2 = 1.f - S*S;
			const float P77 = -135135.f  * OneMinusS2 * OneMinusS2 * OneMinusS2  * FMath::Sqrt(OneMinusS2);
			constexpr float N7_7 = 0.221574f / 11944.359359455508f;
			return N7_7 * P77 * T.CosAzimuthOrder[7];
		}
	}
}

void FSphericalHarmonicCalculator::ComputeSoundfieldChannelGains(const int32 Order, const float Azimuth, const float Elevation, float* OutGains)
{
	ComputeSoundfieldChannelGains(Order, Azimuth, Elevation, MakeArrayView(OutGains, OrderToNumChannels(Order)));
}

void FSphericalHarmonicCalculator::ComputeSoundfieldChannelGains(
	const int32 Order, const float Azimuth, const float Elevation, TArrayView<float> OutGains)
{
	using namespace Audio::SphericalHarmonicsPrivate;
	check(Order >= 0 && Order <= 7);
	check(OutGains.Num() >= OrderToNumChannels(Order));

	using namespace Audio;
	const FTrigCache TrigCache(Azimuth, Elevation, Order);

	switch (Order)
	{
	case 7:
		OutGains[63] = SHBasisFunction_ACN_63(TrigCache);
		OutGains[62] = SHBasisFunction_ACN_62(TrigCache);
		OutGains[61] = SHBasisFunction_ACN_61(TrigCache);
		OutGains[60] = SHBasisFunction_ACN_60(TrigCache);
		OutGains[59] = SHBasisFunction_ACN_59(TrigCache);
		OutGains[58] = SHBasisFunction_ACN_58(TrigCache);
		OutGains[57] = SHBasisFunction_ACN_57(TrigCache);
		OutGains[56] = SHBasisFunction_ACN_56(TrigCache);
		OutGains[55] = SHBasisFunction_ACN_55(TrigCache);
		OutGains[54] = SHBasisFunction_ACN_54(TrigCache);
		OutGains[53] = SHBasisFunction_ACN_53(TrigCache);
		OutGains[52] = SHBasisFunction_ACN_52(TrigCache);
		OutGains[51] = SHBasisFunction_ACN_51(TrigCache);
		OutGains[50] = SHBasisFunction_ACN_50(TrigCache);
		OutGains[49] = SHBasisFunction_ACN_49(TrigCache);
		[[fallthrough]];
	case 6:
		OutGains[48] = SHBasisFunction_ACN_48(TrigCache);
		OutGains[47] = SHBasisFunction_ACN_47(TrigCache);
		OutGains[46] = SHBasisFunction_ACN_46(TrigCache);
		OutGains[45] = SHBasisFunction_ACN_45(TrigCache);
		OutGains[44] = SHBasisFunction_ACN_44(TrigCache);
		OutGains[43] = SHBasisFunction_ACN_43(TrigCache);
		OutGains[42] = SHBasisFunction_ACN_42(TrigCache);
		OutGains[41] = SHBasisFunction_ACN_41(TrigCache);
		OutGains[40] = SHBasisFunction_ACN_40(TrigCache);
		OutGains[39] = SHBasisFunction_ACN_39(TrigCache);
		OutGains[38] = SHBasisFunction_ACN_38(TrigCache);
		OutGains[37] = SHBasisFunction_ACN_37(TrigCache);
		OutGains[36] = SHBasisFunction_ACN_36(TrigCache);
		[[fallthrough]];
	case 5:
		OutGains[35] = SHBasisFunction_ACN_35(TrigCache);
		OutGains[34] = SHBasisFunction_ACN_34(TrigCache);
		OutGains[33] = SHBasisFunction_ACN_33(TrigCache);
		OutGains[32] = SHBasisFunction_ACN_32(TrigCache);
		OutGains[31] = SHBasisFunction_ACN_31(TrigCache);
		OutGains[30] = SHBasisFunction_ACN_30(TrigCache);
		OutGains[29] = SHBasisFunction_ACN_29(TrigCache);
		OutGains[28] = SHBasisFunction_ACN_28(TrigCache);
		OutGains[27] = SHBasisFunction_ACN_27(TrigCache);
		OutGains[26] = SHBasisFunction_ACN_26(TrigCache);
		OutGains[25] = SHBasisFunction_ACN_25(TrigCache);
		[[fallthrough]];
	case 4:
		OutGains[24] = SHBasisFunction_ACN_24(TrigCache);
		OutGains[23] = SHBasisFunction_ACN_23(TrigCache);
		OutGains[22] = SHBasisFunction_ACN_22(TrigCache);
		OutGains[21] = SHBasisFunction_ACN_21(TrigCache);
		OutGains[20] = SHBasisFunction_ACN_20(TrigCache);
		OutGains[19] = SHBasisFunction_ACN_19(TrigCache);
		OutGains[18] = SHBasisFunction_ACN_18(TrigCache);
		OutGains[17] = SHBasisFunction_ACN_17(TrigCache);
		OutGains[16] = SHBasisFunction_ACN_16(TrigCache);
		[[fallthrough]];
	case 3:
		OutGains[15] = SHBasisFunction_ACN_15(TrigCache);
		OutGains[14] = SHBasisFunction_ACN_14(TrigCache);
		OutGains[13] = SHBasisFunction_ACN_13(TrigCache);
		OutGains[12] = SHBasisFunction_ACN_12(TrigCache);
		OutGains[11] = SHBasisFunction_ACN_11(TrigCache);
		OutGains[10] = SHBasisFunction_ACN_10(TrigCache);
		OutGains[9]  = SHBasisFunction_ACN_9(TrigCache);
		[[fallthrough]];
	case 2:
		OutGains[8]  = SHBasisFunction_ACN_8(TrigCache);
		OutGains[7]  = SHBasisFunction_ACN_7(TrigCache);
		OutGains[6]  = SHBasisFunction_ACN_6(TrigCache);
		OutGains[5]  = SHBasisFunction_ACN_5(TrigCache);
		OutGains[4]  = SHBasisFunction_ACN_4(TrigCache);
		[[fallthrough]];
	case 1:
		OutGains[3]  = SHBasisFunction_ACN_3(TrigCache);
		OutGains[2]  = SHBasisFunction_ACN_2(TrigCache);
		OutGains[1]  = SHBasisFunction_ACN_1(TrigCache);
		[[fallthrough]];
	case 0:
		OutGains[0]  = SHBasisFunction_ACN_0(TrigCache);
		break;
	default:
		checkNoEntry();
	}
}


void FSphericalHarmonicCalculator::GenerateFirstOrderRotationMatrixGivenRadians(const float RotXRadians, const float RotYRadians, const float RotZRadians, FMatrix & OutMatrix)
{
	const float SinX = FMath::Sin(RotXRadians);
	const float SinY = FMath::Sin(RotYRadians);
	const float SinZ = FMath::Sin(RotZRadians);

	const float CosX = FMath::Cos(RotXRadians);
	const float CosY = FMath::Cos(RotYRadians);
	const float CosZ = FMath::Cos(RotZRadians);


	// Build out Rotation Matrix
	OutMatrix.SetIdentity();

	// row 0:
	/*	1.0f						0.0f															0.0f									0.0f									*/

	// row 1:
	/*	0.0f	*/	OutMatrix.M[1][1] = (CosX * CosZ) + (SinX * SinY * SinZ);		OutMatrix.M[1][2] = -SinX * CosY;		OutMatrix.M[1][3] = (CosX * SinZ) - (SinX * SinY * CosZ);

	// row 2:
	/*	0.0f	*/	OutMatrix.M[2][1] = (SinX * CosZ) - (CosX * SinY * SinZ);		OutMatrix.M[2][2] = CosX * CosY;		OutMatrix.M[2][3] = (CosX * SinY * CosZ) + (SinX * SinZ);

	// row 3:
	/*	0.0f	*/	OutMatrix.M[3][1] = -CosY * SinZ;								OutMatrix.M[3][2] = -SinY;				OutMatrix.M[3][3] = CosY * CosZ;
}

void FSphericalHarmonicCalculator::GenerateFirstOrderRotationMatrixGivenDegrees(const float RotXDegrees, const float RotYDegrees, const float RotZDegrees, FMatrix & OutMatrix)
{
	constexpr const float DEG_2_RAD = PI / 180.0f;
	GenerateFirstOrderRotationMatrixGivenRadians(RotXDegrees * DEG_2_RAD, RotYDegrees * DEG_2_RAD, RotZDegrees * DEG_2_RAD, OutMatrix);
}

bool FSphericalHarmonicCalculator::IndexToOrderAndDegree(const int32 InChannelIndex, int32& OutOrder, int32& OutDegree)
{
	if (InChannelIndex >= 0)
	{
		OutOrder = FMath::FloorToInt(FMath::Sqrt(static_cast<float>(InChannelIndex)));
		for (; OutOrder * (OutOrder + 1) > InChannelIndex; --OutOrder)
		{
		}
		OutDegree = InChannelIndex - OutOrder * (OutOrder + 1) - OutOrder;
		return true;
	}
	return false;
}

void FSphericalHarmonicCalculator::NormalizeGains(TArrayView<float> Gains)
{
	// Normalize. gains.
	float SumSqr = 0.f;
	for (int32 i = 0; i < Gains.Num(); ++i)
	{
		SumSqr+=Gains[i]*Gains[i];
	}
	if (!FMath::IsNearlyZero(SumSqr))
	{
		const float InverseLength = 1.0f / FMath::Sqrt(SumSqr);
		for (int32 i = 0; i < Gains.Num(); ++i)
		{
			Gains[i]*=InverseLength;
		}
	}
}
