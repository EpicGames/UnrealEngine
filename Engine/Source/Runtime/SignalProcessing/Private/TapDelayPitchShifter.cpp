// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/TapDelayPitchShifter.h"

namespace Audio
{
	namespace PitchShiftUtils
	{
		static float GetPitchShiftClamped(const float InPitchShift)
		{
			return FMath::Clamp(InPitchShift, -12.0f * FTapDelayPitchShifter::MaxAbsPitchShiftInOctaves, 12.0f * FTapDelayPitchShifter::MaxAbsPitchShiftInOctaves);
		}

		static float GetDelayLengthClamped(const float InBufferLength)
		{
			return FMath::Clamp(InBufferLength, FTapDelayPitchShifter::MinDelayLength, FTapDelayPitchShifter::MaxDelayLength);	
		}
	}

	FTapDelayPitchShifter::FTapDelayPitchShifter()
	{
	}

	FTapDelayPitchShifter::~FTapDelayPitchShifter()
	{
	}

	void FTapDelayPitchShifter::Init(const float InSampleRate, const float InPitchShift, const float InDelayLength)
	{
		SampleRate = InSampleRate;
		CurrentTargetDelayLength = PitchShiftUtils::GetDelayLengthClamped(InDelayLength);
		CurrentDelayLength.Init(CurrentTargetDelayLength);
		CurrentPitchShift = PitchShiftUtils::GetPitchShiftClamped(InPitchShift);
		CurrentPitchShiftRatio = Audio::GetFrequencyMultiplier(CurrentPitchShift);

		UpdatePhasorPhaseIncrement();
	}
	
	void FTapDelayPitchShifter::UpdatePhasorPhaseIncrement()
	{
		const float CurrentLengthSecondsClamped = 0.001f * FMath::Max(CurrentDelayLength.PeekCurrentValue(), 1.0f);
		const float PhasorFrequency =  (1.0f - CurrentPitchShiftRatio) / CurrentLengthSecondsClamped;
		PhasorPhaseIncrement =  PhasorFrequency / SampleRate;
	}
	
 
	void FTapDelayPitchShifter::SetDelayLength(const float InDelayLength)
	{
		const float NewDelayLength = PitchShiftUtils::GetDelayLengthClamped(InDelayLength);
		if (!FMath::IsNearlyEqual(NewDelayLength, CurrentTargetDelayLength))
		{
			CurrentTargetDelayLength = InDelayLength;
			CurrentDelayLength.SetValue(NewDelayLength);
			UpdatePhasorPhaseIncrement();
		}
	}

	void FTapDelayPitchShifter::SetPitchShift(const float InPitchShift)
	{
		const float NewPitchShift = PitchShiftUtils::GetPitchShiftClamped(InPitchShift);
		if (!FMath::IsNearlyEqual(NewPitchShift, CurrentPitchShift))
		{
			CurrentPitchShift = NewPitchShift;
			CurrentPitchShiftRatio = Audio::GetFrequencyMultiplier(CurrentPitchShift);
			UpdatePhasorPhaseIncrement();
		}
	}

	void FTapDelayPitchShifter::SetPitchShiftRatio(const float InPitchShiftRatio)
	{
		if (!FMath::IsNearlyEqual(InPitchShiftRatio, CurrentPitchShiftRatio))
		{
			CurrentPitchShiftRatio = InPitchShiftRatio;
			UpdatePhasorPhaseIncrement();
		}
	}

	float FTapDelayPitchShifter::ReadDopplerShiftedTapFromDelay(const Audio::FDelay& InDelayBuffer, const float InReadOffsetMilliseconds)
	{
		// Update the interpolated delay length value
		if (!CurrentDelayLength.IsDone())
		{
			UpdatePhasorPhaseIncrement();
			CurrentDelayLength.GetNextValue();
		}
		
		// Compute the two tap delay read locations, one shifted 90 degrees out of phase
		const float PhasorPhaseOffset = FMath::Fmod(PhasorPhase + 0.5f, 1.0f);
		const float DelayTapRead1 = InReadOffsetMilliseconds + CurrentDelayLength.PeekCurrentValue() * PhasorPhase;
		const float DelayTapRead2 = InReadOffsetMilliseconds + CurrentDelayLength.PeekCurrentValue() * PhasorPhaseOffset;

		// This produces an overlapping cosine function that avoids pops in the output
		const float DelayTapGain1 = FMath::Cos(PI * (PhasorPhase - 0.5f));
		const float DelayTapGain2 = FMath::Cos(PI * (PhasorPhaseOffset - 0.5f));

		// Update the phasor state
		PhasorPhase += PhasorPhaseIncrement;
		// Make sure we wrap to between 0.0 and 1.0 after incrementing the phase 
		PhasorPhase = FMath::Wrap(PhasorPhase, 0.0f, 1.0f);
		
		// Read the delay lines at the given tap indices, apply the gains
		const float Sample1 = DelayTapGain1 * InDelayBuffer.ReadDelayAt(DelayTapRead1);
		const float Sample2 = DelayTapGain2 * InDelayBuffer.ReadDelayAt(DelayTapRead2);

		return Sample1 + Sample2;
	}

	void FTapDelayPitchShifter::ProcessAudio(Audio::FDelay& InDelayBuffer, const float* InAudioBuffer, const int32 InNumFrames, float* OutAudioBuffer)
	{
		for (int32 FrameIndex = 0; FrameIndex < InNumFrames; ++FrameIndex)
		{
			OutAudioBuffer[FrameIndex] = ReadDopplerShiftedTapFromDelay(InDelayBuffer);
			InDelayBuffer.WriteDelayAndInc(InAudioBuffer[FrameIndex]);
		}
	}

}
