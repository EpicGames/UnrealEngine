// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniformPartitionConvolution.h"
#include "CoreMinimal.h"
#include "SignalProcessingModule.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/ConvolutionAlgorithm.h"
#include "DSP/FFTAlgorithm.h"
#include "DSP/FloatArrayMath.h"

///////////////////////////////////////////////////////////////////////////////
// Note on OVERLAP_SAVE
//
// This algorithm is implemented using the overlap save method of convolution.
//
// The block size is set to FFT->Size() / 2 samples.
//
// Inputs FFTs are performed on the input samples using a rectaangular sliding 
// window of 2 * BlockSize samples with a hop of BlockSize samples.
//
// Impulse response FFTs 2 * BlockSize with BlockSize zeropadded samples. 
//
// Output inverse FFTs produce 2 * BlockSize samples, but the first BlockSize
// samples are discarded because they are time aliased. 
///////////////////////////////////////////////////////////////////////////////

namespace Audio
{
	FUniformPartitionConvolution::FUniformPartitionConvolution(const FUniformPartitionConvolutionSettings& InSettings, FSharedFFTRef InFFTAlgorithm)
	:	Settings(InSettings)
	,	BlockSize(0)
	,	NumFFTOutputFloats(0)
	,	NumBlocks(0)
	,	FFTAlgorithm(InFFTAlgorithm)
	{
		// Input to FFT has to be zero padded in order to prevent aliasing from circular 
		// convolution. Hence, block size is half the fft size. 
		BlockSize = FFTAlgorithm->Size() / 2;

		check(BlockSize > 0);

		// Number of floats in fft output. 
		NumFFTOutputFloats = FFTAlgorithm->NumOutputFloats();

		TransformedAndScaledInput.AddUninitialized(NumFFTOutputFloats);

		NumBlocks = Settings.MaxNumImpulseResponseSamples / BlockSize;
		// Add extra block if needed. Accounting for integer division rounding down.
		NumBlocks += Settings.MaxNumImpulseResponseSamples % BlockSize == 0 ? 0 : 1;

		// Create inputs 
		for (int32 i = 0; i < Settings.NumInputChannels; i++)
		{
			Inputs.Emplace(FFTAlgorithm);
		}

		// Create transforms
		for (int32 i = 0; i < Settings.NumImpulseResponses; i++)
		{
			ImpulseResponses.Emplace(NumBlocks, FFTAlgorithm);
		}

		// Create outputs 
		for (int32 i = 0; i < Settings.NumOutputChannels; i++)
		{
			Outputs.Emplace(NumBlocks, FFTAlgorithm);
		}
		IsOutputZero.Init(true, Settings.NumOutputChannels);
	}

	FUniformPartitionConvolution::~FUniformPartitionConvolution()
	{
	}

	/** Returns the number of samples in an audio block. */
	int32 FUniformPartitionConvolution::GetNumSamplesInBlock() const
	{
		return BlockSize;
	}

	/** Returns number of audio inputs. */
	int32 FUniformPartitionConvolution::GetNumAudioInputs() const
	{
		return Settings.NumInputChannels;
	}

	/** Returns number of audio outputs. */
	int32 FUniformPartitionConvolution::GetNumAudioOutputs() const
	{
		return Settings.NumOutputChannels;
	}

	/** Process one block of audio.
	*
	* InSamples is processed by the impulse responses. The output is placed in OutSamples.
	*
	* @params InSamples - A 2D array of input deinterleaved audio samples. InSamples[GetNumAudioInputs()][GetNumSamplesInBlock()]
	* @params OutSamples - A 2D array of output deinterleaved audio samples. OutSamples[GetNumAudioOutputs()][GetNumSamplesInBlock()]
	*
	*/
	void FUniformPartitionConvolution::ProcessAudioBlock(const float* const InSamples[], float* const OutSamples[])
	{
		
		// Add input audio to input channels
		for (int32 i = 0; i < Settings.NumInputChannels; i++)
		{
			Inputs[i].PushBlock(InSamples[i]);
		}
		
		// Apply filtering for gains above zero
		for (auto GainMatrixEntry : NonZeroGains)
		{
			// Get input, transform and output.
			const FInputTransformOutputGroup& Group = GainMatrixEntry.Key;

			const FInput& Input = Inputs[Group.GetInputIndex()];
			const FImpulseResponse& Transform = ImpulseResponses[Group.GetTransformIndex()];
			FOutput& Output = Outputs[Group.GetOutputIndex()];

			// Get gain
			const float Gain = GainMatrixEntry.Value;

			// Only process the number of blocks needed to process the specific transform.
			const int32 NumBlocksToProcess = Transform.GetNumActiveBlocks();

			if (NumBlocksToProcess > 0)
			{
				// Scale input by gain
				VectorMultiplyByConstant(Input.GetTransformedBlock(), Gain, TransformedAndScaledInput);

				// Multiply transformed input by each block of transformed impulse response.
				// Accumulate the result of that multiplication in the appropriate output block.
				for (int32 i = 0; i < NumBlocksToProcess; i++)
				{
					VectorComplexMultiplyAdd(TransformedAndScaledInput, Transform.GetTransformedBlock(i), Output.GetTransformedBlock(i));
				}
			}
		}

		// Pop blocks off of outputs.
		for (int32 i = 0; i < Settings.NumOutputChannels; i++)
		{
			if (IsOutputZero[i])
			{
				FMemory::Memset(OutSamples[i], 0, sizeof(float) * BlockSize);

				// This avoids calling inverse FFT when output is zero due to 
				// no non-zero gains. 
				Outputs[i].PopBlock();
			}
			else
			{
				Outputs[i].PopBlock(OutSamples[i]);
			}
		}
	}

	/** Reset internal history buffers . */
	void FUniformPartitionConvolution::ResetAudioHistory()
	{
		for (int32 i = 0; i < Settings.NumInputChannels; i++)
		{
			Inputs[i].Reset();
		}

		for (int32 i = 0; i < Settings.NumOutputChannels; i++)
		{
			Outputs[i].Reset();
		}
	}

	/** Maximum supported length of impulse response. */
	int32 FUniformPartitionConvolution::GetMaxNumImpulseResponseSamples() const
	{
		return Settings.MaxNumImpulseResponseSamples;
	}

	/** Return the number of impulse responses. */
	int32 FUniformPartitionConvolution::GetNumImpulseResponses() const
	{
		return Settings.NumImpulseResponses;
	}

	/** Return the number of samples in an impulse response. */
	int32 FUniformPartitionConvolution::GetNumImpulseResponseSamples(int32 InImpulseResponseIndex) const
	{
		return ImpulseResponses[InImpulseResponseIndex].GetNumImpulseResponseSamples();
	}

	/** Set impulse response values. */
	void FUniformPartitionConvolution::SetImpulseResponse(int32 InImpulseResponseIndex, const float* InSamples, int32 InNumSamples)
	{
		check(InNumSamples <= Settings.MaxNumImpulseResponseSamples);
		ImpulseResponses[InImpulseResponseIndex].SetImpulseResponse(InSamples, InNumSamples);
	}

	/** Sets the gain between an audio input, impulse response and audio output.
	*
	* ([audio inputs] * [impulse responses]) x [gain matrix] = [audio outputs]
	*/
	void FUniformPartitionConvolution::SetMatrixGain(int32 InAudioInputIndex, int32 InImpulseResponseIndex, int32 InAudioOutputIndex, float InGain)
	{
		// Create index into 3D gain matrix
		const FInputTransformOutputGroup Group(InAudioInputIndex, InImpulseResponseIndex, InAudioOutputIndex);

		if (InGain == 0.f)
		{
			// Remove gain entry if it's zero
			NonZeroGains.Remove(Group);

			// First, assume if setting gain to zero that no audio is sent to output.
			IsOutputZero[InAudioOutputIndex] = true;

			// Check that no non-zero gain is associated with this output.
			for (auto GainMapEntry : NonZeroGains)
			{
				if (GainMapEntry.Key.GetOutputIndex() == InAudioOutputIndex)
				{
					// There is a non-zero gain associated with this output. 
					IsOutputZero[InAudioOutputIndex] = false;
					break;
				}
			}
		}
		else
		{
			NonZeroGains.Add(Group, InGain);
			IsOutputZero[InAudioOutputIndex] = false;
		}
	}

	/** Gets the gain between an audio input, impulse response and audio output.
	*
	* ([audio inputs] * [impulse responses]) x [gain matrix] = [audio outputs]
	*/
	float FUniformPartitionConvolution::GetMatrixGain(int32 InAudioInputIndex, int32 InImpulseResponseIndex, int32 InAudioOutputIndex) const
	{
		float OutGain = 0.f;

		const FInputTransformOutputGroup Group(InAudioInputIndex, InImpulseResponseIndex, InAudioOutputIndex);
		if (NonZeroGains.Contains(Group))
		{
			OutGain = NonZeroGains[Group];
		}

		return OutGain;
	}

	// If you bitwise and with this, it gives you (Num = Num - (Num % 4)). This is an easy way
	// to determine how many values can be SIMD'd.
	const int32 FUniformPartitionConvolution::NumSimdMask = 0xFFFFFFFC;

	void FUniformPartitionConvolution::VectorComplexMultiplyAdd(const FAlignedFloatBuffer& InA, const FAlignedFloatBuffer& InB, FAlignedFloatBuffer& Out) const
	{
		ArrayComplexMultiplyAdd(InA, InB, Out);
	}

	// Multiply aligned buffer by constant gain.
	void FUniformPartitionConvolution::VectorMultiplyByConstant(const FAlignedFloatBuffer& InBuffer, float InConstant, FAlignedFloatBuffer& OutBuffer) const
	{
		check(InBuffer.Num() == OutBuffer.Num());

		VectorRegister4Float VectorConstant = MakeVectorRegisterFloat(InConstant, InConstant, InConstant, InConstant);

		const int32 Num = InBuffer.Num();
		const int32 NumSimd = Num & NumSimdMask;

		const float* InData = InBuffer.GetData();
		float* OutData = OutBuffer.GetData();

		for (int32 i = 0; i < NumSimd; i += 4)
		{
			VectorRegister4Float VectorIn = VectorLoad(&InData[i]);

			VectorRegister4Float VectorOut = VectorMultiply(VectorIn, VectorConstant);
			
			VectorStore(VectorOut, &OutData[i]);
		}

		for (int32 i = NumSimd; i < Num; i++)
		{
			OutData[i] = InData[i] * InConstant;
		}
	}

	FUniformPartitionConvolution::FInput::FInput(FSharedFFTRef InFFTAlgorithm)
	:	BlockSize(InFFTAlgorithm->Size() / 2)
	,	FFTAlgorithm(InFFTAlgorithm)
	{
		check(BlockSize > 0);

		InputBuffer.AddZeroed(FFTAlgorithm->NumInputFloats());
		OutputBuffer.AddUninitialized(FFTAlgorithm->NumOutputFloats());
	}

	void FUniformPartitionConvolution::FInput::PushBlock(const float* InSamples)
	{
		float* InputData = InputBuffer.GetData();
		float* OutputData = OutputBuffer.GetData();

		// The two copies here effectively perform a hop of "BlockSize" number of samples
		// between subsequent calls to FFTAlgorithm->ForwardRealToComplex
		FMemory::Memcpy(&InputData[BlockSize], InSamples, BlockSize * sizeof(float));

		FFTAlgorithm->ForwardRealToComplex(InputData, OutputData);

		FMemory::Memcpy(InputData, InSamples, BlockSize * sizeof(float));
	}

	const FAlignedFloatBuffer& FUniformPartitionConvolution::FInput::GetTransformedBlock() const
	{
		return OutputBuffer;
	}

	void FUniformPartitionConvolution::FInput::Reset()
	{
		FMemory::Memset(InputBuffer.GetData(), 0, sizeof(float) * InputBuffer.Num());
		FMemory::Memset(OutputBuffer.GetData(), 0, sizeof(float) * OutputBuffer.Num());
	}

	FUniformPartitionConvolution::FOutput::FOutput(int32 InNumBlocks, FSharedFFTRef InFFTAlgorithm)
	:	NumBlocks(InNumBlocks)
	,	BlockSize(InFFTAlgorithm->Size() / 2)
	,	NumFFTOutputFloats(InFFTAlgorithm->NumOutputFloats())
	,	FFTAlgorithm(InFFTAlgorithm)
	,	HeadBlockIndex(0)
	{
		Blocks.AddDefaulted(NumBlocks);

		for (int32 i = 0; i < NumBlocks; i++)
		{
			Blocks[i].AddZeroed(NumFFTOutputFloats);
		}

		OutputBuffer.AddZeroed(FFTAlgorithm->NumInputFloats());
	}


	FUniformPartitionConvolution::FOutput::~FOutput()
	{
	}

	FAlignedFloatBuffer& FUniformPartitionConvolution::FOutput::GetTransformedBlock(int32 InBlockIndex)
	{
		// Make block index relative to head index.
		InBlockIndex = (HeadBlockIndex + InBlockIndex) % NumBlocks;

		return Blocks[InBlockIndex];
	}

	void FUniformPartitionConvolution::FOutput::PopBlock()
	{
		const int32 TailBlockIndex = HeadBlockIndex;

		// Increment head pointer
		HeadBlockIndex = (HeadBlockIndex + 1) % NumBlocks;

		// Set tail block to zero
		FAlignedFloatBuffer& TailBuffer = Blocks[TailBlockIndex];
		FMemory::Memset(TailBuffer.GetData(), 0, NumFFTOutputFloats * sizeof(float));

	}

	void FUniformPartitionConvolution::FOutput::PopBlock(float* OutSamples)
	{
		const FAlignedFloatBuffer& HeadBuffer = Blocks[HeadBlockIndex];
		float* OutputData = OutputBuffer.GetData();

		FFTAlgorithm->InverseComplexToReal(HeadBuffer.GetData(), OutputData);

		// Overlap save method of convolution. Only use 2nd half of output buffer
		FMemory::Memcpy(OutSamples, &OutputData[BlockSize], sizeof(float) * BlockSize);

		// Perform head block increment. 
		PopBlock();
	}

	void FUniformPartitionConvolution::FOutput::Reset()
	{
		for (int32 i = 0; i < NumBlocks; i++)
		{
			FMemory::Memset(Blocks[i].GetData(), 0, sizeof(float) * NumFFTOutputFloats);
		}

		FMemory::Memset(OutputBuffer.GetData(), 0, sizeof(float) * 2 * BlockSize);
	}


	// Hold FFTs of IR
	FUniformPartitionConvolution::FImpulseResponse::FImpulseResponse(int32 InNumBlocks, FSharedFFTRef InFFTAlgorithm)
	:	NumBlocks(InNumBlocks)
	,	BlockSize(InFFTAlgorithm->Size() / 2)
	,	FFTAlgorithm(InFFTAlgorithm)
	,	NumFFTOutputFloats(InFFTAlgorithm->NumOutputFloats())
	,	NumActiveBlocks(0)
	,	NumImpulseResponseSamples(0)
	{
		check(FFTAlgorithm->Size() > 0);
		check(NumFFTOutputFloats > 0);
		check(BlockSize > 0);

		FFTInput.AddZeroed(FFTAlgorithm->Size());

		Blocks.AddDefaulted(NumBlocks);

		for (int32 i = 0; i < NumBlocks; i++)
		{
			Blocks[i].AddZeroed(NumFFTOutputFloats);
		}
	}

	FUniformPartitionConvolution::FImpulseResponse::~FImpulseResponse()
	{
	}

	void FUniformPartitionConvolution::FImpulseResponse::SetImpulseResponse(const float* InSamples, int32 InNum)
	{
		check(InNum >= 0);

		// Check that input length is within limits
		const int32 MaxNumSamples = NumBlocks * BlockSize;
		if (InNum > MaxNumSamples)
		{
			UE_LOGF(LogSignalProcessing, Warning, "Truncating impulse response of %d samples to maximum length of %d samples", InNum, MaxNumSamples);
			InNum = MaxNumSamples;
		}
		NumImpulseResponseSamples = InNum;

		// Zero out previous blocks
		for (int32 i = 0; i < NumBlocks; i++)
		{
			FMemory::Memset(Blocks[i].GetData(), 0, sizeof(float) * NumFFTOutputFloats);
		}

		NumActiveBlocks = InNum / BlockSize;
		// Add extra block to account for round-down nature of integer division.
		NumActiveBlocks += InNum % BlockSize == 0 ? 0 : 1;

		// Store partitioned & transformed version of impulse response. 
		for (int32 i = 0; i < NumActiveBlocks; i++)
		{
			// Copy samples to input buffer.
			const int32 ReadPos = i * BlockSize;
			const int32 NumSamplesToCopy = FMath::Min(BlockSize, NumImpulseResponseSamples - ReadPos);
			const int32 NumSamplesToZero = BlockSize - NumSamplesToCopy;

			FMemory::Memcpy(FFTInput.GetData(), &InSamples[ReadPos], NumSamplesToCopy * sizeof(float));
			if (NumSamplesToZero > 0)
			{
				FMemory::Memset(&FFTInput.GetData()[NumSamplesToCopy], 0, NumSamplesToZero * sizeof(float));
			}

			// Perform fft
			FFTAlgorithm->ForwardRealToComplex(FFTInput.GetData(), Blocks[i].GetData());	
		}
	}

	const FAlignedFloatBuffer& FUniformPartitionConvolution::FImpulseResponse::GetTransformedBlock(int32 InBlockIndex) const
	{
		return Blocks[InBlockIndex];
	}

	int32 FUniformPartitionConvolution::FImpulseResponse::GetNumActiveBlocks() const
	{
		return NumActiveBlocks;
	}

	int32 FUniformPartitionConvolution::FImpulseResponse::GetNumImpulseResponseSamples() const
	{
		return NumImpulseResponseSamples;
	}


	FUniformPartitionConvolutionFactory::~FUniformPartitionConvolutionFactory()
	{
	}

	/** Name of this particular factory. */
	const FName FUniformPartitionConvolutionFactory::GetFactoryName() const
	{
		static const FName FactoryName("UniformPartitionConvolutionFactory");
		return FactoryName;
	}

	/** If true, this implementation uses hardware acceleration. */
	bool FUniformPartitionConvolutionFactory::IsHardwareAccelerated() const
	{
		return false;
	}

	/** Returns true if the input settings are supported by this factory. */
	bool FUniformPartitionConvolutionFactory::AreConvolutionSettingsSupported(const FConvolutionSettings& InSettings) const
	{
		bool bIsValidNumInputs = InSettings.NumInputChannels > 0;
		bool bIsValidNumOutputs = InSettings.NumOutputChannels > 0;
		bool bIsValidNumImpulseResponses = InSettings.NumImpulseResponses > 0;
		bool bIsValidMaxNumImpulseResponseSamples = InSettings.MaxNumImpulseResponseSamples > 0;
		// Block Size must be greater than zero and a power of 2.
		bool bIsValidBlockSize = (InSettings.BlockNumSamples > 0) && (FMath::CountBits(InSettings.BlockNumSamples) == 1);

		// Check if FFFTFactory supports required fftsettings
		FFFTSettings FFTSettings = FUniformPartitionConvolutionFactory::FFTSettingsFromConvolutionSettings(InSettings);
		bool bSupportedFFTSettings = FFFTFactory::AreFFTSettingsSupported(FFTSettings);

		bool bIsSupported = bIsValidNumInputs;
		bIsSupported &= bIsValidNumOutputs;
		bIsSupported &= bIsValidNumImpulseResponses;
		bIsSupported &= bIsValidMaxNumImpulseResponseSamples;
		bIsSupported &= bIsValidBlockSize;
		bIsSupported &= bSupportedFFTSettings;

		return bIsSupported;
	}

	/** Creates a new Convolution algorithm. */
	TUniquePtr<IConvolutionAlgorithm> FUniformPartitionConvolutionFactory::NewConvolutionAlgorithm(const FConvolutionSettings& InSettings)
	{
		check(AreConvolutionSettingsSupported(InSettings));

		// Create required FFT settings based on convolution settings.
		FFFTSettings FFTSettings = FUniformPartitionConvolutionFactory::FFTSettingsFromConvolutionSettings(InSettings);

		// Create an FFT algorithm
		TUniquePtr<IFFTAlgorithm> UniqueFFTAlgorithm = FFFTFactory::NewFFTAlgorithm(FFTSettings);
		if (!UniqueFFTAlgorithm.IsValid())
		{
			UE_LOGF(LogSignalProcessing, Error, "Failed to create uniform partition convolution algorithm due to failed FFT creation");
			return TUniquePtr<IConvolutionAlgorithm>();
		}

		// Convert FFT to shared ref 
		TSharedRef<IFFTAlgorithm> FFTRef(UniqueFFTAlgorithm.Release());

		// Create UniformPartitionConvolutionSettings
		FUniformPartitionConvolutionSettings UPConvSettings;
		UPConvSettings.NumInputChannels = InSettings.NumInputChannels;
		UPConvSettings.NumOutputChannels = InSettings.NumOutputChannels;
		UPConvSettings.NumImpulseResponses = InSettings.NumImpulseResponses;
		UPConvSettings.MaxNumImpulseResponseSamples = InSettings.MaxNumImpulseResponseSamples;

		return MakeUnique<FUniformPartitionConvolution>(UPConvSettings, FFTRef);
	}

	FFFTSettings FUniformPartitionConvolutionFactory::FFTSettingsFromConvolutionSettings(const FConvolutionSettings& InConvolutionSettings)
	{
		FFFTSettings FFTSettings;
		FFTSettings.bEnableHardwareAcceleration = InConvolutionSettings.bEnableHardwareAcceleration;
		FFTSettings.bArrays128BitAligned = true;

		int32 Log2FFTSize = 0;

		if (InConvolutionSettings.BlockNumSamples)
		{
			int32 Log2BlockSize = FMath::CountTrailingZeros(InConvolutionSettings.BlockNumSamples);
			// For overlap save method of convolution, FFT is twice as big as audio block. 
			Log2FFTSize = Log2BlockSize + 1;
		}

		FFTSettings.Log2Size = Log2FFTSize;

		return FFTSettings;
	}
}
