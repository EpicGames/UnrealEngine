// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/ConvolutionAlgorithm.h"
#include "DSP/FFTAlgorithm.h"

namespace Audio
{
	/** Settings for creating a uniform partition convolution. */
	struct FUniformPartitionConvolutionSettings
	{
		/** Number of input channels for convolution. */
		int32 NumInputChannels = 0;

		/** Number of output channels for convolution. */
		int32 NumOutputChannels = 0; 

		/** Number of impulse responses for convolution. */
		int32 NumImpulseResponses = 0; 

		/** Maximum number of samples in an impulse response. */
		int32 MaxNumImpulseResponseSamples = 0;
	};

	/** FUniformPartitionConvolution
	 *
	 * FUniformPartitionConvolution implements a fft based convolution algorithm supporting multiple 
	 * inputs, impulse responses and outputs. The algorithm subdivides an impulse response into 
	 * uniform blocks which results in several positive side effects:
	 *
	 * 	1. Latency is set to the block size which in many cases is less than the length of the impulse response.
	 * 	2. Increased performance over brute time domain convolution.
	 * 	3. Reuse of fourier transformed data when inputs are mapped to multiple outputs.
	 *
	 * The block size and latency is determined by the FFT algorithm passed to the constructor. 
	 * 	BlockSize = FFTAlgorithm->Size() / 2
	 * 	Latency = FFTAlgorithm->Size() / 2
	 */
	class FUniformPartitionConvolution : public IConvolutionAlgorithm
	{
		public:
			/** Shared reference to FFT algorithm. */
			using FSharedFFTRef = TSharedRef<IFFTAlgorithm>;

			/** Create a Uniform Partition Convolution Algorithm
			 *
			 * @params InSettings - Settings for algorithm.
			 * @param InFFTAlgorithm - FFTAlgorithm used to transform inputs, outputs and impulse responses.
			 */
			FUniformPartitionConvolution(const FUniformPartitionConvolutionSettings& InSettings, FSharedFFTRef InFFTAlgorithm);

			/** Destructor. */
			virtual ~FUniformPartitionConvolution();

			/** Returns the number of samples in an audio block. */
			virtual int32 GetNumSamplesInBlock() const override;

			/** Returns number of audio inputs. */
			virtual int32 GetNumAudioInputs() const override;

			/** Returns number of audio outputs. */
			virtual int32 GetNumAudioOutputs() const override;

			/** Process one block of audio.
			 *
			 * InSamples is processed by the impulse responses. The output is placed in OutSamples.
			 *
			 * @params InSamples - A 2D array of input deinterleaved audio samples. InSamples[GetNumAudioInputs()][GetNumSamplesInBlock()]
			 * @params OutSamples - A 2D array of output deinterleaved audio samples. OutSamples[GetNumAudioOutputs()][GetNumSamplesInBlock()]
			 *
			 */
			virtual void ProcessAudioBlock(const float* const InSamples[], float* const OutSamples[]) override;

			/** Reset internal history buffer for all audio inputs. */
			virtual void ResetAudioHistory() override;

			/** Maximum supported length of impulse response. */
			virtual int32 GetMaxNumImpulseResponseSamples() const override;

			/** Return the number of impulse responses. */
			virtual int32 GetNumImpulseResponses() const override;

			/** Return the number of samples in an impulse response. */
			virtual int32 GetNumImpulseResponseSamples(int32 InImpulseResponseIndex) const override;

			/** Set impulse response values. */
			virtual void SetImpulseResponse(int32 InImpulseResponseIndex, const float* InSamples, int32 InNumSamples) override;

			/** Sets the gain between an audio input channel, impulse response and audio output channel.
			 *
			 * ([audio inputs] * [impulse responses]) x [gain matrix] = [audio outputs]
			 */
			virtual void SetMatrixGain(int32 InAudioInputIndex, int32 InImpulseResponseIndex, int32 InAudioOutputIndex, float InGain) override;

			/** Gets the gain between an audio input channel, impulse response and audio output channel.
			 *
			 * ([audio inputs] * [impulse responses]) x [gain matrix] = [audio outputs]
			 */
			virtual float GetMatrixGain(int32 InAudioInputIndex, int32 InImpulseResponseIndex, int32 InAudioOutputIndex) const override;

		private:

			// Bit mask to determine how many values to apply SIMD optimization. 
			static const int32 NumSimdMask;

			// Triple index into 3D mixing matrix.
			struct FInputTransformOutputGroup : public TTuple<int32, int32, int32>
			{
				FInputTransformOutputGroup(int32 InInputIndex, int32 InTransformIndex, int32 InOutputIndex)
				:	TTuple<int32, int32, int32>(InInputIndex, InTransformIndex, InOutputIndex)
				{}

				int32 GetInputIndex() const
				{
					return Get<0>();
				}

				int32 GetTransformIndex() const
				{
					return Get<1>();
				}

				int32 GetOutputIndex() const
				{
					return Get<2>();
				}
			};

			// Vector optimized complex multiply add
			void VectorComplexMultiplyAdd(const FAlignedFloatBuffer& InA, const FAlignedFloatBuffer& InB, FAlignedFloatBuffer& Out) const;

			// Vector optimized multiply
			void VectorMultiplyByConstant(const FAlignedFloatBuffer& InBuffer, float InConstant, FAlignedFloatBuffer& OutBuffer) const;

			// Contains a single input channel.
			//
			// The input channel holds the FFT of the latest input block.
			struct FInput
			{
					// 
					FInput(FSharedFFTRef InFFTAlgorithm);

					// Add one block of audio samples.
					void PushBlock(const float* InSamples);

					// Reference to latest transformed block.
					const FAlignedFloatBuffer& GetTransformedBlock() const;

					// Clears the transformed block
					void Reset();

					// Number of samples in a block
					const int32 BlockSize;

				private: 

					FSharedFFTRef FFTAlgorithm;
					FAlignedFloatBuffer InputBuffer;
					FAlignedFloatBuffer OutputBuffer;
			};

			// Contains a single output channel
			//
			// The output channel holds a running history of fourier transformed output blocks. 
			// When a block is popped, it converts it from the fourier domain to the time domain.
			struct FOutput
			{
					// Construct an output
					//
					// @param InNumBlocks - Number of running history blocks
					// @param InFFTAlgorithm - FFT algorithm for performing inverse FFT.
					FOutput(int32 InNumBlocks, FSharedFFTRef InFFTAlgorithm);

					~FOutput();

					// Returns the history block at the given index. 
					FAlignedFloatBuffer& GetTransformedBlock(int32 InBlockIndex);	

					// Pop the latest block without performing inverse FFT
					void PopBlock();

					// Pop the latest block and perform inverse fft.
					// Out samples should hold InFFTAlgorithm->Size() / 2 floats. 
					void PopBlock(float* OutSamples);

					// Reset history of output blocks.
					void Reset();

					const int32 NumBlocks;
					const int32 BlockSize;
					const int32 NumFFTOutputFloats;
				private:

					FSharedFFTRef FFTAlgorithm;
					FAlignedFloatBuffer OutputBuffer;

					int32 HeadBlockIndex;
					TArray<FAlignedFloatBuffer> Blocks;

			};

			// Contains a single impulse response 
			//
			// The impulse response is broken into blocks and transformed to the fourier domain.
			struct FImpulseResponse
			{
					// Constructor
					//
					// @params InNumBlocks - Maximum number of FFT blocks
					// @param InFFTAlgorithm - FFT algorithm to convert samples to fourier.
					FImpulseResponse(int32 InNumBlocks, FSharedFFTRef InFFTAlgorithm);

					~FImpulseResponse();

					// Sets the impulse response.
					void SetImpulseResponse(const float* InSamples, int32 InNum);

					// Returns a block of the transformed impulse response.
					const FAlignedFloatBuffer& GetTransformedBlock(int32 InBlockIndex) const;

					// Returns the number of active blocks. This may be less than the maximum
					// number of blocks if the impulse response is shorter than the maximum
					// number of impulse response samples. 
					int32 GetNumActiveBlocks() const;

					// Returns the number of samples in the impulse response.
					int32 GetNumImpulseResponseSamples() const;

					const int32 NumBlocks;
					const int32 BlockSize;
				private:

					FSharedFFTRef FFTAlgorithm;
					int32 NumFFTOutputFloats;		
					int32 NumActiveBlocks;
					int32 NumImpulseResponseSamples;
					FAlignedFloatBuffer FFTInput;
					TArray<FAlignedFloatBuffer> Blocks;
			};

			FUniformPartitionConvolutionSettings Settings;

			int32 BlockSize;
			int32 NumFFTOutputFloats;
			int32 NumBlocks;
			FAlignedFloatBuffer TransformedAndScaledInput;
			TSharedRef<IFFTAlgorithm> FFTAlgorithm;

			TArray<FInput> Inputs;
			TArray<FOutput> Outputs;
			TArray<FImpulseResponse> ImpulseResponses;

			TMap<FInputTransformOutputGroup, float> NonZeroGains;
			TArray<bool> IsOutputZero;
	};

	/** FUniformPartitionConvolutionFactory creates FUniformPartitionConvolution algorithms. */
	class FUniformPartitionConvolutionFactory : public IConvolutionAlgorithmFactory
	{
		public:
			virtual ~FUniformPartitionConvolutionFactory();

			/** Name of this particular factory. */
			virtual const FName GetFactoryName() const override;

			/** If true, this implementation uses hardware acceleration. */
			virtual bool IsHardwareAccelerated() const override;

			/** Returns true if the input settings are supported by this factory. */
			virtual bool AreConvolutionSettingsSupported(const FConvolutionSettings& InSettings) const override;

			/** Creates a new Convolution algorithm. */
			virtual TUniquePtr<IConvolutionAlgorithm> NewConvolutionAlgorithm(const FConvolutionSettings& InSettings) override;

		private:

			// Convert the convolution settings into FFTAlgorithm settings.
			static FFFTSettings FFTSettingsFromConvolutionSettings(const FConvolutionSettings& InConvolutionSettings);
	};
}
