// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/VariablePoleFilter.h"

namespace Audio
{
	void FVariablePoleFilter::Init(EFilterOrder InOrder
								   , const float InSampleRate
								   , const int32 InNumChannels
								   , const float InFrequency
								   , const EBiquadFilter::Type InType
								   , const float InBandwidth
								   , const float InGain)
	{
		NumFilters = (int32)InOrder;
		NumChannels = InNumChannels;
		Type = InType;

		if (Filters.Num() > 0)
		{
			Filters.Reset(NumFilters);
		}

		Filters.AddDefaulted(NumFilters);
		for (int32 FilterId = 0; FilterId < NumFilters; FilterId++)
		{
			Filters[FilterId].Init(InSampleRate, InNumChannels, InType, InFrequency, InBandwidth, InGain);
		}
	}

	void FVariablePoleFilter::SetParams(const EBiquadFilter::Type InFilterType, const float InCutoffFrequency, const float InBandwidth, const float InGain)
	{
		for (int32 FilterId = 0; FilterId < NumFilters; FilterId++)
		{
			Filters[FilterId].SetParams(InFilterType, InCutoffFrequency, InBandwidth, InGain);
		}
	}

	void FVariablePoleFilter::ProcessAudioFrame(const float* InFrame, float* OutFrame)
	{
		if (!ensure(NumFilters >= 1))
		{
			return;
		}

		Filters[0].ProcessAudioFrame(InFrame, OutFrame);

		for (int32 FilterId = 1; FilterId < NumFilters; FilterId++)
		{
			Filters[FilterId].ProcessAudioFrame(OutFrame, OutFrame);
		}
	}

	void FVariablePoleFilter::ProcessAudioBuffer(const float* InFrame, float* OutFrame, const int32 NumFrames)
	{
		if (!ensure(NumFilters >= 1))
		{
			return;
		}

		const float* InPtr = InFrame;
		float* OutPtr = OutFrame;

		for (int32 FrameId = 0; FrameId < NumFrames; FrameId++)
		{
			Filters[0].ProcessAudioFrame(InPtr, OutPtr);

			for (int32 FilterId = 1; FilterId < NumFilters; FilterId++)
			{
				Filters[FilterId].ProcessAudioFrame(OutPtr, OutPtr);
			}

			InPtr += NumChannels;
			OutPtr += NumChannels;
		}
	}

	void FVariablePoleFilter::Reset()
	{
		for (int32 FilterId = 1; FilterId < NumFilters; FilterId++)
		{
			Filters[FilterId].Reset();
		}
	}
}