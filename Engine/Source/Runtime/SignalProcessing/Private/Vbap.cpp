// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Vbap.h"

#include "Math/Vector4.h"
#include "Math/Vector2D.h"
#include "Math/Matrix.h"
#include "Math/TransformCalculus2D.h"

// We define speakers assuming the following. 
// (x-y plane, top down, +y is forwards, -y is backwards)
// We rotate -90 degrees, counter clock wise, so that 0 degrees is forward.
// This makes the most sense for listener orientated speaker definitions.
// Front Speaker for example would be -30 (left) and +30 degrees right.
//
// 
//             (y+)                        
//	  FL(-30)   |    FR (+30)            unit lengths, so h = 1.
//              |                          /|
// (-x)  _______________ (+x)       (hyp) / | (opp) = sin(phi)*h 
//              |                  (phi) /__|         
//              |                       (adj)  = cos(phi)*h
//				|
//			  (y-)

// Elevation is 0 degrees if there is no height component, i.e 2D.
// (imagine standing on floor plane (x+y)
//            (+z) (up)  
//				|      (height channels) (30 degree elevation is typical)
//              |
///   --------------------   (x+y)    (most speakers lane on the 0 degree line).
//				|	
//			    |      (there's not normally speakers below z, but they'd be negative if there are).
//              |
//			   (-z) (down)

// To adjust we do the following. This matches UE spherical convention.
//    -90 degrees from Azimuth (Phi), that makes 0 degrees in front.
//	  90 - Elevation (Theta), that puts 0 degrees on x+y plane, so that 0 degrees elevation is no height, or 2d.

namespace Audio
{
	FVector3f IDiscretePanner::SphericalToUnitCartesian(const float InAzimuthDegs, const float InElevationDegs)
	{
		// To adjust we do the following. This matches UE spherical convention.
		//   -90 degrees from Azimuth (Phi), that makes 0 degrees in front.
		//	  90 - Elevation (Theta), that puts 0 degrees on x+y plane, so that 0 degrees elevation is no height, or 2d.
		
		// Convert spherical to unit vector. (rotate 90 degrees, so ahead is straight forward).
		const float AzimuthRads = FMath::DegreesToRadians(InAzimuthDegs - 90.f);
		const float ElevationRads = FMath::DegreesToRadians(90.f - InElevationDegs);
		const FVector3f Dir = FVector2f(ElevationRads,AzimuthRads).SphericalToUnitCartesian();
		return Dir;
	}

	FVBap2D::FVBap2D(const TArray<FSpeaker>& InSpeakers)
	{
		BuildSpans(InSpeakers);
	}
	
	int32 FVBap2D::FindBestSpan(const float InAzimuthDegrees) const
	{
		// Spans are sorted on azimuth. [-180 - 180].
		// So we can binary search for first spans A greater than this azimuth
		auto CompareByAzimuth = [](const FSpan& Value) -> float { return Value.A.AzimuthDegrees; }; 
		int32 Index = Algo::UpperBoundBy(Spans, InAzimuthDegrees, CompareByAzimuth);

		// Always choose previous one. UpperBound can return Num().
		// ... and handle 0 case with wrap.
		Index = (Index - 1 + Spans.Num()) % Spans.Num();
		return Index;
	} 
	
	bool FVBap2D::ComputeGains(const FInputParams& InParams, FOutputParams& OutResults) const
	{
		OutResults.Reset();
		
		// Edge case no spans
		if (Spans.Num() == 0)
		{
			return false;
		}

		// Single speaker? 
		if (Spans.Num() == 1 && Spans[0].A.ChannelID == Spans[0].B.ChannelID)
		{
			// Ignore params, return full gain.
			OutResults.Results.Emplace(Spans[0].A.ChannelID, Spans[0].A.ChannelIndex, 1.f);
			return true;
		}

		// Find span that contains azimuth
		const int32 BestSpan = FindBestSpan(InParams.AzimuthDegrees);
		if (BestSpan == INDEX_NONE)
		{
			return false;
		}

		check(Spans.IsValidIndex(BestSpan));
		const FSpan& Span = Spans[BestSpan];

		// Clamp incoming Azimuth to valid range. (-180 to +180)
		float AzimuthClamped = InParams.AzimuthDegrees;
		AzimuthClamped = FMath::Clamp(AzimuthClamped, -180.f, 180.f);

		// Vector Based Amplitude Panning in 2D.
		// Source Direction (p) is Linear Combination of the Both Speakers (l1 and l2) and two gains (g1 and g2). p = l1*g1 + l2*g2
		// We know p and we know l1 and l2, so we solve for g1 and g2.
		// L(l1,l2) = Matrix of both Speaker Directions.
		// g(g1,g2) = Resulting Vector of both gains.
		// g = L(-1) * p (Transform vector with inverse of L)

		// Edge cases:
		// If span is > 180 the math will fail.
		
		// Optionally mirror the northern hemisphere if there's nothing in the southern hemisphere
		if (InParams.bAllowAzimuthMirroring && MinAzimuth > -90.f && MaxAzimuth <= 90.f )
		{
			if (AzimuthClamped > 90.f)	// 90 to 180. (right side).
			{
				AzimuthClamped = 180.f - AzimuthClamped; 
			}
			else if (AzimuthClamped < -90.f) // -90 to -180 (left side).
			{
				AzimuthClamped = -180 - AzimuthClamped; 
			}
		}

		//const int32 SpanRangeMin = Span.A.AzimuthDegrees;
		//const int32 SpanRangeMax = Span.B.AzimuthDegrees;
		
		// Clamp to valid extent of chosen span. (so each edge will be 100% if off the end).
		//AzimuthClamped = FMath::Clamp(AzimuthClamped, SpanRangeMin, SpanRangeMax);
						
		const FVector3f SourceDir = SphericalToUnitCartesian(AzimuthClamped, 0.f);
		
		FVector2f P(SourceDir.X, SourceDir.Y);															// P = Source dir 2d.
		P = P.GetSafeNormal();																			// Safely normalize.
		const TMatrix2x2 L(
			Span.DirA.X, Span.DirA.Y, 
			Span.DirB.X, Span.DirB.Y);																	// L = 2x2 Matrix of Speaker Directions A+B.
		const float LDet = L.Determinant();
		if (!ensure(!FMath::IsNearlyZero(LDet)))
		{
			return false;																				// We reject large span, should this shouldn't happen
		}
		const TMatrix2x2 Li = L.Inverse();																// LI = Inverted L.
		
		FVector2f G = Li.TransformVector(P);															// Produce gains.
		G = FVector2f::Clamp(G, FVector2f::ZeroVector,FVector2f::UnitVector);		// Possible to get -0.
		G = G.GetSafeNormal();																			// Normalize. (gives us equal power).
		
		OutResults.Results.Emplace( Span.A.ChannelID, Span.A.ChannelIndex, G.X);
		OutResults.Results.Emplace( Span.B.ChannelID, Span.B.ChannelIndex, G.Y);

		// Stabilize results by always sorting on Channel ID
		OutResults.Results.Sort([](const FPanResult& A, const FPanResult& B)
		{
			return A.ChannelID.FastLess(B.ChannelID);
		});
		
		return true;
	}
	
	void FVBap2D::BuildSpans(const TArray<FSpeaker>& InSpeakers)
	{
		// We need at least one speaker.
		check(InSpeakers.Num() >= 1);
		
		struct FSortedSpeaker
		{
			FSpeaker Speaker;
			FVector2f Dir; 
		};

		// Filter speakers for everything on the ground, no height channels. (i.e. anything off of the 0 elevation line).
		TArray<FSortedSpeaker> SortedSpeakers;
		for (int32 i = 0; i < InSpeakers.Num(); ++i)
		{
			const FSpeaker& Speaker = InSpeakers[i];

			// Filter out any speakers that have a height component.
			if (!FMath::IsNearlyZero(Speaker.ElevationDegrees))
			{
				continue;
			}

			// Convert from polar/spherical into unit cartesian.
			check(Speaker.AzimuthDegrees <= 180.f);
			check(Speaker.AzimuthDegrees >= -180.f);
			const float AzimuthRads = FMath::DegreesToRadians(Speaker.AzimuthDegrees - 90);
			const FVector2f Spherical{ FMath::DegreesToRadians(90), AzimuthRads };
			const FVector3f Dir3D = Spherical.SphericalToUnitCartesian();
			FVector2f Dir2D(Dir3D); Dir2D.Normalize();
			check(FMath::IsNearlyEqual(Dir2D.Length(), 1, KINDA_SMALL_NUMBER));
			check(!SortedSpeakers.ContainsByPredicate([&](const FSortedSpeaker& Value) { return Value.Speaker.ChannelID == Speaker.ChannelID; }));
			check(!SortedSpeakers.ContainsByPredicate([&](const FSortedSpeaker& Value) { return FMath::IsNearlyEqual(Value.Speaker.AzimuthDegrees,Speaker.AzimuthDegrees); }));
			SortedSpeakers.Emplace(Speaker, Dir2D);
			MinAzimuth = FMath::Min(MinAzimuth, Speaker.AzimuthDegrees);
			MaxAzimuth = FMath::Max(MaxAzimuth, Speaker.AzimuthDegrees);
		}
		
		// Sort on Azimuth.
		SortedSpeakers.Sort([](const FSortedSpeaker& A, const FSortedSpeaker& B) -> bool { return A.Speaker.AzimuthDegrees < B.Speaker.AzimuthDegrees; });

		auto CalculateWidth = [](const float A, const float B) -> float
		{
			if (A < B)
			{
				return B - A;
			}
			return 360 - (A - B);
		};
		
		// From spans consecutive pairs around circle.
		Spans.Empty();
		for (int32 i = 0; i < SortedSpeakers.Num(); i++)
		{
			const int32 NextIndex = (i + 1) % SortedSpeakers.Num(); // Wrap around
			const FSortedSpeaker& Speaker = SortedSpeakers[i];
			const FSortedSpeaker& NextSpeaker = SortedSpeakers[NextIndex];

			// Check we don't have any spans bigger than 180.
			const float SpanWidth = CalculateWidth(Speaker.Speaker.AzimuthDegrees, NextSpeaker.Speaker.AzimuthDegrees);
			
			// Only add spans that are less than 180. 
			// In the single speaker special case, that'll be > 180, but allow it anyway.
			if (FMath::Abs(SpanWidth) < 180.f || SortedSpeakers.Num() == 1)
			{
				Spans.Add(
					{
						.A = Speaker.Speaker,
						.B = NextSpeaker.Speaker,
						.DirA = Speaker.Dir,
						.DirB = NextSpeaker.Dir
					});
			}
			else 
			{
				// Don't warn about the wrap span if it's too wide.
				//UE_CLOGF(NextIndex != 0, LogTemp, Warning, "Speaker A=%d,Az=%.2f and Speaker B=%d,Az=%.2f has Width %.2f",
				//	Speaker.Speaker.ChannelID,
				//	Speaker.Speaker.AzimuthDegrees,
				//	NextSpeaker.Speaker.ChannelID,
				//	NextSpeaker.Speaker.AzimuthDegrees,
				//	SpanWidth );
			}
		}
			
	}
}
