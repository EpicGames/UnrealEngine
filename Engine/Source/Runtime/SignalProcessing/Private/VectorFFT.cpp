// Copyright Epic Games, Inc. All Rights Reserved.

#include "VectorFFT.h"
#include "SignalProcessingModule.h"

#include "Templates/UniquePtr.h"
#include "DSP/FFTAlgorithm.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	// Implementation of a complex FFT.
	class FVectorComplexFFT
	{
		public:
			// Minimum size of fft is required to support 2 radix-4 fft stages. 
			static const int32 MinLog2FFTSize = 4;

			// Maximum size of fft is set to avoid exceedingly large allocs and support
			// various logic for indexing blocks internally.
			static const int32 MaxLog2FFTSize = 16;

			// Constructor
			//
			// @param InLog2FFTSize - Log2 size of the FFT.
			FVectorComplexFFT(int32 InLog2FFTSize)
			:	Log2FFTSize(InLog2FFTSize)
			,	FFTSize(0)
			,	NumFloats(0)
			{
				check(Log2FFTSize >= MinLog2FFTSize);
				check(Log2FFTSize <= MaxLog2FFTSize);

				FFTSize = 1 << Log2FFTSize;
				NumFloats = 2 * FFTSize; // Takes 2 floats to represent a complex number

				InverseWorkBuffer.AddUninitialized(NumFloats);

				// Pregenerate weights needed to calculate this size of FFT.
				GenerateRadix4Weights();
				GenerateFinalIndices();
				GenerateFinalWeights();
			}

			~FVectorComplexFFT()
			{
			}

			// Perform forward complex FFT
			//
			// @param InComplex - Interleaved complex data with (2 * FFTSize) num floats.
			// @param OutComplex - Interleaved complex data with (2 * FFTSize) num floats.
			void ForwardComplexToComplex(const float* RESTRICT InComplex, float* RESTRICT OutComplex)
			{
				// To perform FFT, must complete Log2FFTSize stages.  Each radix pass performs 2^m stages
				// where 2^m is the radix number. So a radix-4 stage is radix-2^m or radix-2^2. Hence radix
				// 4 performs two stages. Radix-8 is Radix-2^3, so it performs 3 stages.
				int32 CompletedStages = 0;

				if (Log2FFTSize & 1)
				{
					// If we have an odd number of stages, start with a radix-8 to 
					// perform first 3 stages. 
					Radix8ButterflyConstantWeight(InComplex, OutComplex, Log2FFTSize);
					CompletedStages = 3;
				}
				else
				{
					// Ifawe have an even number of stages, start with a radix-4 to 
					// perform first 2 stages.
					Radix4ButterflyConstantWeight(InComplex, OutComplex, Log2FFTSize);
					CompletedStages = 2;
				}

				// Fit in a few more constant weight radix4s if possible. This routine is faster
				// than the default Radix4Butterfly because it does not need weights.
				for (int32 StageIndex = CompletedStages; StageIndex < Log2FFTSize - 4 ; StageIndex += 2 )
				{
					Radix4ButterflyConstantWeight(OutComplex, OutComplex, Log2FFTSize - StageIndex);
				}

				// Perform a bunch of radix4 ffts with varying weights
				// Logically you would arrange this loop to first iterate over stage indices, and then over
				// butterfly indices, but by reorganizing the loop the code is more cache coherent.
				for (int32 ButterflyIndex = 1 ; CompletedStages < (Log2FFTSize - 4) ; CompletedStages += 2)
				{
					for ( ; ButterflyIndex < (1 << CompletedStages); ++ButterflyIndex )
					{
						for (int32 StageIndex = CompletedStages; StageIndex < Log2FFTSize - 4 ; StageIndex += 2 )
						{
							Radix4Butterfly(OutComplex, ButterflyIndex, Log2FFTSize - StageIndex, Radix4Weights[ButterflyIndex]);
						}
					}
				}

				if (CompletedStages < (Log2FFTSize - 2))
				{
					// Special case for 2nd to last stage of fft which has better cache coherency
					Radix4Butterfly2ndToFinal(OutComplex, Log2FFTSize - 4);
				}

				// Special case for last stage of fft with cache coherency tricks and index reversal built in.
				Radix4ButterflyFinal(OutComplex, Log2FFTSize - 2);
			}

			void InverseComplexToComplex(const float* RESTRICT InComplex, float* RESTRICT OutComplex)
			{
				// Perform inverse FFT by complex conjugating the input and output.
				
				float* WorkData = InverseWorkBuffer.GetData();

				ScaledComplexConjugate(InComplex, 1.f, WorkData, NumFloats);

				ForwardComplexToComplex(WorkData, OutComplex);

				const float Scale = 1.f / static_cast<float>(FFTSize);

				ScaledComplexConjugate(OutComplex, Scale, OutComplex, NumFloats);
			}

		private:

			// Weight structure for general radix 4 pass.
			struct FRadix4Weight 
			{
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W1R[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W1RNeg[4]; // Negative version of W1R.

				// These have specialized sign flips for calculations of D2 and  D3
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W1RD2[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W1RD3[4]; 

				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W1I[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W2R[4];
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W2RNeg[4]; // Negative Version of W2R
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W2I[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W3R[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W3RNeg[4]; // Negative Version of W4R
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W3I[4];
			}; 


			// Structure to hold loaded inputs on final radix 4 pass
			struct FFinalInputs
			{
				VectorRegister4Float A0;
				VectorRegister4Float A1;
				VectorRegister4Float A2;
				VectorRegister4Float A3;
				VectorRegister4Float A4;
				VectorRegister4Float A5;
				VectorRegister4Float A6;
				VectorRegister4Float A7;
			};

			// Structure to hold loaded outputs on final radix 4 pass
			struct FFinalOutputs
			{
				VectorRegister4Float D0;
				VectorRegister4Float D1;
				VectorRegister4Float D2;
				VectorRegister4Float D3;
				VectorRegister4Float D4;
				VectorRegister4Float D5;
				VectorRegister4Float D6;
				VectorRegister4Float D7;
			};

			// Weight structure for final Radix4 pass
			struct FFinalWeights
			{
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W1R[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W1I[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W2R[4];
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W2I[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W3R[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W3I[4];
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W4R[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W4I[4];
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W5R[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W5I[4];
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W6R[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W6I[4];
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W7R[4]; 
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W7I[4];

				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W2RNeg[4]; // Negative version of W2R
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W2RD4[4]; // Special case for calculating D4
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W2RD6[4]; // Special case for calculating D5
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W3RD5[4];  // Special case for calculating D6
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W3RD7[4];  // Special case for calculating D7
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W3RNeg[4];  // Negative version of W3R
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W4RNeg[4];  // Negative version of W4R
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W5RNeg[4];  // Negative version of W5R
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W6RNeg[4];  // Negative version of W6R
				alignas(AUDIO_SIMD_BYTE_ALIGNMENT) float W7RNeg[4];  // Negative version of W7R
			};

			// Radix4 index locations for performing final Radix4 passes and bit reversal
			// without overwriting needed data.
			struct FFinalIndices
			{
				int32 ReadIndex;
				int32 WriteIndex;
			};

			// Perform complex conjugate as well as scale.
			//
			// @param InValues - Array of floats representing complex values in interleave format.
			// @param Scale - Scale to apply.
			// @param OutValues - Array of floats representing complex values in interleave format.
			// @param Num - Number of floats in array (NOT number of complex values).
			void ScaledComplexConjugate(const float* InValues, float Scale, float* OutValues, int32 Num)
			{
				ArrayScaledComplexConjugate(InValues, Num, OutValues, Scale);
			}

			// Perform a radix-4 butterfly which uses constant weights.
			void Radix4ButterflyConstantWeight(const float* InValues, float* OutValues, int32 InStageIndex)
			{
				// This routine only supported when stage index is greater than or equal to 3. 
				// This comes about for two reasons, 
				// 	1. A Radix-4 processes two stages. (m = StageCount, 2^m = 4).
				// 	2. SIMD optimizations processes two butterflies in parallel, so must have an
				// 	   even number of butterflies. This sets the mimum stage index to 3, which will
				// 	   result in minimally 2 radix-4 butterflies being calculated. 
				check(InStageIndex >= 3);

				// Calculate number of constant weight butterflies in this stage.
				const int NumButterflies = 1 << (InStageIndex - 2);

				const VectorRegister4Float SignFlipImag = MakeVectorRegisterFloat(1.f, -1.f, 1.f, -1.f);

				const int32 Offset0 = 0;
				const int32 Offset1 = 2 * NumButterflies;
				const int32 Offset2 = 4 * NumButterflies;
				const int32 Offset3 = 6 * NumButterflies;

				for (int32 i = 0; i < NumButterflies; i += 2)
				{
					const int32 Pos = 2 * i;
					const int32 Pos0 = Offset0 + Pos;
					const int32 Pos1 = Offset1 + Pos;
					const int32 Pos2 = Offset2 + Pos;
					const int32 Pos3 = Offset3 + Pos;

					VectorRegister4Float A0 = VectorLoad(&InValues[Pos0]);
					VectorRegister4Float A1 = VectorLoad(&InValues[Pos1]);
					VectorRegister4Float A2 = VectorLoad(&InValues[Pos2]);
					VectorRegister4Float A3 = VectorLoad(&InValues[Pos3]);

					VectorRegister4Float C0 = VectorAdd(A0, A2);
					VectorRegister4Float C2 = VectorSubtract(A0, A2);
					VectorRegister4Float C1 = VectorAdd(A1, A3);
					VectorRegister4Float C3 = VectorSubtract(A1, A3);

					VectorRegister4Float C3Conj = VectorMultiply(C3, SignFlipImag);
					VectorRegister4Float C3ConjSwizzle = VectorSwizzle(C3Conj, 1, 0, 3, 2);

					VectorRegister4Float D0 = VectorAdd(C1, C0);
					VectorRegister4Float D1 = VectorSubtract(C0, C1);
					VectorRegister4Float D2 = VectorAdd(C2, C3ConjSwizzle);
					VectorRegister4Float D3 = VectorSubtract(C2, C3ConjSwizzle);

					VectorStore(D0, &OutValues[Pos0]);
					VectorStore(D1, &OutValues[Pos1]);
					VectorStore(D2, &OutValues[Pos2]);
					VectorStore(D3, &OutValues[Pos3]);
				}
			}

			// Perform a radix-8 butterfly which uses constant weights.
			void Radix8ButterflyConstantWeight(const float* InValues, float* OutValues, int32 InStageIndex)
			{

				const float Sqrt2D2 = .7071067811865475244f;

				// This routine only supported when stage index is greater than or equal to four. 
				// This comes about for two reasons, 
				// 	1. A Radix-8 processes three stages. (m = StageCount, 2^m = 8).
				// 	2. SIMD optimizations processes two butterflies in parallel, so must have an
				// 	   even number of butterflies. This sets the mimum stage index to 4, which will
				// 	   result in minimally 2 radix-8 butterflies being calculated. 
				check(InStageIndex >= 4);

				// Calculate number of constant weight butterflies in this stage.
				const int32 NumButterflies = 1 << (InStageIndex - 3);

				const VectorRegister4Float SignFlipImag = MakeVectorRegisterFloat(1.f, -1.f, 1.f, -1.f);
				const VectorRegister4Float VectorSqrt2D2 = MakeVectorRegisterFloat(Sqrt2D2, Sqrt2D2, Sqrt2D2, Sqrt2D2);
				const VectorRegister4Float VectorNegSqrt2D2 = MakeVectorRegisterFloat(-Sqrt2D2, -Sqrt2D2, -Sqrt2D2, -Sqrt2D2);

				const int32 Offset0 = 0;
				const int32 Offset1 = 2 * NumButterflies;
				const int32 Offset2 = 4 * NumButterflies;
				const int32 Offset3 = 6 * NumButterflies;
				const int32 Offset4 = 8 * NumButterflies;
				const int32 Offset5 = 10 * NumButterflies;
				const int32 Offset6 = 12 * NumButterflies;
				const int32 Offset7 = 14 * NumButterflies;

				for (int32 i = 0; i < NumButterflies; i += 2)
				{
					const int32 Pos = 2 * i;
					const int32 Pos0 = Pos;
					const int32 Pos1 = Offset1 + Pos;
					const int32 Pos2 = Offset2 + Pos;
					const int32 Pos3 = Offset3 + Pos;
					const int32 Pos4 = Offset4 + Pos;
					const int32 Pos5 = Offset5 + Pos;
					const int32 Pos6 = Offset6 + Pos;
					const int32 Pos7 = Offset7 + Pos;

					VectorRegister4Float A0 = VectorLoad(&InValues[Pos0]);
					VectorRegister4Float A1 = VectorLoad(&InValues[Pos1]);
					VectorRegister4Float A2 = VectorLoad(&InValues[Pos2]);
					VectorRegister4Float A3 = VectorLoad(&InValues[Pos3]);
					VectorRegister4Float A4 = VectorLoad(&InValues[Pos4]);
					VectorRegister4Float A5 = VectorLoad(&InValues[Pos5]);
					VectorRegister4Float A6 = VectorLoad(&InValues[Pos6]);
					VectorRegister4Float A7 = VectorLoad(&InValues[Pos7]);

					VectorRegister4Float B0 = VectorAdd(A0, A4);
					VectorRegister4Float B1 = VectorAdd(A1, A5);
					VectorRegister4Float B2 = VectorAdd(A2, A6);
					VectorRegister4Float B3 = VectorAdd(A3, A7);
					VectorRegister4Float B4 = VectorSubtract(A0, A4);
					VectorRegister4Float B5 = VectorSubtract(A1, A5);
					VectorRegister4Float B6 = VectorSubtract(A2, A6);
					VectorRegister4Float B7 = VectorSubtract(A3, A7);

					VectorRegister4Float B6Conj = VectorMultiply(SignFlipImag, B6);
					VectorRegister4Float B6ConjSwizzle = VectorSwizzle(B6Conj, 1, 0, 3, 2);

					VectorRegister4Float B7Conj = VectorMultiply(SignFlipImag, B7);
					VectorRegister4Float B7ConjSwizzle = VectorSwizzle(B7Conj, 1, 0, 3, 2);

					VectorRegister4Float C0 = VectorAdd(B0, B2);
					VectorRegister4Float C1 = VectorAdd(B1, B3);
					VectorRegister4Float C2 = VectorSubtract(B0, B2);
					VectorRegister4Float C3 = VectorSubtract(B1, B3);
					VectorRegister4Float C4 = VectorAdd(B4, B6ConjSwizzle);
					VectorRegister4Float C5 = VectorAdd(B5, B7ConjSwizzle);
					VectorRegister4Float C6 = VectorSubtract(B4, B6ConjSwizzle);
					VectorRegister4Float C7 = VectorSubtract(B5, B7ConjSwizzle);

					VectorRegister4Float C3Conj = VectorMultiply(SignFlipImag, C3);
					VectorRegister4Float C3ConjSwizzle = VectorSwizzle(C3Conj, 1, 0, 3, 2);

					VectorRegister4Float C5Conj = VectorMultiply(SignFlipImag, C5);
					VectorRegister4Float C5ConjSwizzle = VectorSwizzle(C5Conj, 1, 0, 3, 2);
					VectorRegister4Float T5 = VectorAdd(C5, C5ConjSwizzle);

					VectorRegister4Float C7Swizzle = VectorSwizzle(C7, 1, 0, 3, 2);

					VectorRegister4Float T7 = VectorMultiplyAdd(SignFlipImag, C7, C7Swizzle);
					VectorRegister4Float T7Conj = VectorMultiply(T7, SignFlipImag);

					VectorRegister4Float D0 = VectorAdd(C0, C1);
					VectorRegister4Float D1 = VectorSubtract(C0, C1);
					VectorRegister4Float D2 = VectorAdd(C2, C3ConjSwizzle);
					VectorRegister4Float D3 = VectorSubtract(C2, C3ConjSwizzle);
					VectorRegister4Float D4 = VectorMultiplyAdd(T5, VectorSqrt2D2, C4);
					VectorRegister4Float D5 = VectorMultiplyAdd(T5, VectorNegSqrt2D2, C4);
					VectorRegister4Float D6 = VectorMultiplyAdd(VectorNegSqrt2D2, T7Conj, C6);
					VectorRegister4Float D7 = VectorMultiplyAdd(VectorSqrt2D2, T7Conj, C6);

					VectorStore(D0, &OutValues[Pos0]);
					VectorStore(D1, &OutValues[Pos1]);
					VectorStore(D2, &OutValues[Pos2]);
					VectorStore(D3, &OutValues[Pos3]);
					VectorStore(D4, &OutValues[Pos4]);
					VectorStore(D5, &OutValues[Pos5]);
					VectorStore(D6, &OutValues[Pos6]);
					VectorStore(D7, &OutValues[Pos7]);
				}
			}

			// Perform a radix4 butterfly with dynamic weights.
			void Radix4Butterfly(float* InOutValues, int32 ButterflyIndex, int32 InStageIndex, const FRadix4Weight& Weights)
			{
				// Number of values between butterflies.
				const int32 Stride = 1 << InStageIndex;

				const int32 NumButterflies = 1 << (InStageIndex - 2);

				// Load weights for butterfly
				const VectorRegister4Float Weight1Real = VectorLoad(Weights.W1R);
				const VectorRegister4Float Weight1Imag = VectorLoad(Weights.W1I);
				const VectorRegister4Float Weight2Real = VectorLoad(Weights.W2R);
				const VectorRegister4Float Weight2Imag = VectorLoad(Weights.W2I);
				const VectorRegister4Float Weight3Real = VectorLoad(Weights.W3R);
				const VectorRegister4Float Weight3Imag = VectorLoad(Weights.W3I);

				const VectorRegister4Float Weight3RealNeg = VectorLoad(Weights.W3RNeg);
				const VectorRegister4Float Weight2RealNeg = VectorLoad(Weights.W2RNeg);
				const VectorRegister4Float Weight1RealNeg = VectorLoad(Weights.W1RNeg);
				const VectorRegister4Float Weight1RealD2 = VectorLoad(Weights.W1RD2);
				const VectorRegister4Float Weight1RealD3 = VectorLoad(Weights.W1RD3);

				// Perform butterflies.
				for (int32 i = 0; i < NumButterflies; i += 2)
				{
					const int32 Pos0 = 2 * (Stride * ButterflyIndex + i);
					const int32 Pos1 = 2 * (Stride * ButterflyIndex + 1 * NumButterflies + i);
					const int32 Pos2 = 2 * (Stride * ButterflyIndex + 2 * NumButterflies + i);
					const int32 Pos3 = 2 * (Stride * ButterflyIndex + 3 * NumButterflies + i);

					VectorRegister4Float A0 = VectorLoad(&InOutValues[Pos0]);
					VectorRegister4Float A1 = VectorLoad(&InOutValues[Pos1]);
					VectorRegister4Float A2 = VectorLoad(&InOutValues[Pos2]);
					VectorRegister4Float A3 = VectorLoad(&InOutValues[Pos3]);

					VectorRegister4Float A1Swizzle = VectorSwizzle(A1, 1, 0, 3, 2);
					VectorRegister4Float A2Swizzle = VectorSwizzle(A2, 1, 0, 3, 2);
					VectorRegister4Float A3Swizzle = VectorSwizzle(A3, 1, 0, 3, 2);

					VectorRegister4Float B1 = VectorMultiplyAdd(A1Swizzle, Weight1Imag, A1);
					VectorRegister4Float B2 = VectorMultiplyAdd(A2Swizzle, Weight2Imag, A2);
					VectorRegister4Float B3 = VectorMultiplyAdd(A3Swizzle, Weight3Imag, A3);

					VectorRegister4Float C0 = VectorMultiplyAdd(B2, Weight2Real, A0);
					VectorRegister4Float C2 = VectorMultiplyAdd(B2, Weight2RealNeg, A0);
					VectorRegister4Float C1 = VectorMultiplyAdd(B3, Weight3Real, B1);
					VectorRegister4Float C3 = VectorMultiplyAdd(B3, Weight3RealNeg, B1);

					VectorRegister4Float C3Swizzle = VectorSwizzle(C3, 1, 0, 3, 2);

					VectorRegister4Float D0 = VectorMultiplyAdd(C1, Weight1Real, C0);
					VectorRegister4Float D1 = VectorMultiplyAdd(C1, Weight1RealNeg, C0);
					VectorRegister4Float D2 = VectorMultiplyAdd(C3Swizzle, Weight1RealD2, C2);
					VectorRegister4Float D3 = VectorMultiplyAdd(C3Swizzle, Weight1RealD3, C2);

					VectorStore(D0, &InOutValues[Pos0]);
					VectorStore(D1, &InOutValues[Pos1]);
					VectorStore(D2, &InOutValues[Pos2]);
					VectorStore(D3, &InOutValues[Pos3]);
				}
			}

			// Special case of 2nd to last radix4 which has to load new weights for
			// each iteration.
			void Radix4Butterfly2ndToFinal(float* InOutValues, int32 StageIndex)
			{
				int32 NumPasses = 1 << StageIndex;

				// Elements between passes.
				const int32 Stride = 16;

				for (int32 i = 0; i < NumPasses; ++i)
				{
					// Load values for current weight.
					const FRadix4Weight& Weights = Radix4Weights[i];

					const VectorRegister4Float Weight1Real = VectorLoad(Weights.W1R);
					const VectorRegister4Float Weight1Imag = VectorLoad(Weights.W1I);
					const VectorRegister4Float Weight2Real = VectorLoad(Weights.W2R);
					const VectorRegister4Float Weight2Imag = VectorLoad(Weights.W2I);
					const VectorRegister4Float Weight3Real = VectorLoad(Weights.W3R);
					const VectorRegister4Float Weight3Imag = VectorLoad(Weights.W3I);

					const VectorRegister4Float Weight3RealNeg = VectorLoad(Weights.W3RNeg);
					const VectorRegister4Float Weight2RealNeg = VectorLoad(Weights.W2RNeg);
					const VectorRegister4Float Weight1RealNeg = VectorLoad(Weights.W1RNeg);
					const VectorRegister4Float Weight1RealD2 = VectorLoad(Weights.W1RD2);
					const VectorRegister4Float Weight1RealD3 = VectorLoad(Weights.W1RD3);

					for (int32 j = 0; j < 4 ; j += 2)
					{
						const int32 Pos0 = 2 * (Stride * i + j);
						const int32 Pos1 = 2 * (Stride * i + 4 + j);
						const int32 Pos2 = 2 * (Stride * i + 8 + j);
						const int32 Pos3 = 2 * (Stride * i + 12 + j);

						VectorRegister4Float A0 = VectorLoad(&InOutValues[Pos0]);
						VectorRegister4Float A1 = VectorLoad(&InOutValues[Pos1]);
						VectorRegister4Float A2 = VectorLoad(&InOutValues[Pos2]);
						VectorRegister4Float A3 = VectorLoad(&InOutValues[Pos3]);

						VectorRegister4Float A1Swizzle = VectorSwizzle(A1, 1, 0, 3, 2);
						VectorRegister4Float A2Swizzle = VectorSwizzle(A2, 1, 0, 3, 2);
						VectorRegister4Float A3Swizzle = VectorSwizzle(A3, 1, 0, 3, 2);

						VectorRegister4Float B1 = VectorMultiplyAdd(A1Swizzle, Weight1Imag, A1);
						VectorRegister4Float B2 = VectorMultiplyAdd(A2Swizzle, Weight2Imag, A2);
						VectorRegister4Float B3 = VectorMultiplyAdd(A3Swizzle, Weight3Imag, A3);
						
						VectorRegister4Float C0 = VectorMultiplyAdd(B2, Weight2Real, A0);
						VectorRegister4Float C1 = VectorMultiplyAdd(B3, Weight3Real, B1);
						VectorRegister4Float C2 = VectorMultiplyAdd(B2, Weight2RealNeg, A0);
						VectorRegister4Float C3 = VectorMultiplyAdd(B3, Weight3RealNeg, B1);
						VectorRegister4Float C3Swizzle = VectorSwizzle(C3, 1, 0, 3, 2);
						
						VectorRegister4Float D0 = VectorMultiplyAdd(C1, Weight1Real, C0);
						VectorRegister4Float D1 = VectorMultiplyAdd(C1, Weight1RealNeg, C0);
						VectorRegister4Float D2 = VectorMultiplyAdd(C3Swizzle, Weight1RealD2, C2);
						VectorRegister4Float D3 = VectorMultiplyAdd(C3Swizzle, Weight1RealD3, C2);

						VectorStore(D0, &InOutValues[Pos0]);
						VectorStore(D1, &InOutValues[Pos1]);
						VectorStore(D2, &InOutValues[Pos2]);
						VectorStore(D3, &InOutValues[Pos3]);
					}
				}
			}


			// Read data for final butterfly.  Performs part of bit reversal order.
			void ReadFinalButterflyInputs(const float* InValues, int32 InNumButterflies, int32 InReadIndex, FFinalInputs& OutValues)
			{
				const int32 Pos0 = 2 * (0 * InNumButterflies + 4 * InReadIndex);
				const int32 Pos1 = Pos0 + 4;
				const int32 Pos2 = 2 * (2 * InNumButterflies + 4 * InReadIndex);
				const int32 Pos3 = Pos2 + 4;
				const int32 Pos4 = 2 * (1 * InNumButterflies + 4 * InReadIndex);
				const int32 Pos5 = Pos4 + 4;
				const int32 Pos6 = 2 * (3 * InNumButterflies + 4 * InReadIndex);
				const int32 Pos7 = Pos6 + 4;
				
				VectorRegister4Float T0 = VectorLoad(&InValues[Pos0]);
				VectorRegister4Float T1 = VectorLoad(&InValues[Pos1]);
				VectorRegister4Float T2 = VectorLoad(&InValues[Pos2]);
				VectorRegister4Float T3 = VectorLoad(&InValues[Pos3]);
				VectorRegister4Float T4 = VectorLoad(&InValues[Pos4]);
				VectorRegister4Float T5 = VectorLoad(&InValues[Pos5]);
				VectorRegister4Float T6 = VectorLoad(&InValues[Pos6]);
				VectorRegister4Float T7 = VectorLoad(&InValues[Pos7]);

				OutValues.A0 = VectorShuffle(T0, T2, 0, 1, 0, 1);
				OutValues.A1 = VectorShuffle(T4, T6, 0, 1, 0, 1);
				OutValues.A2 = VectorShuffle(T0, T2, 2, 3, 2, 3);
				OutValues.A3 = VectorShuffle(T4, T6, 2, 3, 2, 3);
				OutValues.A4 = VectorShuffle(T1, T3, 0, 1, 0, 1);
				OutValues.A5 = VectorShuffle(T5, T7, 0, 1, 0, 1);
				OutValues.A6 = VectorShuffle(T1, T3, 2, 3, 2, 3);
				OutValues.A7 = VectorShuffle(T5, T7, 2, 3, 2, 3);
			}

			// Write data for final butterfly.  Performs part of bit reversal order.
			void WriteFinalButterflyOutputs(const FFinalOutputs& InResult, int32 InNumButterflies, int32 InWriteIndex, float* OutValues)
			{
				const int32 Pos0 = 2 * (0 * InNumButterflies + 4 * InWriteIndex);
				const int32 Pos1 = Pos0 + 4;
				const int32 Pos2 = 2 * (2 * InNumButterflies + 4 * InWriteIndex);
				const int32 Pos3 = Pos2 + 4;
				const int32 Pos4 = 2 * (1 * InNumButterflies + 4 * InWriteIndex);
				const int32 Pos5 = Pos4 + 4;
				const int32 Pos6 = 2 * (3 * InNumButterflies + 4 * InWriteIndex);
				const int32 Pos7 = Pos6 + 4;

				VectorStore(InResult.D0, &OutValues[Pos0]);
				VectorStore(InResult.D1, &OutValues[Pos1]);
				VectorStore(InResult.D2, &OutValues[Pos2]);
				VectorStore(InResult.D3, &OutValues[Pos3]);
				VectorStore(InResult.D4, &OutValues[Pos4]);
				VectorStore(InResult.D5, &OutValues[Pos5]);
				VectorStore(InResult.D6, &OutValues[Pos6]);
				VectorStore(InResult.D7, &OutValues[Pos7]);
			}


			// Compute butterfly in final stage.
			void Radix4ButterflyFinalIteration(const FFinalInputs& Inputs, const FFinalWeights& InWeights, FFinalOutputs& Outputs)
			{
				// Note: Some weights are altered to bake in sign flips to avoid an extra multiply later on.
				const VectorRegister4Float W2I = VectorLoad(InWeights.W2I);
				const VectorRegister4Float W2R = VectorLoad(InWeights.W2R);
				const VectorRegister4Float W3I = VectorLoad(InWeights.W3I);
				const VectorRegister4Float W3R = VectorLoad(InWeights.W3R);
				const VectorRegister4Float W4I = VectorLoad(InWeights.W4I);
				const VectorRegister4Float W4R = VectorLoad(InWeights.W4R);
				const VectorRegister4Float W5I = VectorLoad(InWeights.W5I);
				const VectorRegister4Float W5R = VectorLoad(InWeights.W5R);
				const VectorRegister4Float W6I = VectorLoad(InWeights.W6I);
				const VectorRegister4Float W6R = VectorLoad(InWeights.W6R);
				const VectorRegister4Float W7I = VectorLoad(InWeights.W7I);
				const VectorRegister4Float W7R = VectorLoad(InWeights.W7R);

				const VectorRegister4Float W2RNeg = VectorLoad(InWeights.W2RNeg);
				const VectorRegister4Float W3RNeg = VectorLoad(InWeights.W3RNeg);
				const VectorRegister4Float W4RNeg = VectorLoad(InWeights.W4RNeg);
				const VectorRegister4Float W5RNeg = VectorLoad(InWeights.W5RNeg);
				const VectorRegister4Float W6RNeg = VectorLoad(InWeights.W6RNeg);
				const VectorRegister4Float W7RNeg = VectorLoad(InWeights.W7RNeg);

				const VectorRegister4Float W2RD4 = VectorLoad(InWeights.W2RD4);
				const VectorRegister4Float W2RD6 = VectorLoad(InWeights.W2RD6);
				const VectorRegister4Float W3RD5 = VectorLoad(InWeights.W3RD5);
				const VectorRegister4Float W3RD7 = VectorLoad(InWeights.W3RD7);

				VectorRegister4Float A2Swizzle = VectorSwizzle(Inputs.A2, 1, 0, 3, 2);
				VectorRegister4Float A3Swizzle = VectorSwizzle(Inputs.A3, 1, 0, 3, 2);
				VectorRegister4Float A4Swizzle = VectorSwizzle(Inputs.A4, 1, 0, 3, 2);
				VectorRegister4Float A5Swizzle = VectorSwizzle(Inputs.A5, 1, 0, 3, 2);
				VectorRegister4Float A6Swizzle = VectorSwizzle(Inputs.A6, 1, 0, 3, 2);
				VectorRegister4Float A7Swizzle = VectorSwizzle(Inputs.A7, 1, 0, 3, 2);

				VectorRegister4Float B2 = VectorMultiplyAdd(A2Swizzle, W2I, Inputs.A2);
				VectorRegister4Float B3 = VectorMultiplyAdd(A3Swizzle, W3I, Inputs.A3);
				VectorRegister4Float B4 = VectorMultiplyAdd(A4Swizzle, W4I, Inputs.A4);
				VectorRegister4Float B5 = VectorMultiplyAdd(A5Swizzle, W5I, Inputs.A5);
				VectorRegister4Float B6 = VectorMultiplyAdd(A6Swizzle, W6I, Inputs.A6);
				VectorRegister4Float B7 = VectorMultiplyAdd(A7Swizzle, W7I, Inputs.A7);

				VectorRegister4Float C0 = VectorMultiplyAdd(B4, W4R, Inputs.A0);
				VectorRegister4Float C1 = VectorMultiplyAdd(B5, W5R, Inputs.A1);
				VectorRegister4Float C2 = VectorMultiplyAdd(B6, W6R, B2);
				VectorRegister4Float C3 = VectorMultiplyAdd(B7, W7R, B3);
				VectorRegister4Float C4 = VectorMultiplyAdd(B4, W4RNeg, Inputs.A0);
				VectorRegister4Float C5 = VectorMultiplyAdd(B5, W5RNeg, Inputs.A1);
				VectorRegister4Float C6 = VectorMultiplyAdd(B6, W6RNeg, B2);
				VectorRegister4Float C7 = VectorMultiplyAdd(B7, W7RNeg, B3);

				VectorRegister4Float C6Swizzle = VectorSwizzle(C6, 1, 0, 3, 2);
				VectorRegister4Float C7Swizzle = VectorSwizzle(C7, 1, 0, 3, 2);

				Outputs.D0 = VectorMultiplyAdd(C2, W2R, C0);
				Outputs.D1 = VectorMultiplyAdd(C3, W3R, C1);
				Outputs.D2 = VectorMultiplyAdd(C2, W2RNeg, C0);
				Outputs.D3 = VectorMultiplyAdd(C3, W3RNeg, C1);
				Outputs.D4 = VectorMultiplyAdd(C6Swizzle, W2RD4, C4);
				Outputs.D5 = VectorMultiplyAdd(C7Swizzle, W3RD5, C5);
				Outputs.D6 = VectorMultiplyAdd(C6Swizzle, W2RD6, C4);
				Outputs.D7 = VectorMultiplyAdd(C7Swizzle, W3RD7, C5);
			}


			// Perform last set of radix 4 butterflies.
			//
			// This method is special since it also performs bit order reversal in a
			// moderately cache coherent manner.
			void Radix4ButterflyFinal(float* InOutValues, int InStageIndex)
			{
				int32 NumButterflies = 1 << InStageIndex;
				int32 NumIterations = NumButterflies >> 2;

				FFinalInputs Inputs;
				FFinalOutputs Outputs;

				int32 Iteration = 0;

				ReadFinalButterflyInputs(InOutValues, NumButterflies, FinalIndices[Iteration].ReadIndex, Inputs);
				Radix4ButterflyFinalIteration(Inputs, FinalWeights[Iteration], Outputs);

				for (Iteration = 1; Iteration < NumIterations; ++Iteration)
				{
					ReadFinalButterflyInputs(InOutValues, NumButterflies, FinalIndices[Iteration].ReadIndex, Inputs);
					WriteFinalButterflyOutputs(Outputs, NumButterflies, FinalIndices[Iteration - 1].WriteIndex, InOutValues);
					Radix4ButterflyFinalIteration(Inputs, FinalWeights[Iteration], Outputs);
				}

				WriteFinalButterflyOutputs(Outputs, NumButterflies, FinalIndices[Iteration - 1].WriteIndex, InOutValues);
			}

			int32 IntLog2(int32 InValue)
			{
				check(InValue > 0);
				check(FMath::CountBits(InValue) == 1);
				return FMath::CountTrailingZeros(InValue);
			}

			void GenerateFinalIndices()
			{
				// Each pass of the final radix 4 operates on 16 complex values.  
				const int32 FinalPassSize = 16;
				const int32 NumFinalIndices = FFTSize / FinalPassSize;

				// We need to ensure that FFTSize is at least 16 or else final
				// FFT pass go past end of buffer.
				check(FFTSize >= FinalPassSize);

				FinalIndices.Reset();
				FinalIndices.AddUninitialized(NumFinalIndices);

				// These indices perform part of the bit order reversal on 16 element boundaries.
				// Need to shift bits to get the bit reversed order on 16 element blocks.
				const int32 Shift = 32 - (IntLog2(FFTSize) - 4);

				int32 Index = 0;

				for (int32 ReadIndex = 0; (ReadIndex < NumFinalIndices) && (Index < NumFinalIndices); ++ReadIndex)
				{
					// Get bit reversed order on 16 element block
					const int32 WriteIndex = ReverseBits(ReadIndex) >> Shift;
					
					// If ReadIndex > WriteIndex, then ReadIndex in a previous iteration had the
					// value WriteIndex has now, and we do not want to repeat it 
					if (ReadIndex == WriteIndex)
					{
						// If equal, add one entry to read and write to same index.
						FinalIndices[Index] = { ReadIndex, WriteIndex };
						Index++;
					}
					else if (ReadIndex < WriteIndex)
					{
						// If ReadIndex < WriteIndex, add table entries in both orders.
						// Loop logic in final radix pass will make sure that nothing
						// gets overwritten.
						FinalIndices[Index] = { ReadIndex, WriteIndex };
						Index++;

						FinalIndices[Index] = { WriteIndex, ReadIndex };
						Index++;
					}
				}
			}

			void GenerateFinalWeights()
			{
				// Each pass of the final radix 4 operates on 16 complex values.  
				const int32 FinalPassSize = 16;
				const int32 NumFinalWeights = FFTSize / FinalPassSize;

				// We need to ensure that FFTSize is at least 16 or else final
				// FFT pass go past end of buffer.
				check(FFTSize >= FinalPassSize);

				FinalWeights.Reset();
				FinalWeights.AddUninitialized(NumFinalWeights);

				const double Scale = 1. / static_cast<double>(FFTSize);

				for (int32 Pass = 0; Pass < NumFinalWeights; ++Pass)
				{
					const int ReadIndex = FinalIndices[Pass].ReadIndex;
					const double RotatedBitFraction = RotateBitsAroundPoint(4 * ReadIndex);

					const double Phase = 2. * PI * (RotatedBitFraction + 0 * Scale);
					const double Phase1 = 2. * PI * (RotatedBitFraction + 1 * Scale);
					const double Phase2 = 2. * PI * (RotatedBitFraction + 2 * Scale);
					const double Phase3 = 2. * PI * (RotatedBitFraction + 3 * Scale);

					FinalWeights[Pass].W2I[0] = -FMath::Tan(Phase);
					FinalWeights[Pass].W2I[1] = FMath::Tan(Phase);
					FinalWeights[Pass].W2I[2] = -FMath::Tan(Phase1);
					FinalWeights[Pass].W2I[3] = FMath::Tan(Phase1);

					FinalWeights[Pass].W3I[0] = -FMath::Tan(Phase2);
					FinalWeights[Pass].W3I[1] = FMath::Tan(Phase2);
					FinalWeights[Pass].W3I[2] = -FMath::Tan(Phase3);
					FinalWeights[Pass].W3I[3] = FMath::Tan(Phase3);

					FinalWeights[Pass].W2R[0] = FMath::Cos(Phase);
					FinalWeights[Pass].W2R[1] = FMath::Cos(Phase);
					FinalWeights[Pass].W2R[2] = FMath::Cos(Phase1);
					FinalWeights[Pass].W2R[3] = FMath::Cos(Phase1);

					FinalWeights[Pass].W2RD4[0] = -FMath::Cos(Phase);
					FinalWeights[Pass].W2RD4[1] = FMath::Cos(Phase);
					FinalWeights[Pass].W2RD4[2] = -FMath::Cos(Phase1);
					FinalWeights[Pass].W2RD4[3] = FMath::Cos(Phase1);

					FinalWeights[Pass].W2RD6[0] = FMath::Cos(Phase);
					FinalWeights[Pass].W2RD6[1] = -FMath::Cos(Phase);
					FinalWeights[Pass].W2RD6[2] = FMath::Cos(Phase1);
					FinalWeights[Pass].W2RD6[3] = -FMath::Cos(Phase1);

					FinalWeights[Pass].W2RNeg[0] = -FMath::Cos(Phase);
					FinalWeights[Pass].W2RNeg[1] = -FMath::Cos(Phase);
					FinalWeights[Pass].W2RNeg[2] = -FMath::Cos(Phase1);
					FinalWeights[Pass].W2RNeg[3] = -FMath::Cos(Phase1);

					FinalWeights[Pass].W3R[0] = FMath::Cos(Phase2);
					FinalWeights[Pass].W3R[1] = FMath::Cos(Phase2);
					FinalWeights[Pass].W3R[2] = FMath::Cos(Phase3);
					FinalWeights[Pass].W3R[3] = FMath::Cos(Phase3);

					FinalWeights[Pass].W3RD5[0] = -FMath::Cos(Phase2);
					FinalWeights[Pass].W3RD5[1] = FMath::Cos(Phase2);
					FinalWeights[Pass].W3RD5[2] = -FMath::Cos(Phase3);
					FinalWeights[Pass].W3RD5[3] = FMath::Cos(Phase3);

					FinalWeights[Pass].W3RD7[0] = FMath::Cos(Phase2);
					FinalWeights[Pass].W3RD7[1] = -FMath::Cos(Phase2);
					FinalWeights[Pass].W3RD7[2] = FMath::Cos(Phase3);
					FinalWeights[Pass].W3RD7[3] = -FMath::Cos(Phase3);

					FinalWeights[Pass].W3RNeg[0] = -FMath::Cos(Phase2);
					FinalWeights[Pass].W3RNeg[1] = -FMath::Cos(Phase2);
					FinalWeights[Pass].W3RNeg[2] = -FMath::Cos(Phase3);
					FinalWeights[Pass].W3RNeg[3] = -FMath::Cos(Phase3);

					FinalWeights[Pass].W4I[0] = -FMath::Tan(Phase + Phase);
					FinalWeights[Pass].W4I[1] = FMath::Tan(Phase + Phase);
					FinalWeights[Pass].W4I[2] = -FMath::Tan(Phase1 + Phase1);
					FinalWeights[Pass].W4I[3] = FMath::Tan(Phase1 + Phase1);

					FinalWeights[Pass].W5I[0] = -FMath::Tan(Phase2 + Phase2);
					FinalWeights[Pass].W5I[1] = FMath::Tan(Phase2 + Phase2);
					FinalWeights[Pass].W5I[2] = -FMath::Tan(Phase3 + Phase3);
					FinalWeights[Pass].W5I[3] = FMath::Tan(Phase3 + Phase3);

					FinalWeights[Pass].W4R[0] = FMath::Cos(Phase + Phase);
					FinalWeights[Pass].W4R[1] = FMath::Cos(Phase + Phase);
					FinalWeights[Pass].W4R[2] = FMath::Cos(Phase1 + Phase1);
					FinalWeights[Pass].W4R[3] = FMath::Cos(Phase1 + Phase1);

					FinalWeights[Pass].W4RNeg[0] = -FMath::Cos(Phase + Phase);
					FinalWeights[Pass].W4RNeg[1] = -FMath::Cos(Phase + Phase);
					FinalWeights[Pass].W4RNeg[2] = -FMath::Cos(Phase1 + Phase1);
					FinalWeights[Pass].W4RNeg[3] = -FMath::Cos(Phase1 + Phase1);

					FinalWeights[Pass].W5R[0] = FMath::Cos(Phase2 + Phase2);
					FinalWeights[Pass].W5R[1] = FMath::Cos(Phase2 + Phase2);
					FinalWeights[Pass].W5R[2] = FMath::Cos(Phase3 + Phase3);
					FinalWeights[Pass].W5R[3] = FMath::Cos(Phase3 + Phase3);

					FinalWeights[Pass].W5RNeg[0] = -FMath::Cos(Phase2 + Phase2);
					FinalWeights[Pass].W5RNeg[1] = -FMath::Cos(Phase2 + Phase2);
					FinalWeights[Pass].W5RNeg[2] = -FMath::Cos(Phase3 + Phase3);
					FinalWeights[Pass].W5RNeg[3] = -FMath::Cos(Phase3 + Phase3);

					FinalWeights[Pass].W6I[0] = -FMath::Tan(3. * Phase);
					FinalWeights[Pass].W6I[1] = FMath::Tan(3. * Phase);
					FinalWeights[Pass].W6I[2] = -FMath::Tan(3. * Phase1);
					FinalWeights[Pass].W6I[3] = FMath::Tan(3. * Phase1);

					FinalWeights[Pass].W7I[0] = -FMath::Tan(3. * Phase2);
					FinalWeights[Pass].W7I[1] = FMath::Tan(3. * Phase2);
					FinalWeights[Pass].W7I[2] = -FMath::Tan(3. * Phase3);
					FinalWeights[Pass].W7I[3] = FMath::Tan(3. * Phase3);

					FinalWeights[Pass].W6R[0] = 2 * FMath::Cos(Phase + Phase) - 1;
					FinalWeights[Pass].W6R[1] = 2 * FMath::Cos(Phase + Phase) - 1;
					FinalWeights[Pass].W6R[2] = 2 * FMath::Cos(Phase1 + Phase1) - 1;
					FinalWeights[Pass].W6R[3] = 2 * FMath::Cos(Phase1 + Phase1) - 1; 

					FinalWeights[Pass].W6RNeg[0] = -(2 * FMath::Cos(Phase + Phase) - 1);
					FinalWeights[Pass].W6RNeg[1] = -(2 * FMath::Cos(Phase + Phase) - 1);
					FinalWeights[Pass].W6RNeg[2] = -(2 * FMath::Cos(Phase1 + Phase1) - 1);
					FinalWeights[Pass].W6RNeg[3] = -(2 * FMath::Cos(Phase1 + Phase1) - 1); 

					FinalWeights[Pass].W7R[0] = 2 * FMath::Cos(Phase2 + Phase2) - 1;
					FinalWeights[Pass].W7R[1] = 2 * FMath::Cos(Phase2 + Phase2) - 1;
					FinalWeights[Pass].W7R[2] = 2 * FMath::Cos(Phase3 + Phase3) - 1;
					FinalWeights[Pass].W7R[3] = 2 * FMath::Cos(Phase3 + Phase3) - 1;

					FinalWeights[Pass].W7RNeg[0] = -(2 * FMath::Cos(Phase2 + Phase2) - 1);
					FinalWeights[Pass].W7RNeg[1] = -(2 * FMath::Cos(Phase2 + Phase2) - 1);
					FinalWeights[Pass].W7RNeg[2] = -(2 * FMath::Cos(Phase3 + Phase3) - 1);
					FinalWeights[Pass].W7RNeg[3] = -(2 * FMath::Cos(Phase3 + Phase3) - 1);
				}
			}

			void GenerateRadix4Weights()
			{
				const int32 Radix4PassSize = 16;
				check(FFTSize >= Radix4PassSize);

				// Each radix-4 butterfly
				int32 MaxNumButterfliesInStage = FFTSize / Radix4PassSize;

				Radix4Weights.Reset();
				Radix4Weights.AddUninitialized(MaxNumButterfliesInStage);

				for (int32 ButterflyIndex = 0; ButterflyIndex < MaxNumButterfliesInStage; ++ButterflyIndex)
				{
					const double Phase = 2. * PI * RotateBitsAroundPoint(4 * ButterflyIndex);

					const float W1R = static_cast<float>(FMath::Cos(Phase));
					const float W1I = static_cast<float>(FMath::Tan(Phase));
					const float W2R = static_cast<float>(FMath::Cos(Phase + Phase));
					const float W2I = static_cast<float>(FMath::Tan(Phase + Phase));
					const float W3R = static_cast<float>(2. * W2R - 1.);
					const float W3I = static_cast<float>(FMath::Tan(3. * Phase));


					Radix4Weights[ButterflyIndex].W1R[0] = W1R;
					Radix4Weights[ButterflyIndex].W1I[0] = -W1I;
					Radix4Weights[ButterflyIndex].W2R[0] = W2R;
					Radix4Weights[ButterflyIndex].W2I[0] = -W2I;
					Radix4Weights[ButterflyIndex].W3R[0] = W3R;
					Radix4Weights[ButterflyIndex].W3I[0] = -W3I;

					Radix4Weights[ButterflyIndex].W1RNeg[0] = -W1R;
					Radix4Weights[ButterflyIndex].W1RD2[0] = -W1R;
					Radix4Weights[ButterflyIndex].W1RD3[0] = W1R;
					Radix4Weights[ButterflyIndex].W2RNeg[0] = -W2R;
					Radix4Weights[ButterflyIndex].W3RNeg[0] = -W3R;


					Radix4Weights[ButterflyIndex].W1R[1] = W1R;
					Radix4Weights[ButterflyIndex].W1I[1] = W1I;
					Radix4Weights[ButterflyIndex].W2R[1] = W2R;
					Radix4Weights[ButterflyIndex].W2I[1] = W2I;
					Radix4Weights[ButterflyIndex].W3R[1] = W3R;
					Radix4Weights[ButterflyIndex].W3I[1] = W3I;

					Radix4Weights[ButterflyIndex].W1RNeg[1] = -W1R;
					Radix4Weights[ButterflyIndex].W1RD2[1] = W1R;
					Radix4Weights[ButterflyIndex].W1RD3[1] = -W1R;
					Radix4Weights[ButterflyIndex].W2RNeg[1] = -W2R;
					Radix4Weights[ButterflyIndex].W3RNeg[1] = -W3R;


					Radix4Weights[ButterflyIndex].W1R[2] = W1R;
					Radix4Weights[ButterflyIndex].W1I[2] = -W1I;
					Radix4Weights[ButterflyIndex].W2R[2] = W2R;
					Radix4Weights[ButterflyIndex].W2I[2] = -W2I;
					Radix4Weights[ButterflyIndex].W3R[2] = W3R;
					Radix4Weights[ButterflyIndex].W3I[2] = -W3I;

					Radix4Weights[ButterflyIndex].W1RNeg[2] = -W1R;
					Radix4Weights[ButterflyIndex].W1RD2[2] = -W1R;
					Radix4Weights[ButterflyIndex].W1RD3[2] = W1R;
					Radix4Weights[ButterflyIndex].W2RNeg[2] = -W2R;
					Radix4Weights[ButterflyIndex].W3RNeg[2] = -W3R;


					Radix4Weights[ButterflyIndex].W1R[3] = W1R;
					Radix4Weights[ButterflyIndex].W1I[3] = W1I;
					Radix4Weights[ButterflyIndex].W2R[3] = W2R;
					Radix4Weights[ButterflyIndex].W2I[3] = W2I;
					Radix4Weights[ButterflyIndex].W3R[3] = W3R;
					Radix4Weights[ButterflyIndex].W3I[3] = W3I;

					Radix4Weights[ButterflyIndex].W1RNeg[3] = -W1R;
					Radix4Weights[ButterflyIndex].W1RD2[3] = W1R;
					Radix4Weights[ButterflyIndex].W1RD3[3] = -W1R;
					Radix4Weights[ButterflyIndex].W2RNeg[3] = -W2R;
					Radix4Weights[ButterflyIndex].W3RNeg[3] = -W3R;
				}
			}

			// A funny but useful function which reverses bits and
			// places them behind a point.  For example
			// 0101 is transformed to 0.1010
			double RotateBitsAroundPoint(uint32 InValue)
			{
				uint32 ReversedValue = ReverseBits(InValue);
				double OutValue = ReversedValue / 4294967296.;
				OutValue = 1. / 4294967296. * ReversedValue;
				return OutValue;
			}

			uint32 ReverseBits(uint32 InValue)
			{
				static const uint8 ByteReversal[256] = {
					 0, 128, 64, 192, 32, 160, 96, 224, 16, 144, 80, 208, 48, 176, 112, 240,
					 8, 136, 72, 200, 40, 168, 104, 232, 24, 152, 88, 216, 56, 184, 120, 248,
					 4, 132, 68, 196, 36, 164, 100, 228, 20, 148, 84, 212, 52, 180, 116, 244,
					 12, 140, 76, 204, 44, 172, 108, 236, 28, 156, 92, 220, 60, 188, 124, 252,
					 2, 130, 66, 194, 34, 162, 98, 226, 18, 146, 82, 210, 50, 178, 114, 242,
					 10, 138, 74, 202, 42, 170, 106, 234, 26, 154, 90, 218, 58, 186, 122, 250,
					6, 134, 70, 198, 38, 166, 102, 230, 22, 150, 86, 214, 54, 182, 118, 246,
					 14, 142, 78, 206, 46, 174, 110, 238, 30, 158, 94, 222, 62, 190, 126, 254,
					 1, 129, 65, 193, 33, 161, 97, 225, 17, 145, 81, 209, 49, 177, 113, 241,
					 9, 137, 73, 201, 41, 169, 105, 233, 25, 153, 89, 217, 57, 185, 121, 249,
					 5, 133, 69, 197, 37, 165, 101, 229, 21, 149, 85, 213, 53, 181, 117, 245,
					 13, 141, 77, 205, 45, 173, 109, 237, 29, 157, 93, 221, 61, 189, 125, 253,
					 3, 131, 67, 195, 35, 163, 99, 227, 19, 147, 83, 211, 51, 179, 115, 243,
					 11, 139, 75, 203, 43, 171, 107, 235, 27, 155, 91, 219, 59, 187, 123, 251,
					 7, 135, 71, 199, 39, 167, 103, 231, 23, 151, 87, 215, 55, 183, 119, 247,
					 15, 143, 79, 207, 47, 175, 111, 239, 31, 159, 95, 223, 63, 191, 127, 255
				};

				uint8 Byte0 = ByteReversal[InValue >> 0*8 & 0xff];
				uint8 Byte1 = ByteReversal[InValue >> 1*8 & 0xff];
				uint8 Byte2 = ByteReversal[InValue >> 2*8 & 0xff];
				uint8 Byte3 = ByteReversal[InValue >> 3*8 & 0xff];

				uint32 OutValue = Byte0 << 3*8 | Byte1 << 2*8 | Byte2 << 1*8 | Byte3 << 0*8;
				return OutValue;
			}
			
			int32 Log2FFTSize;
			int32 FFTSize;
			int32 NumFloats;

			TArray<FRadix4Weight> Radix4Weights;
			TArray<FFinalIndices> FinalIndices;
			TArray<FFinalWeights> FinalWeights;

			FAlignedFloatBuffer InverseWorkBuffer;
	};

	// Maximum log 2 size of fft
	const int32 FVectorRealToComplexFFT::MinLog2FFTSize = FVectorComplexFFT::MinLog2FFTSize + 1;

	// Maximum log 2 size of fft
	const int32 FVectorRealToComplexFFT::MaxLog2FFTSize = FVectorComplexFFT::MaxLog2FFTSize + 1;

	void FVectorRealToComplexFFT::InitRealSequenceConversionBuffers()
	{
		// Conversion buffers for performing a real valued FFT using a complex fft
		// The values in the buffer are setup to support SIMD operations resulting
		// in some duplicate data.
		ForwardConvBuffers.AlphaReal.AddUninitialized(FFTSize);
		ForwardConvBuffers.AlphaImag.AddUninitialized(FFTSize);
		ForwardConvBuffers.BetaReal.AddUninitialized(FFTSize);
		ForwardConvBuffers.BetaImag.AddUninitialized(FFTSize);

		InverseConvBuffers.AlphaReal.AddUninitialized(FFTSize);
		InverseConvBuffers.AlphaImag.AddUninitialized(FFTSize);
		InverseConvBuffers.BetaReal.AddUninitialized(FFTSize);
		InverseConvBuffers.BetaImag.AddUninitialized(FFTSize);

		float* AlphaRealForwardBufferData = ForwardConvBuffers.AlphaReal.GetData();
		float* AlphaImagForwardBufferData = ForwardConvBuffers.AlphaImag.GetData();
		float* BetaRealForwardBufferData = ForwardConvBuffers.BetaReal.GetData();
		float* BetaImagForwardBufferData = ForwardConvBuffers.BetaImag.GetData();

		float* AlphaRealInverseBufferData = InverseConvBuffers.AlphaReal.GetData();
		float* AlphaImagInverseBufferData = InverseConvBuffers.AlphaImag.GetData();
		float* BetaRealInverseBufferData = InverseConvBuffers.BetaReal.GetData();
		float* BetaImagInverseBufferData = InverseConvBuffers.BetaImag.GetData();

		float PhaseIncrement = PI / static_cast<float>(FFTSize);

		for (int32 i = 0; i < FFTSize; i += 2)
		{
			const float Phase = PhaseIncrement * i;
			const float BetaReal = 0.5 * (1. - FMath::Sin(Phase));
			const float BetaImag = -0.5 * FMath::Cos(Phase);
			const float AlphaReal = 0.5 * (1. + FMath::Sin(Phase));
			const float AlphaImag = 0.5 * FMath::Cos(Phase);

			AlphaRealForwardBufferData[i] 		= AlphaReal;
			AlphaRealForwardBufferData[i + 1] 	= -AlphaReal;// Sign flipped to simplify SIMD math
			AlphaImagForwardBufferData[i] 		= AlphaImag; 
			AlphaImagForwardBufferData[i + 1] 	= AlphaImag;

			BetaRealForwardBufferData[i] 		= BetaReal;
			BetaRealForwardBufferData[i + 1] 	= BetaReal;
			BetaImagForwardBufferData[i] 		= -BetaImag; // Sign flipped to simplify SIMD math 
			BetaImagForwardBufferData[i + 1] 	= BetaImag;
			
			AlphaRealInverseBufferData[i] 		= AlphaReal;
			AlphaRealInverseBufferData[i + 1] 	= -AlphaReal; // Sign flipped to simplify SIMD math 
			AlphaImagInverseBufferData[i] 		= AlphaImag;
			AlphaImagInverseBufferData[i + 1] 	= AlphaImag; 

			BetaRealInverseBufferData[i] 		= BetaReal;
			BetaRealInverseBufferData[i + 1] 	= BetaReal;
			BetaImagInverseBufferData[i] 		= -BetaImag; // Sign flipped to simplify SIMD math 
			BetaImagInverseBufferData[i + 1] 	= BetaImag; 
		}
	}

	// Performs conversion of buffers required to do real fft using complex fft.
	void FVectorRealToComplexFFT::ConvertSequence(const FConversionBuffers& InBuffers, const float* RESTRICT InValues, int32 InStartIndex, float* RESTRICT OutValues)
	{
		const float* AlphaRealData = InBuffers.AlphaReal.GetData();
		const float* AlphaImagData = InBuffers.AlphaImag.GetData();
		const float* BetaRealData = InBuffers.BetaReal.GetData();
		const float* BetaImagData = InBuffers.BetaImag.GetData();

		if (FFTSize > InStartIndex)
		{
			VectorRegister4Float VInRev1 = VectorLoad(&InValues[FFTSize - InStartIndex]);

			for (int32 i = InStartIndex; i < FFTSize; i += 4)
			{
				VectorRegister4Float VIn = VectorLoad(&InValues[i]);
				VectorRegister4Float VInRISwap = VectorSwizzle(VIn, 1, 0, 3, 2);

				VectorRegister4Float VInRev2 = VectorLoad(&InValues[FFTSize - i - 4]);
				VectorRegister4Float VInRev = VectorShuffle(VInRev1, VInRev2, 0, 1, 2, 3);
				VInRev1 = VInRev2;

				VectorRegister4Float VInRevRISwap = VectorSwizzle(VInRev, 1, 0, 3, 2);

				VectorRegister4Float VAlphaReal = VectorLoad(&AlphaRealData[i]);
				VectorRegister4Float VAlphaImag = VectorLoad(&AlphaImagData[i]);
				VectorRegister4Float VBetaReal = VectorLoad(&BetaRealData[i]);
				VectorRegister4Float VBetaImag = VectorLoad(&BetaImagData[i]);

				// Out1 = [ R * Ar,  I * Ar]
				// Out2 = [ I * Ai,  R * Ai]
				// Out3 = [NR * Br, NI * Br]
				// Out4 = [NI * Bi, NR * Bi]
				//VectorRegister4Float Out1 = VectorMultiply(VIn, VAlphaReal);
				VectorRegister4Float Out2 = VectorMultiply(VInRISwap, VAlphaImag);
				//VectorRegister4Float Out3 = VectorMultiply(VInRev, VBetaReal);
				VectorRegister4Float Out4 = VectorMultiply(VInRevRISwap, VBetaImag);

				// Out12 = [(R * Ar) + (I * Ai), (I * Ar) + (R * Ai)]
				VectorRegister4Float Out12 = VectorMultiplyAdd(VIn, VAlphaReal, Out2);
				// Out34 = [(NR * Br) + (NI * Bi), (NR * Bi) + (NI * Br)]
				VectorRegister4Float Out34 = VectorMultiplyAdd(VInRev, VBetaReal, Out4);

				// Out = [
				// 	(R * Ar) + (I * Ai) + (NR * Br) + (NI * Bi),
				// 	(I * Ar) + (R * Ai) + (NR * Bi) + (NI * Br)
				// ]
				VectorRegister4Float Out = VectorAdd(Out12, Out34);
				VectorStore(Out, &OutValues[i]);
			}
		}
	}

	FVectorRealToComplexFFT::FVectorRealToComplexFFT(int32 InLog2FFTSize)
	:	FFTSize(1 << InLog2FFTSize)
	,	Log2FFTSize(InLog2FFTSize)
	,	ComplexFFT(new FVectorComplexFFT(InLog2FFTSize - 1)) // Utilize a N/2 length complex fft to perform N length fft.
	{
		WorkBuffer.AddUninitialized(FFTSize);

		InitRealSequenceConversionBuffers();
	}

	FVectorRealToComplexFFT::~FVectorRealToComplexFFT()
	{
	}

	int32 FVectorRealToComplexFFT::Size() const
	{
		return FFTSize;
	}

	/** Scaling applied when performing forward FFT. */
	EFFTScaling FVectorRealToComplexFFT::ForwardScaling() const
	{
		return EFFTScaling::MultipliedBySqrtFFTSize;
	}

	/** Scaling applied when performing inverse FFT. */
	EFFTScaling FVectorRealToComplexFFT::InverseScaling() const
	{
		return EFFTScaling::DividedBySqrtFFTSize;
	}

	void FVectorRealToComplexFFT::ForwardRealToComplex(const float* RESTRICT InReal, float* RESTRICT OutComplex)
	{
		// Performs a N sized real-to-complex FFT using an N/2 complex-to-complex FFT.
		float* WorkData = WorkBuffer.GetData();

		ComplexFFT->ForwardComplexToComplex(InReal, WorkData);

		const float* AlphaRealForwardData = ForwardConvBuffers.AlphaReal.GetData();
		const float* AlphaImagForwardData = ForwardConvBuffers.AlphaImag.GetData();
		const float* BetaRealForwardData = ForwardConvBuffers.BetaReal.GetData();
		const float* BetaImagForwardData = ForwardConvBuffers.BetaImag.GetData();

		// Handle special case of this math to account for cyclical index math.
		OutComplex[0] = (WorkData[0] * AlphaRealForwardData[0])
			+ (WorkData[1] * AlphaImagForwardData[0])
			+ (WorkData[0] * BetaRealForwardData[0])
			+ (WorkData[1] * BetaImagForwardData[0]);

		OutComplex[1] = (WorkData[1] * AlphaRealForwardData[1])
			+ (WorkData[0] * AlphaImagForwardData[1])
			+ (WorkData[0] * BetaImagForwardData[1])
			+ (WorkData[1] * BetaRealForwardData[1]);

		OutComplex[2] = (WorkData[2] * AlphaRealForwardData[2])
			+ (WorkData[3] * AlphaImagForwardData[2])
			+ (WorkData[FFTSize - 2] * BetaRealForwardData[2])
			+ (WorkData[FFTSize - 1] * BetaImagForwardData[2]);

		OutComplex[3] = (WorkData[3] * AlphaRealForwardData[3])
			+ (WorkData[2] * AlphaImagForwardData[3])
			+ (WorkData[FFTSize - 2] * BetaImagForwardData[3])
			+ (WorkData[FFTSize - 1] * BetaRealForwardData[3]);

		// Convert all other values using optimized SIMD
		ConvertSequence(ForwardConvBuffers, WorkData, 4, OutComplex);

		// Handle special case of nyquist frequency
		OutComplex[FFTSize] = WorkData[0] - WorkData[1];
		OutComplex[FFTSize + 1] = 0.f;
	}

	void FVectorRealToComplexFFT::InverseComplexToReal(const float* RESTRICT InComplex, float* RESTRICT OutReal)
	{
		// Performs a N sized complex-to-real FFT using an N/2 complex-to-complex FFT.

		float* WorkData = WorkBuffer.GetData();

		const float* AlphaRealInverseData = InverseConvBuffers.AlphaReal.GetData();
		const float* AlphaImagInverseData = InverseConvBuffers.AlphaImag.GetData();
		const float* BetaRealInverseData = InverseConvBuffers.BetaReal.GetData();
		const float* BetaImagInverseData = InverseConvBuffers.BetaImag.GetData();

		// Handle special case of this math to account for cyclical index math.
		WorkData[0] = (InComplex[0] * AlphaRealInverseData[0]) 
			+ (InComplex[1] * AlphaImagInverseData[0]) 
			+ (InComplex[FFTSize] * BetaRealInverseData[0]) 
			+ (InComplex[FFTSize + 1] * BetaImagInverseData[0]);

		WorkData[1] = (InComplex[1] * AlphaRealInverseData[1]) 
			+ (InComplex[0] * AlphaImagInverseData[1]) 
			+ (InComplex[FFTSize] * BetaImagInverseData[1]) 
			+ (InComplex[FFTSize + 1] * BetaRealInverseData[0]);

		WorkData[2] = (InComplex[2] * AlphaRealInverseData[2])
			+ (InComplex[3] * AlphaImagInverseData[2])
			+ (InComplex[FFTSize - 2] * BetaRealInverseData[2])
			+ (InComplex[FFTSize - 1] * BetaImagInverseData[2]);

		WorkData[3] = (InComplex[3] * AlphaRealInverseData[3]) 
			+ (InComplex[2] * AlphaImagInverseData[3]) 
			+ (InComplex[FFTSize - 2] * BetaImagInverseData[3]) 
			+ (InComplex[FFTSize - 1] * BetaRealInverseData[3]);

		// Convert all other values using optimized SIMD
		ConvertSequence(ForwardConvBuffers, InComplex, 4, WorkData);

		// Perform Inverse FFT 
		ComplexFFT->InverseComplexToComplex(WorkData, OutReal);
	}

	void FVectorRealToComplexFFT::BatchForwardRealToComplex(int32 InCount, const float* const RESTRICT InReal[], float* RESTRICT OutComplex[])
	{
		for (int32 i = 0; i < InCount; i++)
		{
			ForwardRealToComplex(InReal[i], OutComplex[i]);
		}
	}

	void FVectorRealToComplexFFT::BatchInverseComplexToReal(int32 InCount, const float* const RESTRICT InComplex[], float* RESTRICT OutReal[])
	{
		for (int32 i = 0; i < InCount; i++)
		{
			InverseComplexToReal(InComplex[i], OutReal[i]);
		}
	}

	/*************************************************************************************************/
	/**************************************** FVectorFFTFactory **************************************/
	/*************************************************************************************************/
	FVectorFFTFactory::~FVectorFFTFactory()
	{
	}

	/** Name of this particular factory. */
	FName FVectorFFTFactory::GetFactoryName() const
	{
		static const FName FactoryName = FName("FVectorFFTFactory");
		return FactoryName;
	}

	/** If true, this implementation uses hardware acceleration. */
	bool FVectorFFTFactory::IsHardwareAccelerated() const
	{
		return false;
	}

	/** If true, this implementation requires input and output arrays to be 128 bit aligned. */
	bool FVectorFFTFactory::Expects128BitAlignedArrays() const
	{
		return false;
	}

	/** Returns true if the input settings are supported by this factory. */
	bool FVectorFFTFactory::AreFFTSettingsSupported(const FFFTSettings& InSettings) const
	{
		// Supports FFT sizes of 5 to 16, though an FFT 
		bool bIsMinSizeSupported = InSettings.Log2Size >= FVectorRealToComplexFFT::MinLog2FFTSize;
		bool bIsMaxSizeSupported = InSettings.Log2Size <= FVectorRealToComplexFFT::MaxLog2FFTSize;
		bool bIsAlignmentSupported = InSettings.bArrays128BitAligned;

		return bIsMinSizeSupported && bIsAlignmentSupported && bIsMaxSizeSupported;
	}

	/** Creates a new FFT algorithm. */
	TUniquePtr<IFFTAlgorithm> FVectorFFTFactory::NewFFTAlgorithm(const FFFTSettings& InSettings)
	{
		if (AreFFTSettingsSupported(InSettings))
		{
			return MakeUnique<FVectorRealToComplexFFT>(InSettings.Log2Size);
		}
		return TUniquePtr<IFFTAlgorithm>();
	}
}
