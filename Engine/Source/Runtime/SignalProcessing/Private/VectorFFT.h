// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/FFTAlgorithm.h"
#include "DSP/BufferVectorOperations.h"

namespace Audio
{
	// Forward declaration
	class FVectorComplexFFT;

	class FVectorRealToComplexFFT : public IFFTAlgorithm
	{
		public:
			/** Minimum log2 size of fft. */
			static const int32 MinLog2FFTSize;

			/** Maximum log 2 size of fft */
			static const int32 MaxLog2FFTSize;

			/** Construct a FVectorRealToComplex FFT
			 *
			 * @param InLog2Size - Determines the size of the FFT.  FFTSize = 2^InLog2Size.
			 */
			FVectorRealToComplexFFT(int32 InLog2Size);

			virtual ~FVectorRealToComplexFFT();

			/** Number of elements in FFT */
			virtual int32 Size() const override;

			/** Scaling applied when performing forward FFT. */
			virtual EFFTScaling ForwardScaling() const override;

			/** Scaling applied when performing inverse FFT. */
			virtual EFFTScaling InverseScaling() const override;

			
			/** ForwardRealToComplex
			 *
			 * Perform the forward FFT on real data and produce complex data.
			 *
			 * @param InReal - Array of floats to input into fourier transform. Must have FFTSize() elements.
			 * @param OutComplex - Array of floats to store output of fourier transform. Must have (FFTSize() + 2) float elements which represent ((FFTSize() / 2) + 1) complex numbers in interleaved format.
			 */
			virtual void ForwardRealToComplex(const float* RESTRICT InReal, float* RESTRICT OutComplex) override;

			/** InverseComplexToReal
			 *
			 * Perform the inverse FFT on complex data and produce real data.
			 *
			 * @param InComplex - Array of floats to input into inverse fourier transform. Must have (FFTSize() + 2) float elements which represent ((FFTSize() / 2) + 1) complex numbers in interleaved format.
			 * @param OutReal - Array of floats to store output of inverse fourier transform. Must have FFTSize() elements.
			 */
			virtual void InverseComplexToReal(const float* RESTRICT InComplex, float* RESTRICT OutReal) override;

			/** BatchForwardRealToComplex
			 *
			 * Perform the forward FFT on real data and produce complex data.
			 *
			 * @param InCount - The number of transforms to compute.
			 * @param InReal - Array of array of floats to input into fourier transform. Must have InCount elements, each containing FFTSize() elements.
			 * @param OutComplex - Array of array of floats to store output of fourier transform. Must have InCount elements, each with (FFTSize() + 2) float elements which represent ((FFTSize() / 2) + 1) complex numbers in interleaved format.
			 */
			virtual void BatchForwardRealToComplex(int32 InCount, const float* const RESTRICT InReal[], float* RESTRICT OutComplex[]) override;

			/** BatchInverseComplexToReal
			 *
			 * Perform the inverse FFT on complex data and produce real data.
			 *
			 * @param InCount - The number of transforms to compute.
			 * @param InComplex - Array of array of floats to input into inverse fourier transform. Must have InCount elements, each with (FFTSize() + 2) float elements which represent ((FFTSize() / 2) + 1) complex numbers in interleaved format.
			 * @param OutReal - Array of array of floats to store output of inverse fourier transform. Must have InCount elements, each with FFTSize() float elements.
			 */
			virtual void BatchInverseComplexToReal(int32 InCount, const float* const RESTRICT InComplex[], float* RESTRICT OutReal[]) override;

		private:

			// Needed to convert buffers for implementation of real-to-complex fft using complex-to-comlex fft.
			struct FConversionBuffers
			{
				FAlignedFloatBuffer AlphaReal;
				FAlignedFloatBuffer AlphaImag;
				FAlignedFloatBuffer BetaReal;
				FAlignedFloatBuffer BetaImag;
			};


			// Initializes conversion buffers
			void InitRealSequenceConversionBuffers();

			// Converts a buffer
			void ConvertSequence(const FConversionBuffers& InBuffers, const float* RESTRICT InValues, int32 InStartIndex, float* RESTRICT OutValues);

			int32 FFTSize;
			int32 Log2FFTSize;

			FAlignedFloatBuffer WorkBuffer;

			FConversionBuffers ForwardConvBuffers;
			FConversionBuffers InverseConvBuffers;

			// Complex FFT implementation.
			TUniquePtr<FVectorComplexFFT> ComplexFFT;
	};

	/** Algorithm factory for FVectorFFT */
	class FVectorFFTFactory : public IFFTAlgorithmFactory
	{
		public:
			virtual ~FVectorFFTFactory();

			/** Name of this particular factory. */
			virtual FName GetFactoryName() const override;

			/** If true, this implementation uses hardware acceleration. */
			virtual bool IsHardwareAccelerated() const override;

			/** If true, this implementation requires input and output arrays to be 128 bit aligned. */
			virtual bool Expects128BitAlignedArrays() const override;

			/** Returns true if the input settings are supported by this factory. */
			virtual bool AreFFTSettingsSupported(const FFFTSettings& InSettings) const override;

			/** Creates a new FFT algorithm. */
			virtual TUniquePtr<IFFTAlgorithm> NewFFTAlgorithm(const FFFTSettings& InSettings) override;
	};
}
