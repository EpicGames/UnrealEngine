// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/VectorLinearResampler.h"

//
// There's a lot here that's about making sure we never have
// to convert to float in the loop. So we keep to advancing
// states - the position in INT space, and the fraction in
// VECTOR space. The difficulty is we need to mask off the
// upper bits as we advance the fraction. The trickery is
// to do this in the actual IEEE float representation. Since
// the most the step can advance is by 65535.0f / 65536.0f,
// we can place the addition operation in an area of the float
// space that prevents the exponent from changing in response
// to that addition. In this case, [2, 3). Addition can only
// ever change the highest mantissa bit, so we can just directly
// mask off that single bit (0xffbfffff).
//
// To do this, we have to take advantage of the mod math identity:
// (a + b) mod c == ((a mod c) + (b mod c)) mod c.
// so we precompute the frac steps, all modded off (0xffff), and the
// step advance vector, also modded off, then scale them all down to [0, 1).
//
// Now, getting the lerp fraction is an AND, and a SUB to get from [2,3) to
// [0, 1).
//
// On top of that, we use 64 bit loads to get the adjacent samples
// for lerping.
//
uint32 Audio::FVectorLinearResampler::ResampleMono(uint32 OutputFramesNeeded, uint32 FixedPointSampleRate, float const* SourceFrames, float* OutputFrames)
{
	// we CANT mask off the upper bits, as it's possible the fraction could
	// jump an entire sample and we need to retain that offset.
	const uint32 FarthestPositionFixed = CurrentFrameFraction + (OutputFramesNeeded - 1) * FixedPointSampleRate;
	const uint32 NextRunStartPositionFixed = FarthestPositionFixed + FixedPointSampleRate;

	float const* LeftSamples = SourceFrames;
	uint32 OutputFrame = 0;
	uint32 CurrentFrameFixed = CurrentFrameFraction;

#if (PLATFORM_ENABLE_VECTORINTRINSICS || PLATFORM_ENABLE_VECTORINTRINSICS_NEON) // SIMD path.
	{

		float CurrentFrameFractionFloat = (float)(CurrentFrameFraction & 0xffff);

		VectorRegister4Float ScaleVec = VectorSetFloat1(1.0f / 65536.0f);
		VectorRegister4Float TwoVec = VectorSetFloat1(2.0f);
		VectorRegister4Float OneVec = VectorSetFloat1(1.0f);
		VectorRegister4Float ThreeVec = VectorSetFloat1(3.0f);

		VectorRegister4Float CurFracStepVec = VectorSet(
			(float)((0 * FixedPointSampleRate) & 0xffff),
			(float)((1 * FixedPointSampleRate) & 0xffff),
			(float)((2 * FixedPointSampleRate) & 0xffff),
			(float)((3 * FixedPointSampleRate) & 0xffff));

		VectorRegister4Float CurFracVec = VectorSetFloat1(CurrentFrameFractionFloat);
		CurFracVec = VectorAdd(CurFracVec, CurFracStepVec);
		CurFracVec = VectorMultiply(CurFracVec, ScaleVec);
		CurFracVec = VectorAdd(CurFracVec, TwoVec);

		VectorRegister4Float MaskVec = MakeVectorRegisterFloat(0xffbfffff, 0xffbfffff, 0xffbfffff, 0xffbfffff);

		VectorRegister4Float StepVec = VectorSetFloat1((float)((FixedPointSampleRate * 4) & 0xffff));
		StepVec = VectorMultiply(StepVec, ScaleVec);

		uint32 OutputFramesNeededSIMD = OutputFramesNeeded & ~3;

		for (; OutputFrame < OutputFramesNeededSIMD; OutputFrame += 4)
		{
			uint32 SourceOffsets[4] = {
				CurrentFrameFixed >> 16,
				(CurrentFrameFixed + FixedPointSampleRate) >> 16,
				(CurrentFrameFixed + FixedPointSampleRate + FixedPointSampleRate) >> 16,
				(CurrentFrameFixed + FixedPointSampleRate + FixedPointSampleRate + FixedPointSampleRate) >> 16
			};

			// [0, 0+1, 1, 1+1]
			VectorRegister4Float LeftSamples01 = VectorLoadTwoPairsFloat(LeftSamples + SourceOffsets[0], LeftSamples + SourceOffsets[1]);
			// [2, 2+1, 3, 3+1]
			VectorRegister4Float LeftSamples23 = VectorLoadTwoPairsFloat(LeftSamples + SourceOffsets[2], LeftSamples + SourceOffsets[3]);

			// [0, 1, 2, 3]
			// [0+1, 1+1, 2+1, 3+1]
			VectorRegister4Float LeftSamplesFrom, LeftSamplesTo;
			VectorDeinterleave(LeftSamplesFrom, LeftSamplesTo, LeftSamples01, LeftSamples23);

			// our lerp vector is CurFracVec, masking off the 1 bit in the mantissa, subtract 2.
			CurFracVec = VectorBitwiseAnd(CurFracVec, MaskVec);
			VectorRegister4Float LerpFactor = VectorSubtract(CurFracVec, TwoVec);
			VectorRegister4Float OneMinusLerpFactor = VectorSubtract(ThreeVec, CurFracVec);

			CurFracVec = VectorAdd(CurFracVec, StepVec);
			VectorRegister4Float OutputVec = VectorAdd(VectorMultiply(LeftSamplesFrom, OneMinusLerpFactor), VectorMultiply(LeftSamplesTo, LerpFactor));

			VectorStore(OutputVec, OutputFrames);

			CurrentFrameFixed += 4 * FixedPointSampleRate;
			OutputFrames += 4;
		}
	}
#endif

	// Remnants
	for (; OutputFrame < OutputFramesNeeded; OutputFrame++)
	{
		uint32 SourceOffset = CurrentFrameFixed >> 16;
		uint32 CurFrac = CurrentFrameFixed & 0xffff;

		float LerpFactor = (float)(CurFrac) * (1.0f / 65536.0f);

		CurrentFrameFixed += FixedPointSampleRate;

		float Sample1 = LeftSamples[SourceOffset];
		float Sample2 = LeftSamples[SourceOffset + 1];
		OutputFrames[0] = Sample1 * (1 - LerpFactor) + (LerpFactor)* Sample2;

		OutputFrames++;
	}
	CurrentFrameFraction = NextRunStartPositionFixed - (FarthestPositionFixed & ~0xffff);

	return FarthestPositionFixed >> 16;
}

uint32 Audio::FVectorLinearResampler::ResampleStereo(uint32 OutputFramesNeeded, uint32 FixedPointSampleRate, float const* SourceFrames, uint32 SourceFramesStrideFloats, float* OutputFrames, uint32 OutputFramesStrideFloats)
{
	// we CANT mask off the upper bits, as it's possible the fraction could
	// jump an entire sample and we need to retain that offset.
	const uint32 FarthestPositionFixed = CurrentFrameFraction + (OutputFramesNeeded - 1) * FixedPointSampleRate;
	const uint32 NextRunStartPositionFixed = FarthestPositionFixed + FixedPointSampleRate;

	float const* LeftSamples = SourceFrames;
	float const* RightSamples = SourceFrames + SourceFramesStrideFloats;
	uint32 OutputFrame = 0;
	uint32 CurrentFrameFixed = CurrentFrameFraction;

#if (PLATFORM_ENABLE_VECTORINTRINSICS || PLATFORM_ENABLE_VECTORINTRINSICS_NEON) // SIMD path.
	{
		float CurrentFrameFractionFloat = (float)(CurrentFrameFraction & 0xffff);

		VectorRegister4Float ScaleVec = VectorSetFloat1(1.0f / 65536.0f);
		VectorRegister4Float TwoVec = VectorSetFloat1(2.0f);
		VectorRegister4Float OneVec = VectorSetFloat1(1.0f);
		VectorRegister4Float ThreeVec = VectorSetFloat1(3.0f);

		VectorRegister4Float CurFracStepVec = VectorSet(
			(float)((0 * FixedPointSampleRate) & 0xffff),
			(float)((1 * FixedPointSampleRate) & 0xffff),
			(float)((2 * FixedPointSampleRate) & 0xffff),
			(float)((3 * FixedPointSampleRate) & 0xffff));

		VectorRegister4Float CurFracVec = VectorSetFloat1(CurrentFrameFractionFloat);
		CurFracVec = VectorAdd(CurFracVec, CurFracStepVec);
		CurFracVec = VectorMultiply(CurFracVec, ScaleVec);
		CurFracVec = VectorAdd(CurFracVec, TwoVec);

		VectorRegister4Float MaskVec = MakeVectorRegisterFloat(0xffbfffff, 0xffbfffff, 0xffbfffff, 0xffbfffff);

		VectorRegister4Float StepVec = VectorSetFloat1((float)((FixedPointSampleRate * 4) & 0xffff));
		StepVec = VectorMultiply(StepVec, ScaleVec);

		uint32 OutputFramesNeededSIMD = OutputFramesNeeded & ~3;

		for (; OutputFrame < OutputFramesNeededSIMD; OutputFrame += 4)
		{
			const uint32 SourceOffsets[4] = {
				CurrentFrameFixed >> 16,
				(CurrentFrameFixed + FixedPointSampleRate) >> 16,
				(CurrentFrameFixed + FixedPointSampleRate + FixedPointSampleRate) >> 16,
				(CurrentFrameFixed + FixedPointSampleRate + FixedPointSampleRate + FixedPointSampleRate) >> 16
			};

			// [0, 0+1, 1, 1+1]
			VectorRegister4Float LeftSamples01 = VectorLoadTwoPairsFloat(LeftSamples + SourceOffsets[0], LeftSamples + SourceOffsets[1]);
			VectorRegister4Float RightSamples01 = VectorLoadTwoPairsFloat(RightSamples + SourceOffsets[0], RightSamples + SourceOffsets[1]);

			// [2, 2+1, 3, 3+1]
			VectorRegister4Float LeftSamples23 = VectorLoadTwoPairsFloat(LeftSamples + SourceOffsets[2], LeftSamples + SourceOffsets[3]);
			VectorRegister4Float RightSamples23 = VectorLoadTwoPairsFloat(RightSamples + SourceOffsets[2], RightSamples + SourceOffsets[3]);

			// want [0, 1, 2, 3]
			// [0+1, 1+1, 2+1, 3+1]
			VectorRegister4Float LeftSamplesFrom, LeftSamplesTo;
			VectorDeinterleave(LeftSamplesFrom, LeftSamplesTo, LeftSamples01, LeftSamples23);
			VectorRegister4Float RightSamplesFrom, RightSamplesTo;
			VectorDeinterleave(RightSamplesFrom, RightSamplesTo, RightSamples01, RightSamples23);

			// our lerp vector is cur_frac_vec, masking off the 1 bit in the mantissa, subtract 2.
			CurFracVec = VectorBitwiseAnd(CurFracVec, MaskVec);
			VectorRegister4Float LerpFactor = VectorSubtract(CurFracVec, TwoVec);
			VectorRegister4Float OneMinusLerpFactor = VectorSubtract(ThreeVec, CurFracVec);

			CurFracVec = VectorAdd(CurFracVec, StepVec);

			VectorRegister4Float LeftOutputVec = VectorAdd(VectorMultiply(LeftSamplesFrom, OneMinusLerpFactor), VectorMultiply(LeftSamplesTo, LerpFactor));
			VectorRegister4Float RightOutputVec = VectorAdd(VectorMultiply(RightSamplesFrom, OneMinusLerpFactor), VectorMultiply(RightSamplesTo, LerpFactor));

			VectorStore(LeftOutputVec, OutputFrames);
			VectorStore(RightOutputVec, OutputFrames + OutputFramesStrideFloats);

			CurrentFrameFixed += 4 * FixedPointSampleRate;
			OutputFrames += 4;
		}
	}
#endif

	for (; OutputFrame < OutputFramesNeeded; OutputFrame++)
	{
		uint32 SourceOffset = CurrentFrameFixed >> 16;
		uint32 CurFrac = CurrentFrameFixed & 0xffff;

		float LerpFactor = (float)(CurFrac) * (1.0f / 65536.0f);

		CurrentFrameFixed += FixedPointSampleRate;

		float Sample1 = LeftSamples[SourceOffset];
		float Sample2 = LeftSamples[SourceOffset + 1];
		OutputFrames[0] = Sample1 * (1 - LerpFactor) + (LerpFactor)* Sample2;

		Sample1 = RightSamples[SourceOffset];
		Sample2 = RightSamples[SourceOffset + 1];
		OutputFrames[OutputFramesStrideFloats] = Sample1 * (1 - LerpFactor) + (LerpFactor)* Sample2;

		OutputFrames++;
	}
	CurrentFrameFraction = NextRunStartPositionFixed - (FarthestPositionFixed & ~0xffff);

	return FarthestPositionFixed >> 16;
}
