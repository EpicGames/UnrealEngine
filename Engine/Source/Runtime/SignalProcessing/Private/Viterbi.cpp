// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Viterbi.h"

namespace Audio
{
	TArray<int32> FViterbi::Decode(const IViterbiObservations& InObservations, const IViterbiInitialProbability& InInitialProbability, const IViterbiTransitionProbability& InTransitionProbability)
	{
		// Get size of state space
		int32 NumStates = InObservations.GetNumStates();
		int32 NumTimeSteps = InObservations.GetNumTimeSteps();

		if ((NumStates < 1) || (NumTimeSteps < 1))
		{
			// Empty sequence if no states or time steps.
			TArray<int32> Empty;
			return Empty;
		}

		// Prepare arrays
		TArray<TArray<int32>> Backtrack;
		TArray<int32> MaxIndices;
		TArray<float> StorageOdd;
		TArray<float> StorageEven;
		TArray<float> StorageTemp;

		MaxIndices.SetNumZeroed(NumStates);
		StorageOdd.SetNumZeroed(NumStates);
		StorageEven.SetNumZeroed(NumStates);
		StorageTemp.SetNumZeroed(NumStates);

		
		// Initial state log probability 
		for (int32 i = 0; i < NumStates; i++)
		{
			StorageEven[i] = InInitialProbability.GetInitialLogProbability(i) + InObservations.GetEmissionLogProbability(0, i);
		}

		float* Temp = StorageTemp.GetData();

		// Iterator through all time steps. 
		for (int32 TimeStep = 1; TimeStep < NumTimeSteps; TimeStep++)
		{
			// Viterbi only uses previous state, so reuse storage while accumulating probability
			// across states. Use the current timestep to determine which storage is previous
			// and which storage is current. 
			bool bIsOdd = TimeStep & 0x00000001;
			float* Previous = bIsOdd ? StorageEven.GetData() : StorageOdd.GetData();
			float* Current = bIsOdd ? StorageOdd.GetData() : StorageEven.GetData();
			int32* Indices = MaxIndices.GetData();

			for (int32 CurrentState = 0; CurrentState < NumStates; CurrentState++)
			{
				// Get emission probability for this state.
				float EmitLogProb = InObservations.GetEmissionLogProbability(TimeStep, CurrentState);

				// Find highest probability previous state. 
				float MaxValue = Previous[0] + InTransitionProbability.GetTransitionLogProbability(TimeStep, 0, CurrentState) + EmitLogProb;
				int32 MaxIndex = 0;

				for (int32 PreviousState = 1; PreviousState < NumStates; PreviousState++)
				{
					float Value = Previous[PreviousState] + InTransitionProbability.GetTransitionLogProbability(TimeStep, PreviousState, CurrentState) + EmitLogProb;
					if (Value > MaxValue)
					{
						MaxValue = Value;
						MaxIndex = PreviousState;
					}
				}

				// Store highest probability prevoius state. 
				Current[CurrentState] = MaxValue;
				Indices[CurrentState] = MaxIndex;
			}

			// Store data for backtracking through states at end. 
			Backtrack.Add(MaxIndices);
		}

		// Find maximum probability ending state. 
		float* Final = NumTimeSteps & 0x00000001 ? StorageEven.GetData() : StorageOdd.GetData();

		float MaxValue = Final[0];
		int32 MaxIndex = 0;

		for (int32 i = 1; i < NumStates; i++)
		{
			if (Final[i] > MaxValue)
			{
				MaxValue = Final[i];
				MaxIndex = i;
			}
		}

		// Backtrack through state space to decode sequence
		TArray<int32> StateSequence;
		StateSequence.Add(MaxIndex);

		for (int32 TimeStep = NumTimeSteps - 2; TimeStep >= 0; TimeStep--)
		{
			int32 PreviousState = Backtrack[TimeStep][StateSequence.Last()];
			StateSequence.Add(PreviousState);
		}

		// Reverse state sequence so it goes in chronological order.
		Algo::Reverse(StateSequence);

		return StateSequence;
	}
}
		
