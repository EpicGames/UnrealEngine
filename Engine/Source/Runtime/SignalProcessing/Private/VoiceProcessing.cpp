// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/VoiceProcessing.h"

#include "DSP/AudioDebuggingUtilities.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{

	FMovingAverager::FMovingAverager(uint32 NumSamples)
		: BufferCursor(0)
		, AccumulatedSum(0.0f)
	{
		AudioBuffer.Reset();
		AudioBuffer.AddZeroed(FMath::Max<uint32>(NumSamples, 1));
	}

	float FMovingAverager::ProcessInput(const float& Input, float& Output)
	{
		FScopeLock ScopeLock(&ProcessCriticalSection);

		float* BufferPtr = AudioBuffer.GetData();
		Output = BufferPtr[BufferCursor];
		BufferPtr[BufferCursor] = Input;
		BufferCursor = (BufferCursor + 1) % AudioBuffer.Num();

		// Instead of summing our entire buffer every tick, we simply add the incoming sample amplitude, and subtract the outgoing amplitude.
		// TODO: With this approach AccumulatedSum will start to slowly drift over time from accumulated rounding error. Every so often we will need to
		// reset AccumulatedSum to the actual sum of AudioBuffer.
		AccumulatedSum += (FMath::Abs(Input) - FMath::Abs(Output));

		return AccumulatedSum;
	}

	void FMovingAverager::SetNumSamples(uint32 NumSamples)
	{
		FScopeLock ScopeLock(&ProcessCriticalSection);

		AudioBuffer.Reset();
		AudioBuffer.AddZeroed(NumSamples);
		AccumulatedSum = 0.0f;
	}

	FMovingVectorAverager::FMovingVectorAverager(uint32 NumSamples)
		: BufferCursor(0)
		, AccumulatedSum(VectorZero())
	{
		checkf(NumSamples % 4 == 0, TEXT("NumSamples must be divisible by 4!"));
		const uint32 NumVectors = NumSamples / 4;
		AudioBuffer.Reset();
		AudioBuffer.AddZeroed(NumVectors);
	}

	float FMovingVectorAverager::ProcessAudio(const VectorRegister4Float& Input, VectorRegister4Float& Output)
	{
		VectorRegister4Float* BufferPtr = AudioBuffer.GetData();
		Output = BufferPtr[BufferCursor];
		BufferPtr[BufferCursor] = Input;
		BufferCursor = (BufferCursor + 1) % AudioBuffer.Num();

		// Instead of summing our entire buffer every tick, we simply add the incoming sample amplitude, and subtract the outgoing amplitude.
		// TODO: With this approach AccumulatedSum will start to slowly drift over time from accumulated rounding error. Every so often we will need to
		// reset AccumulatedSum to the actual sum of AudioBuffer.
		const VectorRegister4Float AbsInput = VectorAbs(Input);
		const VectorRegister4Float AbsOutput = VectorAbs(Output);
		const VectorRegister4Float TotalAccumulation = VectorSubtract(AbsInput, AbsOutput);
		AccumulatedSum = VectorAdd(AccumulatedSum, TotalAccumulation);

		alignas(16) float PartionedSums[4];
		VectorStore(AccumulatedSum, PartionedSums);

		return (PartionedSums[0] + PartionedSums[1] + PartionedSums[2] + PartionedSums[3]) / (AudioBuffer.Num() * 4);
	}

	FSilenceDetection::FSilenceDetection(float InOnsetThreshold, float InReleaseThreshold,  int32 AttackDurationInSamples, int32 ReleaseDurationInSamples)
		: Averager(AttackDurationInSamples)
		, ReleaseTau(FMath::Exp(-1.0f / (ReleaseDurationInSamples / 4)))
		, OnsetThreshold(InOnsetThreshold)
		, ReleaseThreshold(InReleaseThreshold)
		, CurrentAmplitude(0.0f)
		, bOnsetWasInLastBuffer(false)
	{
		ensureMsgf(InOnsetThreshold > InReleaseThreshold, TEXT("The onset threshold should always be louder than the release threshold."));
	}

	int32 FSilenceDetection::ProcessBuffer(const float* InAudio, float* OutAudio, int32 NumSamples)
	{
		if (bOnsetWasInLastBuffer || CurrentAmplitude > ReleaseThreshold)
		{
			bOnsetWasInLastBuffer = false;

			// If we've been outputting audio up until the start of this callback, we are going to continue outputting audio
			// Until the end of this buffer. If the current amplitude is below our threshold at the end of this callback, we will
			// stop outputting audio then.
			for (int32 InSampleIndex = 0; InSampleIndex < NumSamples; InSampleIndex += 4)
			{
				const VectorRegister4Float InputVector = VectorLoad(&InAudio[InSampleIndex]);
				VectorRegister4Float OutputVector;
				float InstantaneousAmplitude = Averager.ProcessAudio(InputVector, OutputVector);
				CurrentAmplitude = ReleaseTau * (CurrentAmplitude - InstantaneousAmplitude) + InstantaneousAmplitude;
				VectorStore(OutputVector, &OutAudio[InSampleIndex]);
			}

			// If we are releasing back to silence at the end of this buffer callback, we perform a short fadeout here.
			if (CurrentAmplitude < ReleaseThreshold)
			{
				static const int32 DefaultNumSamplesToFadeOutOver = 32;
				const int32 NumSamplesToFadeOutOver = FMath::Min(NumSamples, DefaultNumSamplesToFadeOutOver);

				const int32 Offset = NumSamples - NumSamplesToFadeOutOver;
				TArrayView<float> OutAudioView(&OutAudio[Offset], NumSamplesToFadeOutOver);
				Audio::ArrayFade(OutAudioView, 1.0f, 0.0f);
			}

			return NumSamples;
		}
		else
		{
			// If we started this callback in a silent state, we simply buffer audio until we've detected an onset,
			// At which point we begin outputting audio from the Averager.
			int32 OutSampleIndex = 0;
			bool bHitThreshold = false;
			float InstantaneousAmplitude = 0.0f;
			for (int32 InSampleIndex = 0; InSampleIndex < NumSamples; InSampleIndex += 4)
			{
				const VectorRegister4Float InputVector = VectorLoad(&InAudio[InSampleIndex]);
				VectorRegister4Float OutputVector;
				InstantaneousAmplitude = Averager.ProcessAudio(InputVector, OutputVector);

				if (bHitThreshold)
				{	
					VectorStore(OutputVector, &OutAudio[OutSampleIndex]);
					OutSampleIndex += 4;
				}
				else
				{
					bHitThreshold = InstantaneousAmplitude > OnsetThreshold;
				}
			}

			CurrentAmplitude = InstantaneousAmplitude;
			bOnsetWasInLastBuffer = bHitThreshold;
			check(CurrentAmplitude < 100.0f);
			return OutSampleIndex;
		}
	}

	void FSilenceDetection::SetThreshold(float InThreshold)
	{
		OnsetThreshold = InThreshold;
	}

	float FSilenceDetection::GetCurrentAmplitude()
	{
		return CurrentAmplitude;
	}

	FSlowAdaptiveGainControl::FSlowAdaptiveGainControl(float InGainTarget, int32 InAdaptiveRate, float InGainMin /*= 0.5f*/, float InGainMax /*= 2.0f*/)
		: PeakDetector(InAdaptiveRate)
		, GainTarget(InGainTarget)
		, PreviousGain(1.0f)
		, GainMin(InGainMin)
		, GainMax(InGainMax)
	{
	}

	float FSlowAdaptiveGainControl::ProcessAudio(float* InAudio, int32 NumSamples, float InAmplitude)
	{
		float PeakDetectorOutput = 0.0f; // unused
		const float EstimatedPeak = PeakDetector.ProcessInput(InAmplitude, PeakDetectorOutput);
		const float TargetGain = GetTargetGain(EstimatedPeak);
		TArrayView<float> InAudioView(InAudio, NumSamples);
		Audio::ArrayFade(InAudioView, PreviousGain, TargetGain);
		PreviousGain = TargetGain;

		return TargetGain;
	}

	void FSlowAdaptiveGainControl::SetAdaptiveRate(int32 InAdaptiveRate)
	{
		PeakDetector.SetNumSamples(InAdaptiveRate);
	}

	float FSlowAdaptiveGainControl::GetTargetGain(float InAmplitude)
	{
		const float UnclampedGain = GainTarget / InAmplitude;
		return FMath::Clamp(UnclampedGain, GainMin, GainMax);
	}
}
