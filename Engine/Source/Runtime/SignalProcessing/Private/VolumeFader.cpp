// Copyright Epic Games, Inc. All Rights Reserved.
#include "DSP/VolumeFader.h"

#include "DSP/Dsp.h"


namespace Audio
{
	FVolumeFader::FVolumeFader()
		: Alpha(1.0f)
		, Target(1.0f)
		, ActiveDuration(-1.0f)
		, FadeDuration(-1.0f)
		, Elapsed(0.0f)
		, FadeCurve(EFaderCurve::Linear)
	{
	}

	float FVolumeFader::AlphaToVolume(float InAlpha, EFaderCurve InCurve, float InStartingValue, float InTarget)
	{
		switch (InCurve)
		{
		case EFaderCurve::Linear:
		{
			return InAlpha;
		}

		case EFaderCurve::SCurve:
		{
			const float NormalizedAlpha = FMath::GetMappedRangeValueClamped(FVector2f(InStartingValue, InTarget), FVector2f(0.0f, 1.0f), InAlpha);
			float Volume = 0.5f * Audio::FastSin(PI * NormalizedAlpha - HALF_PI) + 0.5f;
			Volume = FMath::GetMappedRangeValueClamped(FVector2f(0.0f, 1.0f), FVector2f(InStartingValue, InTarget), Volume);
			return FMath::Max(0.0f, Volume);
		}

		case EFaderCurve::Sin:
		{
			const float NormalizedAlpha = FMath::GetMappedRangeValueClamped(FVector2f(InStartingValue, InTarget), FVector2f(0.0f, 1.0f), InAlpha);
			float Volume = Audio::FastSin(HALF_PI * NormalizedAlpha);
			Volume = FMath::GetMappedRangeValueClamped(FVector2f(0.0f, 1.0f), FVector2f(InStartingValue, InTarget), Volume);
			return FMath::Max(0.0f, Volume);
		}

		case EFaderCurve::Logarithmic:
		{
			return Audio::ConvertToLinear(InAlpha);
		}

		default:
		{
			static_assert(static_cast<int32>(EFaderCurve::Count) == 4, "Possible missing switch case coverage for EAudioFade");
		}
		break;
		}

		return 1.0f;
	}

	void FVolumeFader::Activate()
	{
		if (ActiveDuration == 0.0f)
		{
			ActiveDuration = -1.0f;
		}
	}

	void FVolumeFader::Deactivate()
	{
		StopFade();
		ActiveDuration = 0.0f;
	}

	float FVolumeFader::GetActiveDuration() const
	{
		return ActiveDuration;
	}

	float FVolumeFader::GetVolume() const
	{
		return AlphaToVolume(Alpha, FadeCurve, StartingValue, Target);
	}

	float FVolumeFader::GetVolumeAfterTime(float InDeltaTime) const
	{
		InDeltaTime = FMath::Max(0.0f, InDeltaTime);

		// Keep stepping towards our target until we hit our stop time & Clamp
		float FutureAlpha = Alpha;
		const float Duration = Elapsed + InDeltaTime;

		// If set to deactivate first, return default volume value of 1.0f
		if (ActiveDuration >= 0.0f && Duration > ActiveDuration)
		{
			return 1.0f;
		}

		if (Duration < FadeDuration)
		{
			// Choose min/max bound and clamp dt to prevent unwanted spikes in volume
			float MinValue = 0.0f;
			float MaxValue = 0.0f;
			if (Alpha < Target)
			{
				MinValue = Alpha;
				MaxValue = Target;
			}
			else
			{
				MinValue = Target;
				MaxValue = Alpha;
			}

			FutureAlpha = Alpha + ((Target - Alpha) * InDeltaTime / (FadeDuration - Elapsed));
			FutureAlpha = FMath::Clamp(FutureAlpha, MinValue, MaxValue);
		}

		return AlphaToVolume(FutureAlpha, FadeCurve, StartingValue, Target);
	}

	float FVolumeFader::GetFadeDuration() const
	{
		return FadeDuration;
	}

	EFaderCurve FVolumeFader::GetCurve() const
	{
		return FadeCurve;
	}

	float FVolumeFader::GetTargetVolume() const
	{
		switch (FadeCurve)
		{
		case EFaderCurve::Linear:
		case EFaderCurve::SCurve:
		case EFaderCurve::Sin:
		{
			return Target;
		}

		case EFaderCurve::Logarithmic:
		{
			return Audio::ConvertToLinear(Target);
		}

		default:
		{
			static_assert(static_cast<int32>(EFaderCurve::Count) == 4, "Possible missing switch case coverage for EAudioFade");
		}
		break;
		}

		return 1.0f;
	}

	bool FVolumeFader::IsActive() const
	{
		return Elapsed < ActiveDuration || ActiveDuration < 0.0f;
	}

	bool FVolumeFader::IsFading() const
	{
		return Elapsed < FadeDuration;
	}

	bool FVolumeFader::IsFadingIn() const
	{
		return IsFading() && Target > Alpha;
	}

	bool FVolumeFader::IsFadingOut() const
	{
		return IsFading() && Target < Alpha;
	}

	void FVolumeFader::SetActiveDuration(float InDuration)
	{
		if (InDuration < 0.0f)
		{
			Activate();
		}
		else
		{
			ActiveDuration = InDuration;
			Elapsed = 0.0f;
		}
	}

	void FVolumeFader::SetVolume(float InVolume)
	{
		Alpha = InVolume;
		Elapsed = 0.0f;
		FadeCurve = EFaderCurve::Linear;
		FadeDuration = -1.0f;
		Target = InVolume;
	}

	void FVolumeFader::StartFade(float InVolume, float InDuration, EFaderCurve InCurve)
	{
		if (InDuration <= 0.0f)
		{
			SetVolume(InVolume);
			return;
		}

		if (InCurve != EFaderCurve::Logarithmic)
		{
			if (FadeCurve == EFaderCurve::Logarithmic)
			{
				Alpha = Audio::ConvertToLinear(Alpha);
			}
			Target = InVolume;
		}
		else
		{
			const float DecibelFloor = KINDA_SMALL_NUMBER; // -80dB
			if (FadeCurve != EFaderCurve::Logarithmic)
			{
				Alpha = Audio::ConvertToDecibels(Alpha, DecibelFloor);
			}
			Target = Audio::ConvertToDecibels(InVolume, DecibelFloor);
		}

		Elapsed = 0.0f;
		FadeCurve = InCurve;
		FadeDuration = InDuration;
		StartingValue = Alpha;
	}

	void FVolumeFader::StopFade()
	{
		if (FadeCurve == EFaderCurve::Logarithmic)
		{
			Alpha = Audio::ConvertToLinear(Alpha);
		}
		Target = Alpha;
		FadeCurve = EFaderCurve::Linear;
		Elapsed = ActiveDuration;
		FadeDuration = -1.0f;
	}

	void FVolumeFader::Update(float InDeltaTime)
	{
		// querying state before incrementing Elapsed time
		// lets sounds with a fade-in time < InDeltaTime play
		const bool bIsFading = IsFading();
		const bool bIsActive = IsActive();

		Elapsed += InDeltaTime;

		if (!bIsFading || !bIsActive)
		{
			return;
		}

		// Keep stepping towards target & clamp until fade duration has expired
		// Choose min/max bound and clamp dt to prevent unwanted spikes in volume
		float MinValue = 0.0f;
		float MaxValue = 0.0f;

		if (FadeDuration < Elapsed)
		{
			Alpha = Target;
			StopFade();
			return;
		}
		else if (Alpha < Target)
		{
			MinValue = Alpha;
			MaxValue = Target;
		}
		else 
		{
			MinValue = Target;
			MaxValue = Alpha;
		}

		Alpha += (Target - Alpha) * InDeltaTime / (FadeDuration - Elapsed);
		Alpha = FMath::Clamp(Alpha, MinValue, MaxValue);


		// Optimization that avoids fade calculations once alpha reaches target
		if (InDeltaTime > SMALL_NUMBER && FMath::IsNearlyEqual(Alpha, Target))
		{
			StopFade();
		}
	}
} // namespace Audio