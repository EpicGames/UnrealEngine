// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/Encoders/WavEncoder.h"

FWavEncoder::FWavEncoder(const FSoundQualityInfo& InInfo, int32 AudioCallbackSize)
	: IAudioEncoder(AudioCallbackSize * 4, AudioCallbackSize * 4 * sizeof(float) * 2)
	, CallbackSize(AudioCallbackSize)
{
	Init(InInfo);
}

int32 FWavEncoder::GetCompressedPacketSize() const
{
	return 0;
}

int64 FWavEncoder::SamplesRequiredPerEncode() const
{
	return CallbackSize;
}

static void AppendToTail(TArray<uint8>& InByteArray, const uint32 Value)
{
	InByteArray.Append((uint8*) &Value, sizeof(uint32));
}

static void AppendToTail(TArray<uint8>& InByteArray, const uint16 Value)
{
	InByteArray.Append((uint8*) &Value, sizeof(uint16));
}

static void AppendToTail(TArray<uint8>& InByteArray, const uint8 Value)
{
	InByteArray.Append(&Value, sizeof(uint8));
}

bool FWavEncoder::StartFile(const FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutFileStart)
{
	// Reserve space for the raw wave data
	OutFileStart.Reset();

	// Wave Format Serialization ----------

	// FieldName: ChunkID
	// FieldSize: 4 bytes
	// FieldValue: RIFF (FourCC value, big-endian)
	uint8 ChunkID[4] = { 'R', 'I', 'F', 'F' };
	OutFileStart.Append(ChunkID, 4);

	// ChunkName: ChunkSize: 4 bytes 
	// Value: NumBytes + 36. Size of the rest of the chunk following this number. Size of entire file minus 8 bytes.
	AppendToTail(OutFileStart, static_cast<uint32>(InQualityInfo.SampleDataSize + 36));

	// FieldName: Format 
	// FieldSize: 4 bytes
	// FieldValue: "WAVE"  (big-endian)
	uint8 Format[4] = { 'W', 'A', 'V', 'E' };
	OutFileStart.Append(Format, 4);

	// FieldName: Subchunk1ID
	// FieldSize: 4 bytes
	// FieldValue: "fmt "
	uint8 Subchunk1ID[4] = { 'f', 'm', 't', ' ' };
	OutFileStart.Append(Subchunk1ID, 4);

	// FieldName: Subchunk1Size
	// FieldSize: 4 bytes
	// FieldValue: 16 for PCM
	AppendToTail(OutFileStart, static_cast<uint32>(16));

	// FieldName: AudioFormat
	// FieldSize: 2 bytes
	// FieldValue: 1 for PCM
	AppendToTail(OutFileStart, static_cast<uint16>(1));

	// FieldName: NumChannels
	// FieldSize: 2 bytes
	// FieldValue: 1 for for mono
	AppendToTail(OutFileStart, static_cast<uint16>(InQualityInfo.NumChannels));

	// FieldName: SampleRate
	// FieldSize: 4 bytes
	// FieldValue: Passed in sample rate
	AppendToTail(OutFileStart, static_cast<uint32>(InQualityInfo.SampleRate));

	// FieldName: ByteRate
	// FieldSize: 4 bytes
	// FieldValue: SampleRate * NumChannels * BitsPerSample/8
	uint32 ByteRate = InQualityInfo.SampleRate * InQualityInfo.NumChannels * 2;
	AppendToTail(OutFileStart, static_cast<uint32>(ByteRate));

	// FieldName: BlockAlign
	// FieldSize: 2 bytes
	// FieldValue: NumChannels * BitsPerSample/8
	uint16 BlockAlign = 2;
	AppendToTail(OutFileStart, static_cast<uint16>(BlockAlign));

	// FieldName: BitsPerSample
	// FieldSize: 2 bytes
	// FieldValue: 16 (16 bits per sample)
	AppendToTail(OutFileStart, static_cast<uint16>(16));

	// FieldName: Subchunk2ID
	// FieldSize: 4 bytes
	// FieldValue: "data" (big endian)
	uint8 Subchunk2ID[4] = { 'd','a','t','a' };
	OutFileStart.Append(Subchunk2ID, 4);

	// FieldName: Subchunk2Size
	// FieldSize: 4 bytes
	// FieldValue: number of bytes of the data
	AppendToTail(OutFileStart, static_cast<uint32>(InQualityInfo.SampleDataSize));

	return true;
}

bool FWavEncoder::EncodeChunk(const TArray<float>& InAudio, TArray<uint8>& OutBytes)
{
	OutBytes.AddUninitialized(InAudio.Num() * sizeof(int16));

	int16* DataPtr = (int16*) OutBytes.GetData();
	for (int32 Index = 0; Index < InAudio.Num(); Index++)
	{
		DataPtr[Index] = (int16)(InAudio[Index] * 32767.0f);
	}

	return true;
}

bool FWavEncoder::EndFile(TArray<uint8>& OutBytes)
{
	// nothing to do here.
	return true;
}


