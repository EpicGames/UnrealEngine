// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/WavFormatUtils.h"

#include "SignalProcessingModule.h"
#include "Algo/Transform.h"
#include "Audio/AudioSpeakers.h"

namespace WavFormatUtils
{
	// Speaker Positions for dwChannelMask in WAVEFORMATEXTENSIBLE:
	// Duplicate of the wav
	enum UE_SpeakerBitfield : uint32
	{
		UE_SPEAKER_FRONT_LEFT             = 0x1,
		UE_SPEAKER_FRONT_RIGHT            = 0x2,
		UE_SPEAKER_FRONT_CENTER           = 0x4,
		UE_SPEAKER_LOW_FREQUENCY          = 0x8,
		UE_SPEAKER_BACK_LEFT              = 0x10,
		UE_SPEAKER_BACK_RIGHT             = 0x20,
		UE_SPEAKER_FRONT_LEFT_OF_CENTER   = 0x40,
		UE_SPEAKER_FRONT_RIGHT_OF_CENTER  = 0x80,
		UE_SPEAKER_BACK_CENTER            = 0x100,
		UE_SPEAKER_SIDE_LEFT              = 0x200,
		UE_SPEAKER_SIDE_RIGHT             = 0x400,
		UE_SPEAKER_TOP_CENTER             = 0x800,
		UE_SPEAKER_TOP_FRONT_LEFT         = 0x1000,
		UE_SPEAKER_TOP_FRONT_CENTER       = 0x2000,
		UE_SPEAKER_TOP_FRONT_RIGHT        = 0x4000,
		UE_SPEAKER_TOP_BACK_LEFT          = 0x8000,
		UE_SPEAKER_TOP_BACK_CENTER        = 0x10000,
		UE_SPEAKER_TOP_BACK_RIGHT         = 0x20000
	};
	
	// Static Assert that AudioMixer Channel Enum is the Same as WAVEEX BitMask Definitions.
	// So we can easily and safely convert between them
	static_assert(UE_SPEAKER_FRONT_LEFT == 1 << EAudioMixerChannel::FrontLeft);
	static_assert(UE_SPEAKER_FRONT_RIGHT == 1 << EAudioMixerChannel::FrontRight);
	static_assert(UE_SPEAKER_FRONT_CENTER == 1 << EAudioMixerChannel::FrontCenter);
	static_assert(UE_SPEAKER_LOW_FREQUENCY == 1 << EAudioMixerChannel::LowFrequency);
	static_assert(UE_SPEAKER_BACK_LEFT == 1 << EAudioMixerChannel::BackLeft);
	static_assert(UE_SPEAKER_BACK_RIGHT == 1 << EAudioMixerChannel::BackRight);
	static_assert(UE_SPEAKER_FRONT_LEFT_OF_CENTER == 1 << EAudioMixerChannel::FrontLeftOfCenter);
	static_assert(UE_SPEAKER_FRONT_RIGHT_OF_CENTER == 1 << EAudioMixerChannel::FrontRightOfCenter);
	static_assert(UE_SPEAKER_BACK_CENTER == 1 << EAudioMixerChannel::BackCenter);
	static_assert(UE_SPEAKER_SIDE_LEFT == 1 << EAudioMixerChannel::SideLeft);
	static_assert(UE_SPEAKER_SIDE_RIGHT == 1 << EAudioMixerChannel::SideRight);
	static_assert(UE_SPEAKER_TOP_CENTER == 1 << EAudioMixerChannel::TopCenter);
	static_assert(UE_SPEAKER_TOP_FRONT_LEFT == 1 << EAudioMixerChannel::TopFrontLeft);
	static_assert(UE_SPEAKER_TOP_FRONT_RIGHT == 1 << EAudioMixerChannel::TopFrontRight);
	static_assert(UE_SPEAKER_TOP_BACK_LEFT == 1 << EAudioMixerChannel::TopBackLeft);
	static_assert(UE_SPEAKER_TOP_BACK_CENTER == 1 << EAudioMixerChannel::TopBackCenter);
	static_assert(UE_SPEAKER_TOP_BACK_RIGHT == 1 << EAudioMixerChannel::TopBackRight);

	namespace CommonFormats
	{
		// Common formats bitfields.
		static constexpr uint32 Mono				 = UE_SPEAKER_FRONT_CENTER;
		static constexpr uint32 Stereo				 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT;
		static constexpr uint32 Stereo_2_1			 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_LOW_FREQUENCY;
		static constexpr uint32 Quad_Back			 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_BACK_LEFT | UE_SPEAKER_BACK_RIGHT;
		static constexpr uint32 Quad_Side			 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_SIDE_LEFT | UE_SPEAKER_SIDE_RIGHT; 
		static constexpr uint32 Surround_3_0		 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER;						
		static constexpr uint32 Surround_3_1		 = Surround_3_0 | UE_SPEAKER_LOW_FREQUENCY;													
		static constexpr uint32 Surround_4_0		 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_BACK_CENTER; 	
		static constexpr uint32 Surround_5_0		 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_BACK_LEFT | UE_SPEAKER_BACK_RIGHT;
		static constexpr uint32 Surround_5_0_Side	 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_SIDE_LEFT | UE_SPEAKER_SIDE_RIGHT;		
		static constexpr uint32 Surround_5_1		 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_LOW_FREQUENCY | UE_SPEAKER_BACK_LEFT | UE_SPEAKER_BACK_RIGHT;
		static constexpr uint32 Surround_5_1_Side	 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_LOW_FREQUENCY | UE_SPEAKER_SIDE_LEFT | UE_SPEAKER_SIDE_RIGHT;
		
		static constexpr uint32 Surround_6_0		= UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_BACK_LEFT | UE_SPEAKER_BACK_RIGHT | UE_SPEAKER_BACK_CENTER;
		static constexpr uint32 Surround_6_1		= UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER |UE_SPEAKER_LOW_FREQUENCY | UE_SPEAKER_BACK_LEFT | UE_SPEAKER_BACK_RIGHT | UE_SPEAKER_BACK_CENTER;
		static constexpr uint32 Surround_7_0		= UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_BACK_LEFT | UE_SPEAKER_BACK_RIGHT | UE_SPEAKER_SIDE_LEFT | UE_SPEAKER_SIDE_RIGHT; 
		static constexpr uint32 Surround_7_1		= UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_LOW_FREQUENCY | UE_SPEAKER_BACK_LEFT | UE_SPEAKER_BACK_RIGHT | UE_SPEAKER_SIDE_LEFT | UE_SPEAKER_SIDE_RIGHT;
		static constexpr uint32 Surround_7_1_Side	= UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_LOW_FREQUENCY | UE_SPEAKER_SIDE_LEFT | UE_SPEAKER_SIDE_RIGHT | UE_SPEAKER_FRONT_LEFT_OF_CENTER | UE_SPEAKER_FRONT_RIGHT_OF_CENTER;
		
		// 5..X
		static constexpr uint32 Surround_5_0_2 = Surround_5_0_Side | UE_SPEAKER_TOP_FRONT_LEFT | UE_SPEAKER_TOP_FRONT_RIGHT;
		static constexpr uint32 Surround_5_1_2 = Surround_5_1_Side | UE_SPEAKER_TOP_FRONT_LEFT | UE_SPEAKER_TOP_FRONT_RIGHT;
		static constexpr uint32 Surround_5_0_4 = Surround_5_0_Side | UE_SPEAKER_TOP_FRONT_LEFT | UE_SPEAKER_TOP_FRONT_RIGHT | UE_SPEAKER_TOP_BACK_LEFT | UE_SPEAKER_TOP_BACK_RIGHT;
		static constexpr uint32 Surround_5_1_4 = Surround_5_1_Side | UE_SPEAKER_TOP_FRONT_LEFT | UE_SPEAKER_TOP_FRONT_RIGHT | UE_SPEAKER_TOP_BACK_LEFT | UE_SPEAKER_TOP_BACK_RIGHT;		

		// 7.dot.X
		static constexpr uint32 Surround_7_0_4 = Surround_7_0 | UE_SPEAKER_TOP_FRONT_LEFT | UE_SPEAKER_TOP_FRONT_RIGHT | UE_SPEAKER_TOP_BACK_LEFT | UE_SPEAKER_TOP_BACK_RIGHT;
		static constexpr uint32 Surround_7_1_6 = Surround_7_1 | UE_SPEAKER_TOP_FRONT_LEFT | UE_SPEAKER_TOP_FRONT_RIGHT | UE_SPEAKER_TOP_CENTER | UE_SPEAKER_TOP_BACK_LEFT | UE_SPEAKER_TOP_BACK_RIGHT | UE_SPEAKER_TOP_BACK_CENTER;
		static constexpr uint32 Surround_7_1_2 = Surround_7_1 | UE_SPEAKER_TOP_FRONT_LEFT | UE_SPEAKER_TOP_FRONT_RIGHT;
		static constexpr uint32 Surround_7_1_4 = UE_SPEAKER_FRONT_LEFT | UE_SPEAKER_FRONT_RIGHT | UE_SPEAKER_FRONT_CENTER | UE_SPEAKER_LOW_FREQUENCY | UE_SPEAKER_BACK_LEFT | UE_SPEAKER_BACK_RIGHT | UE_SPEAKER_SIDE_LEFT | UE_SPEAKER_SIDE_RIGHT | UE_SPEAKER_TOP_FRONT_LEFT | UE_SPEAKER_TOP_FRONT_RIGHT | UE_SPEAKER_TOP_BACK_LEFT | UE_SPEAKER_TOP_BACK_RIGHT;
		
		static constexpr struct FCommonFormat
		{
			uint32 Bitfield = 0;
			const TCHAR* FriendlyName=nullptr;
		} Formats[] =
		{
			{ Mono,				TEXT("Mono")},
			
			{ Stereo,			TEXT("Stereo")},
			{ Stereo_2_1,		TEXT("Stereo 2.1")},
			
			{ Surround_3_0,      TEXT("Surround 3.0 (LCR)") },
			{ Surround_3_1,      TEXT("Surround 3.1 (LCR + LFE)") },
			{ Surround_4_0,      TEXT("Surround 4.0") },
			
			{ Quad_Back,			TEXT("Quad Back")},
			{ Quad_Side,			TEXT("Quad 4.0 (sides)") },
			
			{ Surround_5_0,		TEXT("Surround 5.0")},
			{ Surround_5_1,		TEXT("Surround 5.1")},
			{ Surround_5_1_Side,	TEXT("Surround 5.1 (sides)")},
			
			{ Surround_6_0,      TEXT("Surround 6.0") },
			{ Surround_6_1,      TEXT("Surround 6.1") },
			
			{ Surround_7_0,		TEXT("Surround 7.0")},
			{ Surround_7_1,		TEXT("Surround 7.1")},

			{ Surround_5_0_2,	TEXT("Surround 5.0.2") },
			{ Surround_5_0_4,	TEXT("Surround 5.0.4") },
			{ Surround_5_1_2,    TEXT("Surround 5.1.2") },
			{ Surround_5_1_4,    TEXT("Surround 5.1.4") },
			{ Surround_7_0_4,	TEXT("Surround 7.0.4") },
			{ Surround_7_1_2,    TEXT("Surround 7.1.2") },
			{ Surround_7_1_4,    TEXT("Surround 7.1.4") },
			{ Surround_7_1_Side,	TEXT("Surround 7.1.4) (sides)") },
			{ Surround_7_1_6,	TEXT("Surround (7.1.6)") },
		};
	}
	
	#define CASE_TO_STRING(X) case X: return TEXT(#X)
	
	const TCHAR* LexToStringWavChannelMask(const uint32 InSingleSpeakerMask)
	{
		check(FMath::CountBits(InSingleSpeakerMask) == 1);
		switch (InSingleSpeakerMask)
		{
			CASE_TO_STRING( UE_SPEAKER_FRONT_LEFT              );
			CASE_TO_STRING( UE_SPEAKER_FRONT_RIGHT             );
			CASE_TO_STRING( UE_SPEAKER_FRONT_CENTER            );
			CASE_TO_STRING( UE_SPEAKER_LOW_FREQUENCY           );
			CASE_TO_STRING( UE_SPEAKER_BACK_LEFT               );
			CASE_TO_STRING( UE_SPEAKER_BACK_RIGHT              );
			CASE_TO_STRING( UE_SPEAKER_FRONT_LEFT_OF_CENTER    );
			CASE_TO_STRING( UE_SPEAKER_FRONT_RIGHT_OF_CENTER   );
			CASE_TO_STRING( UE_SPEAKER_BACK_CENTER             );
			CASE_TO_STRING( UE_SPEAKER_SIDE_LEFT               );
			CASE_TO_STRING( UE_SPEAKER_SIDE_RIGHT              );
			CASE_TO_STRING( UE_SPEAKER_TOP_CENTER              );
			CASE_TO_STRING( UE_SPEAKER_TOP_FRONT_LEFT          );
			CASE_TO_STRING( UE_SPEAKER_TOP_FRONT_CENTER        );
			CASE_TO_STRING( UE_SPEAKER_TOP_FRONT_RIGHT         );
			CASE_TO_STRING( UE_SPEAKER_TOP_BACK_LEFT           );
			CASE_TO_STRING( UE_SPEAKER_TOP_BACK_CENTER         );
			CASE_TO_STRING( UE_SPEAKER_TOP_BACK_RIGHT          );
			default : return TEXT("Unknown WavBitMask");
		}
	}
#undef CASE_TO_STRING

	const TCHAR* LexToStringCommonFormat(const uint32 InChannelMask)
	{
		for (const CommonFormats::FCommonFormat& i : CommonFormats::Formats)
		{
			if (i.Bitfield == InChannelMask)
			{
				return i.FriendlyName;
			}
		}
		return TEXT("Unknown WavFormat");
	}
	
	const TCHAR* LexToStringCommonFormat(const TArray<FName>& InChannelIds)
	{
		if (TArray<EAudioMixerChannel::Type> Channels; ChannelIdToMixerChannels(InChannelIds, Channels))
		{
			return LexToStringCommonFormat(Channels);
		}
		return TEXT("Unknown WavFormat");
	}
	const TCHAR* LexToStringCommonFormat(const TArray<EAudioMixerChannel::Type>& InChannels)
	{
		return LexToStringCommonFormat(MixerChannelsToChannelMask(InChannels));
	}

	TOptional<TArray<EAudioMixerChannel::Type>> NumChannelsToMixerChannels(const int32 InNumChannels)
	{
		if (TOptional<uint32> Result = NumChannelsToCommonChannelMask(InNumChannels); Result.IsSet())
		{
			return ChannelMaskToMixerChannels(*Result);
		}
		return {};
	}
	
	const TArray<EAudioMixerChannel::Type>& NumChannelsToMixerChannelsRef(const int32 InNumChannels)
	{
		// Keeps a static of the result of the NumChannelsToMixerChannels so we can return a ref.
		check(InNumChannels > 0); 
		check(InNumChannels <= 8);
		switch (InNumChannels)
		{
		default: { break; }
		case 1: { static TArray<EAudioMixerChannel::Type> Channels = *NumChannelsToMixerChannels(InNumChannels); return Channels; }
		case 2: { static TArray<EAudioMixerChannel::Type> Channels = *NumChannelsToMixerChannels(InNumChannels); return Channels; }
		case 3: { static TArray<EAudioMixerChannel::Type> Channels = *NumChannelsToMixerChannels(InNumChannels); return Channels; }
		case 4: { static TArray<EAudioMixerChannel::Type> Channels = *NumChannelsToMixerChannels(InNumChannels); return Channels; }
		case 5: { static TArray<EAudioMixerChannel::Type> Channels = *NumChannelsToMixerChannels(InNumChannels); return Channels; }
		case 6: { static TArray<EAudioMixerChannel::Type> Channels = *NumChannelsToMixerChannels(InNumChannels); return Channels; }
		case 7: { static TArray<EAudioMixerChannel::Type> Channels = *NumChannelsToMixerChannels(InNumChannels); return Channels; }
		case 8: { static TArray<EAudioMixerChannel::Type> Channels = *NumChannelsToMixerChannels(InNumChannels); return Channels; }
		}
		checkNoEntry();
		static TArray<EAudioMixerChannel::Type> Blank;
		return Blank;
	}

	EAudioMixerChannel::Type ChannelMaskToMixerChannel(const uint32 InChannelMask)
	{
		check(FMath::CountBits(InChannelMask) == 1);
		const uint32 BitIndex = 31 - FMath::CountLeadingZeros(InChannelMask);
		if (BitIndex >= EAudioMixerChannel::ChannelTypeCount)
		{
			return EAudioMixerChannel::Unknown;
		}
		return static_cast<EAudioMixerChannel::Type>(BitIndex);
	}
	
	TArray<EAudioMixerChannel::Type> ChannelMaskToMixerChannels(const uint32 InChannelMask)
	{
		TArray<EAudioMixerChannel::Type> Channels;
		Channels.Reserve(FMath::CountLeadingZeros(InChannelMask));
		for (int32 i=0; i < EAudioMixerChannel::ChannelTypeCount; ++i)
		{
			if (InChannelMask & (1 << i))
			{
				Channels.Emplace(static_cast<EAudioMixerChannel::Type>(i));
			}
		}
		return Channels;
	}

	uint32 MixerChannelsToChannelMask(const TArray<EAudioMixerChannel::Type>& InChannels)
	{
		uint32 ChannelMask = 0;
		for (const EAudioMixerChannel::Type i : InChannels)
		{
			ChannelMask |= 1 << i;
		}
		return ChannelMask;
	}

	TOptional<uint32> NumChannelsToCommonChannelMask(const int32 InNumChannels)
	{
		using namespace CommonFormats;
		switch (InNumChannels)
		{
		case 1: return Mono;
		case 2: return Stereo;
		case 3: return Stereo_2_1;
		case 4: return Quad_Back;
		case 5: return Surround_5_0;
		case 6: return Surround_5_1;
		case 7: return Surround_7_0;
		case 8: return Surround_7_1;
		case 12: return Surround_7_1_4;
		default: break;
		}
		return {};
	}

	FString MakePrettyString(const uint32 InChannelMask)
	{
		return MakePrettyString(ChannelMaskToMixerChannels(InChannelMask));
	}
	
	static FString CamelCaseToAcronym(const FString& InCamelCase)
	{
		TArray<TCHAR> Chars;
		Algo::TransformIf(InCamelCase, Chars, 
			[](const TCHAR Char) -> bool { return FChar::IsUpper(Char); },
			[](const TCHAR Char) -> TCHAR { return Char; });
		return FString(Chars);
	}
	
	const TCHAR* LexToShortString(const EAudioMixerChannel::Type InChannel)
	{
		// LowFrequency is an exception because we want LFE not LF.
		if (InChannel == EAudioMixerChannel::LowFrequency)
		{
			return TEXT("LFE");
		}
		using namespace EAudioMixerChannel; 
		#define CASE_TO_STATIC_SHORT(X) case X:\
		{\
			static const FString ShortName = CamelCaseToAcronym(ToString(InChannel));\
			return *ShortName;\
		}
		switch (InChannel)
		{
			FOREACH_EAUDIOMIXERCHANNEL(CASE_TO_STATIC_SHORT)
			default:
				break;		
		}
		return TEXT("Unknown");
		#undef CASE_TO_STATIC_SHORT
	}
	
	FString MakePrettyShortString(const uint32 InChannelMask)
	{
		return MakePrettyShortString(ChannelMaskToMixerChannels(InChannelMask));
	}
	
	FString MakePrettyShortString(const TArray<EAudioMixerChannel::Type>& InChannels)
	{
		return FString::JoinBy(InChannels, TEXT(", "), [](const EAudioMixerChannel::Type i) -> FString
			{
				return LexToShortString(i);
			});
	}
	
	FString MakePrettyShortString(const TArray<FName>& InChannelIds )
	{
		// If these convert to mixer channels, return short names.
		if (TArray<EAudioMixerChannel::Type> Channels; ChannelIdToMixerChannels(InChannelIds, Channels))
		{
			return MakePrettyShortString(Channels);
		}
		// can't really abbreviate safely otherwise, so return full names.
		return MakePrettyString(InChannelIds); 
	}

	FString MakePrettyString(const TArray<EAudioMixerChannel::Type>& InArray)
	{
		return FString::JoinBy(InArray, TEXT(", "), [](const EAudioMixerChannel::Type i) -> FString
		{
			return EAudioMixerChannel::ToString(i); 
		});
	}
	
	FString MakePrettyString(const TArray<FName>& InChannelIDs)
	{
		return FString::JoinBy(InChannelIDs, TEXT(", "), [](const FName i) -> FString		
		{
			return LexToString(i);
		});
	}
	
	bool ChannelIdToMixerChannels(const TArray<FName>& InChannelIds, TArray<EAudioMixerChannel::Type>& OutChannels)
	{
		bool bResult = true;
		OutChannels.Reset(InChannelIds.Num());
		Algo::Transform(InChannelIds, OutChannels, [&bResult](const FName InName) -> EAudioMixerChannel::Type
		{
			if (TOptional<EAudioMixerChannel::Type> Enum = EAudioMixerChannel::FromName(InName); Enum.IsSet())
			{
				return *Enum;
			}
			bResult = false;
			return EAudioMixerChannel::Unknown;
		});
		return bResult;	
	}
	
	void MixerChannelToChannelIds(const TArray<EAudioMixerChannel::Type>& InMixerChannels, TArray<FName>& OutChannelIds)
	{
		OutChannelIds.Reset(InMixerChannels.Num());
		Algo::Transform(InMixerChannels, OutChannelIds, [](const EAudioMixerChannel::Type InChannel) -> FName
		{
			return EAudioMixerChannel::ToName(InChannel);
		});	
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
	static TOptional<EAudioSpeakers> MixerChannelToSpeaker(const EAudioMixerChannel::Type InChannel)
	{
		using EAudioMixerChannel::Type;
		switch (InChannel)
		{
		case Type::FrontLeft:	return SPEAKER_FrontLeft;
		case Type::FrontRight:	return SPEAKER_FrontRight;
		case Type::FrontCenter:	return SPEAKER_FrontCenter;
		case Type::LowFrequency:return SPEAKER_LowFrequency;
		case Type::SideLeft: 	return SPEAKER_LeftSurround;		// These don't always map cleanly.
		case Type::SideRight:	return SPEAKER_RightSurround;	// but for the general case.
		case Type::BackLeft:	return SPEAKER_LeftBack;
		case Type::BackRight:	return SPEAKER_RightBack;
			default:
				break;
		}
		return {};
	}
	
	bool MixerChannelsToSpeakers(const TArray<EAudioMixerChannel::Type>& InChannels, TArray<EAudioSpeakers>& OutSpeakers)
	{
		bool bAllConverted = true;
		Algo::Transform(InChannels, OutSpeakers, [&bAllConverted](const EAudioMixerChannel::Type InChannel) -> EAudioSpeakers
			{
				if (TOptional<EAudioSpeakers> Speaker =  MixerChannelToSpeaker(InChannel); Speaker.IsSet())
				{
					return *Speaker;
				}
				bAllConverted = false;
				return SPEAKER_FrontLeft;	// Failed to convert, so just use front left.
			});
		return bAllConverted;
	}
	
	TArray<FName> GetSurroundCookOrder(const TArray<FName>& InChannelIds, const FString& WaveName)
	{
		// This can be promoted to cvar, but for now.
		static constexpr bool bUseLegacyCookInterleaveOrder = true;
		if (bUseLegacyCookInterleaveOrder)
		{
			// If using Legacy cook order.
			TArray<EAudioMixerChannel::Type> RequiredChannels;
			using namespace EAudioMixerChannel;
		
			switch (InChannelIds.Num())
			{
			case 4:	// Quad
				{
					RequiredChannels = { FrontLeft, FrontRight, BackLeft, BackRight };
					break;
				}
			case 6: // 5.1
				{
					RequiredChannels = { FrontLeft,  FrontRight, FrontCenter, LowFrequency, BackLeft, BackRight };
					break;
				}
			case 8: // 7.1
				{
					RequiredChannels = { FrontLeft, FrontRight, FrontCenter, LowFrequency, BackLeft, BackRight, SideLeft, SideRight };
					break;
				}
			default:
				// unsupported channel count
				checkNoEntry();
				break;
			}
		
			TArray<FName> LegacyOrder;
			TArray<FName> Missing;
			for (const Type i : RequiredChannels)
			{
				const FName Id = ToName(i);
				if (InChannelIds.Contains(Id))
				{
					LegacyOrder.Add(Id);
				}
				else
				{
					Missing.Add(Id);
				}
			}
	
			// Warn about missing channels.
			if (Missing.Num() > 0)
			{
				const FString MissingList = FString::JoinBy(Missing, TEXT(", "), [](const FName i) { return i.ToString(); } );
				const FString ContainsList = FString::JoinBy(InChannelIds, TEXT(", "), [](const FName i) { return i.ToString(); } );
				UE_LOGF(LogSignalProcessing, Warning, "Legacy Channel Order for [%d] channels requires [%ls] but got [%ls], Wave=%ls", 
					InChannelIds.Num(), *MissingList, *ContainsList, *WaveName );
			}
		
			return LegacyOrder;
		}

		// otherwise import order is fine.
		return InChannelIds;
	}

	static constexpr struct FWaveBitMaskToSpeaker
	{
		uint32 Bit;
		EAudioSpeakers Speaker;
	} Lookup []
		{
			{UE_SPEAKER_FRONT_LEFT, SPEAKER_FrontLeft},
			{UE_SPEAKER_FRONT_RIGHT, SPEAKER_FrontRight},
			{UE_SPEAKER_FRONT_CENTER, SPEAKER_FrontCenter},
			{UE_SPEAKER_LOW_FREQUENCY, SPEAKER_LowFrequency},
			{UE_SPEAKER_BACK_LEFT, SPEAKER_LeftBack},
			{UE_SPEAKER_BACK_RIGHT, SPEAKER_RightBack},
			{UE_SPEAKER_SIDE_LEFT, SPEAKER_LeftSurround},
			{UE_SPEAKER_SIDE_RIGHT, SPEAKER_RightSurround},
		};
	
#define CASE_TO_STRING(X) case X: return TEXT(#X)
	
	const TCHAR* LexToString(const EAudioSpeakers Speaker)
	{
		switch (Speaker)
		{
			CASE_TO_STRING(SPEAKER_FrontLeft);	
			CASE_TO_STRING(SPEAKER_FrontRight);
			CASE_TO_STRING(SPEAKER_FrontCenter);
			CASE_TO_STRING(SPEAKER_LowFrequency);
			CASE_TO_STRING(SPEAKER_LeftSurround);
			CASE_TO_STRING(SPEAKER_RightSurround);
			CASE_TO_STRING(SPEAKER_LeftBack);
			CASE_TO_STRING(SPEAKER_RightBack);
			CASE_TO_STRING(SPEAKER_Count);
		default: return TEXT("Unknown EAudioSpeakers");
		}
	}
	
	const TCHAR* LexToShortString(const EAudioSpeakers InSpeaker)
	{
		static const TCHAR* Names[SPEAKER_Count] = 
		{
			TEXT("FL"),		// SPEAKER_FrontLeft		
			TEXT("FR"),		// SPEAKER_FrontRight
			TEXT("FC"),		// SPEAKER_FrontCenter
			TEXT("LFE"),	// SPEAKER_LowFrequency
			TEXT("SL"),		// SPEAKER_LeftSurround
			TEXT("SR"),		// SPEAKER_RightSurround
			TEXT("BL"),		// SPEAKER_LeftBack
			TEXT("BR"),		// SPEAKER_RightBack
	   };
		static_assert (SPEAKER_Count == 8);
		check(InSpeaker >= 0 && InSpeaker < SPEAKER_Count);
		return Names[InSpeaker];
	}

	FString MakePrettyString(const TArray<EAudioSpeakers>& InSpeakers)
	{
		return FString::JoinBy(InSpeakers, TEXT(", "), [](const EAudioSpeakers i) -> FString
		{
			return LexToString(i);
		});
	}
	
	FString MakePrettyShortString(const TArray<EAudioSpeakers>& InSpeakers)
	{
		return FString::JoinBy(InSpeakers, TEXT(", "), [](const EAudioSpeakers i) -> FString
		{
			return LexToShortString(i);
		});
	}
	
	TArray<EAudioSpeakers> ChannelMaskToSpeakers(const uint32 InChannelMask)
	{
		TArray<EAudioSpeakers> Speakers;
		uint32 Mask = InChannelMask;
		for (const FWaveBitMaskToSpeaker& i : Lookup)
		{
			if (i.Bit & Mask)
			{
				Speakers.Add(i.Speaker);
			}
			Mask &= ~i.Bit;
		}

		if (Mask != 0)
		{
			TArray<uint32> MissingSpeakers;
			for (int32 i=0; i < 32; ++i)
			{
				if (1<<i & Mask)
				{
					MissingSpeakers.Add(1<<i);
				}
			}
			const FString MissingSpeakersMsg = FString::JoinBy(MissingSpeakers, TEXT(","), [](const int32 i) -> FString { return LexToStringWavChannelMask(i); }); 
			UE_LOGF(LogSignalProcessing, Warning, "Failed to map all speakers: RemainingMask=0x%x, SpeakerIDs=%ls", Mask, *MissingSpeakersMsg);
		}

		return Speakers;
	}
	uint32 SpeakersToChannelMask(const TArray<EAudioSpeakers>& InSpeakers)
	{
		uint32 Mask = 0;
		TArray<EAudioSpeakers> Speakers;
		TArrayView<const FWaveBitMaskToSpeaker> AV = MakeArrayView(Lookup);
		for (const EAudioSpeakers i : InSpeakers)
		{
			if (const FWaveBitMaskToSpeaker* Found = AV.FindByPredicate([&i](const FWaveBitMaskToSpeaker& j) -> bool
				{
					return j.Speaker == i;
				}))
			{
				Mask |= Found->Bit;
			}
			else
			{
				UE_LOGF(LogSignalProcessing, Warning, "Failed to map speaker: %ls", LexToString(i));	
			}
		}
		return Mask;
	}
	
	TArray<EAudioMixerChannel::Type> SpeakersToMixerChannels(const TArray<EAudioSpeakers>& InSpeakers)
	{
		TArray<EAudioMixerChannel::Type> Channels;
		const TArrayView<const FWaveBitMaskToSpeaker> Table = MakeArrayView(Lookup);
		for (const EAudioSpeakers i : InSpeakers)
		{
			if (const FWaveBitMaskToSpeaker* Found = Table.FindByPredicate([&i](const FWaveBitMaskToSpeaker& j) -> bool { return j.Speaker == i; }))
			{
				Channels.Add(ChannelMaskToMixerChannel(Found->Bit));
			}
		}
		return Channels;
	}
	TOptional<TArray<EAudioSpeakers>> NumChannelsToSpeakers(const int32 InNumChannels)
	{
		if (TOptional<uint32> Mask = NumChannelsToCommonChannelMask(InNumChannels))
		{
			return ChannelMaskToSpeakers(*Mask);
		}
		
		//	Fail.
		return {};
	}
	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

}

#undef CASE_TO_STRING