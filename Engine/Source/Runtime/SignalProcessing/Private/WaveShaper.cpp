// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/WaveShaper.h"
#include "DSP/Dsp.h"

namespace Audio
{
	FWaveShaper::FWaveShaper()
		: Amount(1.0f)
		, OutputGain(1.0f)
		, Bias(0.f)
		, Type(EWaveShaperType::ATan)
		, OneOverAtanAmount(1.f / (FMath::Atan(Amount)))
		, OneOverTanhAmount(1.f / Audio::FastTanh(Amount))
	{
	}

	FWaveShaper::~FWaveShaper()
	{
	}

	void FWaveShaper::Init(const float InSampleRate)
	{
	}

	void FWaveShaper::SetAmount(const float InAmount)
	{		
		Amount = FMath::Max(InAmount, 1.f);
		OneOverAtanAmount = 1.f / FMath::Atan(Amount);
		OneOverTanhAmount = 1.f / Audio::FastTanh(Amount);
	}

	void FWaveShaper::SetBias(const float InBias)
	{
		Bias = FMath::Clamp(InBias, -1.f, 1.f);
	}

	void FWaveShaper::SetOutputGainDb(const float InGainDb)
	{
		OutputGain = Audio::ConvertToLinear(InGainDb);
	}

	void FWaveShaper::SetOutputGainLinear(const float InGainLinear)
	{
		OutputGain = InGainLinear;
	}

	void FWaveShaper::SetType(const EWaveShaperType InType)
	{
		Type = InType;
	}

	void FWaveShaper::ProcessAudio(const float InSample, float& OutSample)
	{
		OutSample = OutputGain * FMath::Atan((InSample + Bias) * Amount) * OneOverAtanAmount;
	}

	void FWaveShaper::ProcessAudioBuffer(const float* InBuffer, float* OutBuffer, int32 NumFrames)
	{
		switch (Type)
		{
		case EWaveShaperType::ATan:
			ProcessATan(InBuffer, OutBuffer, NumFrames);
			break;
		case EWaveShaperType::Cubic:
			ProcessCubic(InBuffer, OutBuffer, NumFrames);
			break;
		case EWaveShaperType::Sin:
			ProcessSin(InBuffer, OutBuffer, NumFrames);
			break;
		case EWaveShaperType::HardClip:
			ProcessHardClip(InBuffer, OutBuffer, NumFrames);
			break;
		case EWaveShaperType::Tanh:
		default:
			ProcessTanh(InBuffer, OutBuffer, NumFrames);
			break;
		}
	}

	void FWaveShaper::ProcessHardClip(const float* InBuffer, float* OutBuffer, int32 NumSamples)
	{
		for (int32 Sample = 0; Sample < NumSamples; Sample++)
		{
			OutBuffer[Sample] = OutputGain * FMath::Clamp(Amount * (InBuffer[Sample] + Bias), -1.f, 1.f);
		}
	}

	void FWaveShaper::ProcessATan(const float* InBuffer, float* OutBuffer, int32 NumSamples)
	{
		for (int32 Sample = 0; Sample < NumSamples; Sample++)
		{
			OutBuffer[Sample] = OutputGain * FMath::Atan((InBuffer[Sample] + Bias) * Amount) * OneOverAtanAmount;
		}
	}

	void FWaveShaper::ProcessCubic(const float* InBuffer, float* OutBuffer, int32 NumSamples)
	{
		static const float CubicMax = 2.f / 3.f;
		static const float OneThird = 1.f / 3.f;
		float InSampleCopy = 0.f;

		for (int32 Sample = 0; Sample < NumSamples; Sample++)
		{
			InSampleCopy = (InBuffer[Sample] + Bias) * Amount;

			if (FMath::Abs(InSampleCopy) > 1.f)
			{
				OutBuffer[Sample] = FMath::Sign(InSampleCopy) * CubicMax;
			}
			else
			{
				OutBuffer[Sample] = InSampleCopy - (InSampleCopy * InSampleCopy * InSampleCopy * OneThird);
			}

			OutBuffer[Sample] *= OutputGain;
		}
	}

	void FWaveShaper::ProcessSin(const float* InBuffer, float* OutBuffer, int32 NumSamples)
	{
		float InSampleCopy = 0.f;

		for (int32 Sample = 0; Sample < NumSamples; Sample++)
		{
			InSampleCopy = (InBuffer[Sample] + Bias) * Amount;

			if (FMath::Abs(InSampleCopy) > HALF_PI)
			{
				OutBuffer[Sample] = FMath::Sign(InSampleCopy);
			}
			else
			{
				OutBuffer[Sample] = Audio::FastSin3(InSampleCopy);
			}

			OutBuffer[Sample] *= OutputGain;
		}
	}

	void FWaveShaper::ProcessTanh(const float* InBuffer, float* OutBuffer, int32 NumSamples)
	{
		for (int32 Sample = 0; Sample < NumSamples; Sample++)
		{
			OutBuffer[Sample] = OutputGain * Audio::FastTanh((InBuffer[Sample] + Bias) * Amount) * OneOverTanhAmount;
		}
	}
}
