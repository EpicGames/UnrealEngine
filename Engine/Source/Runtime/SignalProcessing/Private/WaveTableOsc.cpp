// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/WaveTableOsc.h"

namespace Audio
{
	FWaveTableOsc::FWaveTableOsc()
		: FrequencyHz(440.0f)
		, SampleRate(44100)
		, NormalPhaseReadIndex(0.0f)
		, QuadPhaseReadIndex(0.0f)
		, PhaseIncrement(0.0f)
		, OutputScale(1.0f)
		, OutputAdd(0.0f)
		, WaveTableType(EWaveTable::None)
	{
		SetFrequencyHz(FrequencyHz);
	}

	FWaveTableOsc::~FWaveTableOsc()
	{
	}

	void FWaveTableOsc::Init(const float InSampleRate, const float InFrequencyHz)
	{
		SampleRate = InSampleRate;
		FrequencyHz = InFrequencyHz;

		Reset();
		UpdateFrequency();
	}

	void FWaveTableOsc::SetSampleRate(const float InSampleRate)
	{
		SampleRate = InSampleRate;
		UpdateFrequency();
	}

	void FWaveTableOsc::SetScaleAdd(const float InScale, const float InAdd)
	{
		OutputScale = InScale;
		OutputAdd = InAdd;
	}

	void FWaveTableOsc::Reset()
	{
		// Normal phase starts at front of table
		NormalPhaseReadIndex = 0.0f;

		// Quad phase starts at 25% of the wave table buffer size
		QuadPhaseReadIndex = 0.25f * WaveTableBuffer.Num();
	}

	// Sets the frequency of the wave table oscillator.
	void FWaveTableOsc::SetFrequencyHz(const float InFrequencyHz)
	{
		FrequencyHz = InFrequencyHz;
		UpdateFrequency();
	}

	void FWaveTableOsc::UpdateFrequency()
	{
		PhaseIncrement = (float)WaveTableBuffer.Num() * FrequencyHz / (float)SampleRate;
	}

	TArray<float>& FWaveTableOsc::GetTable()
	{
		return WaveTableBuffer;
	}

	const TArray<float>& FWaveTableOsc::GetTable() const
	{
		return WaveTableBuffer;
	}

	void FWaveTableOsc::Generate(float* OutputNormalPhase, float* OutputQuadPhase)
	{
		const int32 NormPhaseReadIndexPrev = (int32)NormalPhaseReadIndex;
		const float Alpha = NormalPhaseReadIndex - NormPhaseReadIndexPrev;

		if (OutputNormalPhase)
		{
			const int32 NormPhaseReadIndexNext = (NormPhaseReadIndexPrev + 1) % WaveTableBuffer.Num();
			const float NormalPhaseOutput = FMath::Lerp(WaveTableBuffer[NormPhaseReadIndexPrev], WaveTableBuffer[NormPhaseReadIndexNext], Alpha);
			*OutputNormalPhase = NormalPhaseOutput * OutputScale + OutputAdd;
		}

		if (OutputQuadPhase)
		{
			const int32 QuadPhaseReadIndexPrev = (int32)QuadPhaseReadIndex;
			const int32 QuadPhaseReadIndexNext = (QuadPhaseReadIndexPrev + 1) % WaveTableBuffer.Num();

			float QuadPhaseOutput = FMath::Lerp(WaveTableBuffer[QuadPhaseReadIndexPrev], WaveTableBuffer[QuadPhaseReadIndexNext], Alpha);
			*OutputQuadPhase = QuadPhaseOutput * OutputScale + OutputAdd;
		}

		NormalPhaseReadIndex += PhaseIncrement;
		QuadPhaseReadIndex += PhaseIncrement;

		while (NormalPhaseReadIndex >= WaveTableBuffer.Num())
		{
			NormalPhaseReadIndex -= WaveTableBuffer.Num();
		}

		while (QuadPhaseReadIndex >= WaveTableBuffer.Num())
		{
			QuadPhaseReadIndex -= WaveTableBuffer.Num();
		}
	}

	TSharedPtr<FWaveTableOsc> FWaveTableOsc::CreateWaveTable(const EWaveTable::Type WaveTableType, const int32 WaveTableSize)
	{
		if (WaveTableType == EWaveTable::None || WaveTableSize <= 0)
		{
			return nullptr;
		}

		// Make a new wave table oscillator
		TSharedPtr<FWaveTableOsc> NewWaveTableOsc = TSharedPtr<FWaveTableOsc>(new FWaveTableOsc());

		NewWaveTableOsc->WaveTableBuffer.AddDefaulted(WaveTableSize);
		NewWaveTableOsc->WaveTableType = WaveTableType;
		NewWaveTableOsc->Reset();

		switch (WaveTableType)
		{
			case EWaveTable::SineWaveTable:
			{
				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					NewWaveTableOsc->WaveTableBuffer[i] = FMath::Sin(2.0f * PI * (float)i / WaveTableSize);
				}
			}
			break;

			case EWaveTable::SawWaveTable:
			{
				const int32 HalfTableSize = WaveTableSize / 2;
				const float Slope = 1.0f / HalfTableSize;

				// Rise to half-point, the offset by 1.0, continue to rise
				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					if (i < HalfTableSize)
					{
						NewWaveTableOsc->WaveTableBuffer[i] = Slope * i;
					}
					else
					{
						NewWaveTableOsc->WaveTableBuffer[i] = Slope * (i - HalfTableSize - 1) - 1.0f;
					}
				}
			}
			break;

			case EWaveTable::TriangleWaveTable:
			{
				const int32 QuarterTableSize = WaveTableSize / 4;
				const int32 HalfTableSize = WaveTableSize / 2;
				const int32 ThreeQuarterTableSize = 3 * WaveTableSize / 4;
				const float SlopeUp = 1.0f / QuarterTableSize;
				const float SlopeDown = -2.0f / HalfTableSize;

				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					if (i < QuarterTableSize)
					{
						NewWaveTableOsc->WaveTableBuffer[i] = SlopeUp * i - 1.0f;
					}
					else if (i < ThreeQuarterTableSize)
					{
						NewWaveTableOsc->WaveTableBuffer[i] = SlopeDown * (i - QuarterTableSize) + 1.0f;
					}
					else
					{
						NewWaveTableOsc->WaveTableBuffer[i] = SlopeUp * (i - ThreeQuarterTableSize) - 1.0f;
					}
				}
			}
			break;

			case EWaveTable::SquareWaveTable:
			{
				const int32 HalfTableSize = WaveTableSize / 2;

				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					if (i < HalfTableSize)
					{
						NewWaveTableOsc->WaveTableBuffer[i] = 1.0f;
					}
					else
					{
						NewWaveTableOsc->WaveTableBuffer[i] = -1.0f;
					}
				}
			}
			break;

			case EWaveTable::BandLimitedSawWaveTable:
			{
				float MaxSample = 0.0f;

				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					NewWaveTableOsc->WaveTableBuffer[i] = 0.0f;
					for (int32 g = 1; g <= 6; ++g)
					{
						NewWaveTableOsc->WaveTableBuffer[i] += FMath::Pow(-1.0f, (float)(g + 1)) * (1.0f / g) * FMath::Sin(2.0f * PI * i * (float)g / WaveTableSize);
					}

					if (NewWaveTableOsc->WaveTableBuffer[i] > MaxSample)
					{
						MaxSample = NewWaveTableOsc->WaveTableBuffer[i];
					}
				}

				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					NewWaveTableOsc->WaveTableBuffer[i] /= MaxSample;
				}
			}
			break;

			case EWaveTable::BandLimitedTriangleWaveTable:
			{
				float MaxSample = 0.0f;

				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					NewWaveTableOsc->WaveTableBuffer[i] = 0.0f;
					for (int g = 1; g <= 6; ++g)
					{
						NewWaveTableOsc->WaveTableBuffer[i] += FMath::Pow(-1.0f, (float)g) * (1.0f / FMath::Pow(2.0f * (float)g + 1.0f, 2.0f)) * FMath::Sin(2.0f * PI * (2.0f * (float)g + 1.0f) * i / WaveTableSize);
					}

					if (NewWaveTableOsc->WaveTableBuffer[i] > MaxSample)
					{
						MaxSample = NewWaveTableOsc->WaveTableBuffer[i];
					}
				}

				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					NewWaveTableOsc->WaveTableBuffer[i] /= MaxSample;
				}
			}
			break;

			case EWaveTable::BandLimitedSquareWaveTable:
			{
				float MaxSample = 0.0f;

				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					NewWaveTableOsc->WaveTableBuffer[i] = 0.0f;
					for (int g = 1; g <= 6; ++g)
					{
						NewWaveTableOsc->WaveTableBuffer[i] += (1.0f / g) * (float)FMath::Sin(2.0f * PI * i * (float)g / WaveTableSize);
					}

					if (NewWaveTableOsc->WaveTableBuffer[i] > MaxSample)
					{
						MaxSample = NewWaveTableOsc->WaveTableBuffer[i];
					}
				}

				for (int32 i = 0; i < WaveTableSize; ++i)
				{
					NewWaveTableOsc->WaveTableBuffer[i] /= MaxSample;
				}
			}
			break;

			case EWaveTable::Custom:
			{
				// don't do anything, the caller will fill in the table themselves
			}
			break;
		}
		return NewWaveTableOsc;
	}
}
