// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DSP/AlignedBuffer.h"
#include "DSP/BufferDiagnostics.h"

#define UE_API SIGNALPROCESSING_API

namespace Audio
{
	/**
	 ** Opaque wrapper around FAlignedFloatBuffer.
	 ** Will perform different checks on accessing the buffer, which can be enabled by flags.
	 ** Error states are sticky.
	 **/
	class FCheckedAudioBuffer
	{
	public:
		using ECheckBehavior = Audio::EBufferCheckBehavior;
				
		void SetName(const FString& InName) { DescriptiveName = InName; }
		void SetCheckBehavior(const ECheckBehavior InBehavior) { Behavior = InBehavior; }
		void SetCheckFlags(const ECheckBufferFlags InCheckFlags) { CheckFlags = InCheckFlags; }
		
		// Automatic conversions to FAlignedBuffer with a Check.
		operator FAlignedFloatBuffer&() { return GetBuffer(); };
		operator TArrayView<float>() { return MakeArrayView(GetBuffer()); };
		operator TArrayView<const float>() const { return MakeArrayView(GetBuffer()); };
		operator const FAlignedFloatBuffer&() const { return GetBuffer(); }
		UE_API void operator=(const FAlignedFloatBuffer& InOther);
		const FAlignedFloatBuffer* operator&() const { return &GetBuffer(); }
		FAlignedFloatBuffer* operator&() { return &GetBuffer(); }
				
		UE_API int32 Num() const;
		UE_API void Reserve(const int32 InSize);
		UE_API void Reset(const int32 InSize=0);
		UE_API void AddZeroed(const int32 InSize);
		UE_API void SetNumZeroed(const int32 InSize);
		UE_API void SetNumUninitialized(const int32 InNum);

		// Const.
		UE_API const FAlignedFloatBuffer& GetBuffer() const;

		// Non-const.
		UE_API FAlignedFloatBuffer& GetBuffer();
		UE_API float* GetData();
				
		UE_API void Append(const FAlignedFloatBuffer& InBuffer);
		UE_API void Append(TArrayView<const float> InView);
		UE_API void Append(const FCheckedAudioBuffer& InBuffer);
	private:
		UE_API void DoCheck(TArrayView<const float> InBuffer) const;
		
		FString DescriptiveName;			// String so this can be procedural.
		FAlignedFloatBuffer Buffer;			// Wrapped buffer.
		
		ECheckBehavior Behavior = ECheckBehavior::Nothing;
		ECheckBufferFlags CheckFlags = ECheckBufferFlags::None;
		mutable ECheckBufferFlags FailedFlags = ECheckBufferFlags::None;
		mutable bool bFailedChecks = false;
	};
}


#undef UE_API
