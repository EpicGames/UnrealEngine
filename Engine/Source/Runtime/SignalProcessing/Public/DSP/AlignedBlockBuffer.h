// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "BufferVectorOperations.h"

namespace Audio
{	
	// First In First Out Buffer designed for audio buffers. Generally performs best when buffers are greater
	// than 16 samples, optimally for buffers between 256 to 4096 samples.  Memory operations are designed 
	// for audio applications by ensuring memory allocations only occur in the constructor or when capacity 
	// is altered, memory freeing only happens in the destructor or when capacity is altered. All assignments 
	// are performed using block memory copies.
	class FAlignedBlockBuffer
	{
	public:
		
		// Constructor
		SIGNALPROCESSING_API FAlignedBlockBuffer(int32 InSampleCapacity=8192, int32 InMaxNumInspectSamples=1024, uint32 InByteAlignment=AUDIO_SIMD_BYTE_ALIGNMENT, uint32 InAllocByteAlignment=AUDIO_BUFFER_ALIGNMENT);

		// Destructor
		SIGNALPROCESSING_API ~FAlignedBlockBuffer() throw();
		
		// Maximum size of buffer
		SIGNALPROCESSING_API int32 GetSampleCapacity() const;
		
		// Amount of samples left over in buffer
		SIGNALPROCESSING_API int32 GetNumAvailable() const;

		// Add zeros to the buffer
		SIGNALPROCESSING_API void AddZeros(int32 InNum);
		
		// Add samples to the buffer.
		SIGNALPROCESSING_API void AddSamples(const float* InSamples, int32 InNum);
		
		// Remove samples from the buffer
		SIGNALPROCESSING_API void RemoveSamples(int32 InNum);

		// Clear entire buffer
		SIGNALPROCESSING_API void ClearSamples();

		// Inspect samples in the buffer. Returned samples abide by the memory alignment specified during construction.
		// Warning: Not thread safe. The returned pointer is only valid as long as this object exists 
		// and before any mutating operations are made to this object.
		SIGNALPROCESSING_API const float* InspectSamples(int32 InNum, int32 InOffset=0);
		
		// Maximum number of elements that can be inspected
		SIGNALPROCESSING_API int32 GetMaxNumInspectSamples() const;
		
	protected:

		// Set the maximum number of elements that can be inspected
		SIGNALPROCESSING_API void SetMaxNumInspectSamples(int32 InMax);

		SIGNALPROCESSING_API float* AllocateAlignedFloatArray(int32 InNum) const;
		SIGNALPROCESSING_API void FreeFloatArray(float*& Array) const;
		SIGNALPROCESSING_API void ZeroFloatArray(float* InArray, int32 InNum) const;
		SIGNALPROCESSING_API void CopyFloatArray(float* ToArray, const float* FromArray, int32 InNum) const;

	private:

		int32 CheckNumAndIncrementWriteCount(int32 InNum);

		// Hide copy and assignment operators
		FAlignedBlockBuffer& operator=(const FAlignedBlockBuffer& rhs);
		FAlignedBlockBuffer(const FAlignedBlockBuffer& CopyBuff);
		
		// Alignment definition
		uint32 AllocByteAlignment;
		uint32 ByteAlignment;
		uint32 FloatAlignment;

		// Internal memory
		float* InternalBuffer;
		float* RolloverBuffer;

		// Read and write pointers
		float* ReadPointer;
		float* WritePointer;

		int32 SampleCapacity;
		int32 MaxNumInspectSamples;
		int32 WriteCount;
		int32 WritePos;
		int32 ReadPos;
	};
}

