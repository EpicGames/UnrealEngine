// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"

// TODO: Move BufferVectorOperations AUDIO_BUFFER_ALIGNMENT
// define to more central location and reference here.
#define AUDIO_BUFFER_ALIGNMENT 16

#define AUDIO_SIMD_BYTE_ALIGNMENT (16)
#define AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER (4)


namespace Audio
{
	/** Aligned allocator used for fast operations. */	
	using FAudioBufferAlignedAllocator = TAlignedHeapAllocator<AUDIO_BUFFER_ALIGNMENT>;

	using FAlignedByteBuffer  = TArray<uint8, FAudioBufferAlignedAllocator>;
	using FAlignedFloatBuffer = TArray<float, FAudioBufferAlignedAllocator>;
	using FAlignedInt32Buffer = TArray<int32, FAudioBufferAlignedAllocator>;

	// Deprecated in favor of versions above
	typedef TArray<uint8, FAudioBufferAlignedAllocator> AlignedByteBuffer;
	typedef TArray<float, FAudioBufferAlignedAllocator> AlignedFloatBuffer;
	typedef TArray<int32, FAudioBufferAlignedAllocator> AlignedInt32Buffer;
}
