// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Delay.h"

namespace Audio
{
	// Implementation of a delay line with a feedback/feedforward gain coefficient
	// APF filters pass all frequencies but changes phase relationships of frequencies
	class FDelayAPF : public FDelay
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FDelayAPF();

		// Destructor
		SIGNALPROCESSING_API ~FDelayAPF();

		// Set the APF feedback/feedforward gain coefficient
		void SetG(float InG) { G = InG; }

		// overrides
		SIGNALPROCESSING_API virtual float ProcessAudioSample(const float pInput) override;

	protected:
		// Feedback/Feedforward gain coefficient
		float G;

	};

}
