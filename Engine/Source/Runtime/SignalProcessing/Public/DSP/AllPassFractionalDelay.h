// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BufferVectorOperations.h"
#include "AlignedBlockBuffer.h"

namespace Audio
{
	// Performs a fractional delay utilzing a single order all pass filter.
	class FAllPassFractionalDelay
	{
	public:
		// InMaxDelay sets the maximum allowable delay that this object can support. The minimum delay is 0.5
		// InNumInternalBufferSamples sets the maximum block processing size.
		SIGNALPROCESSING_API FAllPassFractionalDelay(int32 InMaxDelay, int32 InNumInternalBufferSamples);

		SIGNALPROCESSING_API ~FAllPassFractionalDelay();

		// Apply delay to InSamples. Fill OutSamples with data from the delay line at a delay of InDelay.
		// InDelay must be equal length to InSamples. 
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, const FAlignedFloatBuffer& InDelays, FAlignedFloatBuffer& OutSamples);

		// Set all values in internal delay line to zero. 
		SIGNALPROCESSING_API void Reset();

	private:
		void ProcessAudioBlock(const float* InSamples, const float* InDelays, const int32 InNum, float* OutSamples);

		int32 MaxDelay;
		int32 NumInternalBufferSamples;

		// Delay element for all pass filter.
		float Z1;

		TUniquePtr<FAlignedBlockBuffer> DelayLine;

		FAlignedFloatBuffer Coefficients;
		FAlignedFloatBuffer FractionalDelays;
		FAlignedInt32Buffer IntegerDelays;
		FAlignedInt32Buffer IntegerDelayOffsets;
	};
}
