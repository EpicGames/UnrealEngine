// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

namespace Audio
{
	SIGNALPROCESSING_API void EncodeMonoAmbisonicMixIn(TArrayView<const float> Src, TArrayView<float*> Dst, TArrayView<const float> AmbisonicGains);
	SIGNALPROCESSING_API void DecodeMonoAmbisonicMixIn(TArrayView<const float*> Src, TArrayView<float> Dst, TArrayView<const float> AmbisonicGains);
}