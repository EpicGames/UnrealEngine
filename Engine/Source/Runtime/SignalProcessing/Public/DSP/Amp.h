// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Osc.h"

namespace Audio
{
	struct FWetDry
	{
		// Level of wet signal
		float WetLevel;

		// Level of dry signal
		float DryLevel;

		FWetDry(float InWetLevel = 1.0f, float InDryLevel = 0.0f)
			: WetLevel(InWetLevel)
			, DryLevel(InDryLevel)
		{
		}
	};

	// Class which manages scaling audio input and performing panning operations
	class FAmp
	{
	public:
		SIGNALPROCESSING_API FAmp();
		SIGNALPROCESSING_API ~FAmp();

		// Initializes the amp with the mod matrix
		SIGNALPROCESSING_API void Init(const int32 InVoiceId = INDEX_NONE, FModulationMatrix* ModMatrix = nullptr);

		// Sets the direct gain in decibel
		SIGNALPROCESSING_API void SetGainDb(const float InGainDB);

		// Sets the gain modulation in decibel
		SIGNALPROCESSING_API void SetGainModDb(const float InGainModDb);

		// Sets the direct gain in linear
		SIGNALPROCESSING_API void SetGain(const float InGainLinear);

		// Sets the gain modulation in linear. Expects modulation value to be bipolar (for AM synth, etc)
		SIGNALPROCESSING_API void SetGainMod(const float InBipolarGainModLinear);

		// Sets the gain based on an envelope value
		SIGNALPROCESSING_API void SetGainEnv(const float InGainEnv);

		// Sets the gain based on an envelope value
		SIGNALPROCESSING_API void SetGainEnvDb(const float InGainEnvDb);

		// Change the allowed range of the gain output
		SIGNALPROCESSING_API void SetGainRange(const float InMin, const float InMax);
		
		// Sets the gain using a midi velocity value
		SIGNALPROCESSING_API void SetVelocity(const float InVelocity);

		// Sets the pan
		SIGNALPROCESSING_API void SetPan(const float InPan);

		// Sets the pan modulator
		SIGNALPROCESSING_API void SetPanModulator(const float InPanMod);

		// Takes mono input and generates stereo output
		SIGNALPROCESSING_API void ProcessAudio(const float LeftIn, float* LeftOutput, float* RightOutput);

		// Takes stereo input and generates stereo output
		SIGNALPROCESSING_API void ProcessAudio(const float LeftIn, const float RightIn, float* LeftOutput, float* RightOutput);

		// Generates a new gain value for left and right outputs. 
		SIGNALPROCESSING_API void Generate(float& OutGainLeft, float& OutGainRight);

		SIGNALPROCESSING_API void Reset();

		// Updates the final output left and right gain based on current settings
		SIGNALPROCESSING_API void Update();

		const FPatchDestination GetModDestGainScale() const { return GainScaleDest; }
		const FPatchDestination GetModDestGainEnv() const { return GainEnvDest; }
		const FPatchDestination GetModDestPan() const { return GainPanDest; }

	protected:
		int32 VoiceId;

		// The final left and right output gain values
		float LeftGain;
		float RightGain;

		// Values used for lerping gain values to avoid zippering
		float TargetLeftGain;
		float TargetRightGain;

		int32 TargetDeltaSamples;
		int32 CurrentLerpSample;

		float TargetLeftSlope;
		float TargetRightSlope;
		

		// Min and max output gain, can be changed based on need
		float GainMin;
		float GainMax;

		// Direct input gain control (i.e. gain slider)
		float GainControl;

		// Gain due to midi velocity
		float GainVelocity;

		// A modulated value of gain, can be used for tremolo or amplitude modulation
		float GainMod;

		// Gain set from envelope
		float GainEnv;

		// Linear pan value [-1.0, 1.0]
		float Pan;
		
		// Linear pan modulator
		float PanMod;

		// Mod matrix
		FModulationMatrix* ModMatrix;
		FPatchDestination GainScaleDest;
		FPatchDestination GainEnvDest;
		FPatchDestination GainPanDest;

		// Whether or not something changed since the last time generate was called
		bool bChanged;

	};
}
