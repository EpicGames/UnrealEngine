// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Math/Vector2D.h"
#include "DSP/AudioLinearCurve.h"

namespace Audio
{
	// Settings for audio buffer distance attenuation
	struct FAudioBufferDistanceAttenuationSettings
	{
		FAudioBufferDistanceAttenuationSettings() = default;
		~FAudioBufferDistanceAttenuationSettings() = default;

		// The min and max distance range to apply attenuation to the voip stream based on distance from listener.
		FVector2D DistanceRange = { 400.0f, 4000.0f };

		// The attenuation (in Decibels) at max range. The attenuation will be performed in dB and clamped to zero (linear) greater than max attenuation past the max range.
		float AttenuationDbAtMaxRange = -60.0f;

		// A curve (values are expected to be normalized between 0.0 and 1.0) to use to control attenuation vs distance from listener.
		Audio::FLinearCurve AttenuationCurve;
	};

	// Processes the audio buffer with the given distance attenuation
	// InOutAudioFrames - The audio buffer to attenuate. Note there is a function override for int16 and float formats.
	// InFrameCount - The number of frames of audio in the buffer
	// InNumChannels - The number of channels of audio in the buffer
	// InDistance - The current distance to use to compute the attenuation
	// InSettings - The settings to use to compute the attenuation
	// InOutAttenuation - The input value of previous attenuation and the output value after performing the current distance 
	//					  attenuation. Use -1.0 if this is the first call to this function. The value is used to interpolate the attenuation smoothly.
	SIGNALPROCESSING_API void DistanceAttenuationProcessAudio(
		TArrayView<int16>& InOutBuffer,
		uint32 InNumChannels, 
		float InDistance, 
		const FAudioBufferDistanceAttenuationSettings& InSettings, 
		float& InOutAttenuation);
	
	SIGNALPROCESSING_API void DistanceAttenuationProcessAudio(
		TArrayView<float>& InOutBuffer,
		uint32 InNumChannels, 
		float InDistance, 
		const FAudioBufferDistanceAttenuationSettings& InSettings, 
		float& InOutAttenuation);

}
