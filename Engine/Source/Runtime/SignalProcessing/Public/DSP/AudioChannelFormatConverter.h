// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/SortedMap.h"
#include "DSP/BufferVectorOperations.h"

namespace Audio
{
	/** Inteface for Channel Format Converters which process deinterleaved audio. */
	class IChannelFormatConverter
	{
		public:
			/** Description of input audio format. */
			struct FInputFormat
			{
				/** Number of channels in input audio format. */
				int32 NumChannels = 0;
			};

			/** Description of output audio format. */
			struct FOutputFormat
			{
				/** Number of channels in output audio format. */
				int32 NumChannels = 0;
			};

			virtual ~IChannelFormatConverter() = default;

			/** Return the input format handled by this converter. */
			virtual const FInputFormat& GetInputFormat() const = 0;

			/** Return the output format handled by this converter. */
			virtual const FOutputFormat& GetOutputFormat() const = 0;
			
			/** Converter the audio format from the FInputFormat to the FOutputFormat. 
			 *
			 * The input buffer array must have the same number of channels as the
			 * FInputFormat return from GetInputFormat().  Each buffer within that 
			 * array must have the same number of samples.
			 *
			 * @param InInputBuffers - An array of input audio buffers.
			 * @param OutOutputBuffers - An array of buffers where output audio is stored. 
			 */
			virtual void ProcessAudio(const TArray<FAlignedFloatBuffer>& InInputBuffers, TArray<FAlignedFloatBuffer>& OutOutputBuffers) = 0;
	};


	/** FBaseChannelFormatConverter implements channel conversion using a simple
	 * mixing matrix. 
	 */
	class FBaseChannelFormatConverter : public IChannelFormatConverter
	{

		public:
			/** FChannelMixEntry denotes how an input channel should be mixed into
			 * an output channel. 
			 */
			struct FChannelMixEntry
			{
				/** The index of the input channel to use as a source. */
				int32 InputChannelIndex = 0;

				/** The index of the output channel to use as a destination. */
				int32 OutputChannelIndex = 0;

				/** The scalar gain to apply to input audio samples when adding
				 * them to output audio samples. */
				float Gain = 0.f;
			};

			SIGNALPROCESSING_API virtual ~FBaseChannelFormatConverter();

			/** Return the input format handled by this converter. */
			SIGNALPROCESSING_API const FInputFormat& GetInputFormat() const override;

			/** Return the output format handled by this converter. */
			SIGNALPROCESSING_API const FOutputFormat& GetOutputFormat() const override;

			/** Sets the output gain scalar to apply to all output audio. 
			 *
			 * @param InOutputGain - Scalar gain value. 
			 * @param bFadeToGain - If true, gain values are linearly faded over 
			 *                      the duration of a single buffer. If false,
			 *                      gain values are applied immediately. 
			 */
			SIGNALPROCESSING_API void SetOutputGain(float InOutputGain, bool bFadeToGain=true);

			/** Sets the gain scalar to apply to a specific input/output channel pair.
			 *
			 * @param InMixEntry - Description of mix routing and gain.
			 * @param bFadeToGain - If true, gain values are linearly faded over 
			 *                      the duration of a single buffer. If false,
			 *                      gain values are applied immediately. 
			 */
			SIGNALPROCESSING_API void SetMixGain(const FChannelMixEntry& InEntry, bool bFadeToGain=true);

			/** Sets the gain scalar to apply to a specific input/output channel pair.
			 *
			 * @param InInputChannelIndex - The index of the source channel audio. 
			 * @param InOutputChannelIndex - The index of the destination channel audio. 
			 * @param InGain - The scalar gain to apply to the source channel before
			 * 				   adding it to the destination channel.
			 * @param bFadeToGain - If true, gain values are linearly faded over 
			 *                      the duration of a single buffer. If false,
			 *                      gain values are applied immediately. 
			 */
			SIGNALPROCESSING_API void SetMixGain(int32 InInputChannelIndex, int32 InOutputChannelIndex, float InGain, bool bFadeToGain=true);

			/** Returns the scalar gain used to mix the input channel index into 
			 * the output channel index.  
			 *
			 * If the gains are to be faded during the subsequent call to ProcessAudio(),
			 * this will return the ending gain.
			 */
			SIGNALPROCESSING_API float GetTargetMixGain(int32 InInputChannelIndex, int32 InOutputChannelIndex) const;

			/** Returns the scalar gain applied to the output audio.
			 *
			 * If the gain is to be faded during the subsequent call to ProcessAudio(),
			 * this will return the ending gain.
			 */
			SIGNALPROCESSING_API float GetTargetOutputGain() const;

			/** Converter the audio format from the FInputFormat to the FOutputFormat. 
			 *
			 * The input buffer array must have the same number of channels as the
			 * FInputFormat return from GetInputFormat().  Each buffer within that 
			 * array must have `InNumFramesPerCall` as set on creation. 
			 *
			 * @param InInputBuffers - An array of input audio buffers.
			 * @param OutOutputBuffers - An array of buffers where output audio is stored. 
			 */
			SIGNALPROCESSING_API void ProcessAudio(const TArray<FAlignedFloatBuffer>& InInputBuffers, TArray<FAlignedFloatBuffer>& OutOutputBuffers) override;


			/** Create a FBaseChannelFormatConverter
			 *
			 * @param InInputFormat - The format of the input audio.
			 * @param InOutputFormat - The desired output audio format.
			 * @param InMixEntries - An array of mixing values to use to mix input audio to output audio.
			 * @param InNumFramesPerCall - The number of frames used for each call to ProcessAudio.
			 *
			 * @return A TUniquePtr to a FBaseChannelFormatConverter.
			 */
			static SIGNALPROCESSING_API TUniquePtr<FBaseChannelFormatConverter> CreateBaseFormatConverter(const FInputFormat& InInputFormat, const FOutputFormat& InOutputFormat, TArrayView<const FChannelMixEntry> InMixEntries, int32 InNumFramesPerCall);

		protected:

			SIGNALPROCESSING_API FBaseChannelFormatConverter(const FInputFormat& InInputFormat, const FOutputFormat& InOutputFormat, TArrayView<const FChannelMixEntry> InMixEntries, int32 InNumFramesPerCall);

		private:

			// Output gain state holds current and next gain levels. 
			struct FOutputGainState
			{
				float Gain = 1.f;
				float NextGain = 0.f;
				bool bFadeToNextGain = false;
			};

			// Channel mix entry holds current and next gain levels for a mix entry.
			struct FChannelMixState : FChannelMixEntry
			{
				FChannelMixState() = default;

				FChannelMixState(const FChannelMixEntry& InEntry)
				:	FChannelMixEntry(InEntry)
				{
				}

				bool bFadeToNextGain = false;
				float NextGain = 0.f;
			};

			// Channel mix key is used to lookup entries from a TSortedMap
			struct FChannelMixKey
			{
				const int32 InputChannelIndex = 0;
				const int32 OutputChannelIndex = 0;

				FChannelMixKey(int32 InInputChannelIndex, int32 InOutputChannelIndex)
				:	InputChannelIndex(InInputChannelIndex)
				,	OutputChannelIndex(InOutputChannelIndex)
				{
				}

				FChannelMixKey(const FChannelMixEntry& InEntry)
				:	InputChannelIndex(InEntry.InputChannelIndex)
				,	OutputChannelIndex(InEntry.OutputChannelIndex)
				{
				}

				// Operator needed for using in a TSortedMap
				friend bool operator<(const FChannelMixKey& InLHS, const FChannelMixKey& InRHS)
				{
					if (InLHS.InputChannelIndex == InRHS.InputChannelIndex)
					{
						return InLHS.OutputChannelIndex < InRHS.OutputChannelIndex;
					}

					return InLHS.InputChannelIndex < InRHS.InputChannelIndex;
				}
			};

			// The input audio format
			FInputFormat InputFormat;

			// The output audio format. 
			FOutputFormat OutputFormat;

			// Gain states 
			FOutputGainState OutputGainState;
			TSortedMap<FChannelMixKey, FChannelMixState> ChannelMixStates;

			int32 NumFramesPerCall;
	};
}
