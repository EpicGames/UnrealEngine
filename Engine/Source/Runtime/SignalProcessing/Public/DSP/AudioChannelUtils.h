// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioMixerChannel.h"

#define UE_API SIGNALPROCESSING_API

namespace Audio::ChannelUtils
{	
	/** Helper function to get the channel map type at the given index. */
	UE_API bool GetChannelTypeAtIndex(const int32 Index, EAudioMixerChannel::Type& OutType);
	
	/** Helper function to get the channel weight given a channel type. */
	UE_API float GetChannelWeight(EAudioMixerChannel::Type InType);
} // namespace Audio::ChannelUtils

#undef UE_API
