// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define USE_AUDIO_DEBUGGING UE_BUILD_DEBUG

SIGNALPROCESSING_API void BreakWhenAudible(float* InBuffer, int32 NumSamples);
SIGNALPROCESSING_API void BreakWhenTooLoud(float* InBuffer, int32 NumSamples);

#if USE_AUDIO_DEBUGGING
#define BREAK_WHEN_AUDIBLE(Ptr, Num) BreakWhenAudible(Ptr, Num);
#define BREAK_WHEN_TOO_LOUD(Ptr, Num) BreakWhenTooLoud(Ptr, Num);
#else
#define BREAK_WHEN_AUDIBLE(Ptr, Num) 
#define BREAK_WHEN_TOO_LOUD(Ptr, Num)
#endif
