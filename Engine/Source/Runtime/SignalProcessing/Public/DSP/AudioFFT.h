// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreMiscDefines.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/FFTAlgorithm.h"

namespace Audio
{
	// Types of spectrums which can be directly derived from FFTFreqDomainData
	
	enum class ESpectrumType : uint8
	{
		MagnitudeSpectrum,
		PowerSpectrum
	};


	enum class EWindowType : uint8
	{
		None, // No window is applied. Technically a boxcar window.
		Hamming, // Mainlobe width of -3 dB and sidelove attenuation of ~-40 dB. Good for COLA.
		Hann, // Mainlobe width of -3 dB and sidelobe attenuation of ~-30dB. Good for COLA.
		Blackman // Mainlobe width of -3 dB and sidelobe attenuation of ~-60db. Tricky for COLA.
	};

	// Utility functions for generating different types of windows. Called in FWindow::Generate.
	SIGNALPROCESSING_API void GenerateRectangularWindow(float* WindowBuffer, int32 NumFrames, int32 NumChannels);
	SIGNALPROCESSING_API void GenerateHammingWindow(float* WindowBuffer, int32 NumFrames, int32 NumChannels, bool bIsPeriodic);
	SIGNALPROCESSING_API void GenerateHannWindow(float* WindowBuffer, int32 NumFrames, int32 NumChannels, bool bIsPeriodic);
	SIGNALPROCESSING_API void GenerateBlackmanWindow(float* WindowBuffer, int32 NumFrames, int32 NumChannels, bool bIsPeriodic);
	SIGNALPROCESSING_API void GenerateWindow(float* InOutWindowBuffer, EWindowType InType, int32 InNumFrames, int32 InNumChannels, bool bIsPeriodic);

	// Returns the hop size in samples necessary to maintain constant overlap add.
	// For more information on COLA, see the following page:
	// https://ccrma.stanford.edu/~jos/sasp/Overlap_Add_OLA_STFT_Processing.html
	SIGNALPROCESSING_API uint32 GetCOLAHopSizeForWindow(EWindowType InType, uint32 WindowLength);

	/**
	 * Class used to generate, contain and apply a DSP window of a given type.
	 */
	class FWindow
	{
	public:
		FWindow() = delete;

		/**
		 * Constructor. Allocates buffer and generates window inside of it.
		 * @param InType: The type of window that should be generated.
		 * @param InNumFrames: The number of samples that should be generated divided by the number of channels.
		 * @param InNumChannels: The amount of channels that will be used in the signal this is applied to.
		 * @param bIsPeriodic: If false, the window will be symmetrical. If true, the window will be periodic.
		 *                     Generally, set this to false if using this window with an STFT, but use true
		 *                     if this window will be used on an entire, self-contained signal.
		 */
		SIGNALPROCESSING_API FWindow(EWindowType InType, int32 InNumFrames, int32 InNumChannels, bool bIsPeriodic);

		// Apply this window to InBuffer, which is expected to be an interleaved buffer with the same amount of frames
		// and channels this window was constructed with.
		SIGNALPROCESSING_API void ApplyToBuffer(float* InBuffer);

		SIGNALPROCESSING_API EWindowType GetWindowType() const;

	private:
		EWindowType WindowType;
		FAlignedFloatBuffer WindowBuffer;
		int32 NumSamples;

		// Generate the window. Called on constructor.
		SIGNALPROCESSING_API void Generate(int32 NumFrames, int32 NumChannels, bool bIsPeriodic);
	};

	// Return the ceiling of the log2 of InNum
	SIGNALPROCESSING_API int32 CeilLog2(int32 InNum);

	// Return the scaling factor needed to apply to a power spectrum given a current
	// and target FFT scaling. 
	SIGNALPROCESSING_API float GetPowerSpectrumScaling(int32 FFTSize, EFFTScaling InCurrentScaling, EFFTScaling InTargetScaling);

	// Scale the power spectrum to remove any scaling introduced by the FFT algorithm
	// implementation.
	SIGNALPROCESSING_API void ScalePowerSpectrumInPlace(int32 FFTSize, EFFTScaling InCurrentScaling, EFFTScaling InTargetScaling, TArrayView<float> InPowerSpectrum);
}
