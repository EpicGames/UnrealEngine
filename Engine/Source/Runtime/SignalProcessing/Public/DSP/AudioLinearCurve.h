// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Vector2D.h"

namespace Audio
{
	// A simple curve class that takes FVector2D points
	// Required to decouple from Engine module
	class FLinearCurve
	{
	public:
		FLinearCurve() = default;
		SIGNALPROCESSING_API FLinearCurve(const TArray<FVector2D>& InCurve);
		SIGNALPROCESSING_API FLinearCurve(TArray<FVector2D>&& InCurve);

		virtual ~FLinearCurve() = default;

		// Sets curve array
		SIGNALPROCESSING_API void SetCurvePoints(const TArray<FVector2D>& InPoints);
		SIGNALPROCESSING_API void SetCurvePoints(TArray<FVector2D>&& InPoints);

		// Clears the points array
		SIGNALPROCESSING_API void ClearPoints();

		// Returns the number of points in the curve
		int32 Num() const { return Points.Num(); }

		// Adds individual points
		SIGNALPROCESSING_API void AddPoint(const FVector2D& InPoint);
		SIGNALPROCESSING_API void AddPoints(const TArray<FVector2D>& InPoints);

		// Retrieves curve array
		SIGNALPROCESSING_API const TArray<FVector2D>& GetCurvePoints() const;

		// Evaluates the curve with the given domain. 
		// Returns false if the curve was incapable of evaluation (i.e. empty).
		// Will clamp the output to the values of the domain
		SIGNALPROCESSING_API bool Eval(float InDomain, float& OutValue) const;

	private:

		// Will sort the curve according to increasing domain values
		SIGNALPROCESSING_API void Sort();
		TArray<FVector2D> Points;
	};
}
