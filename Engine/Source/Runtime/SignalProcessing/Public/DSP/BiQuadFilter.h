// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"

namespace Audio
{
	// Simple biquad filter structure handling a biquad formulation
	// See: https://en.wikipedia.org/wiki/Digital_biquad_filter
	// Calculations of coefficients are handled outside this class.
	// Filter coefficients are public and are intended to be used externally.
	class UE_DEPRECATED(5.5, "Audio::FBiquad is deprecated in favor of Audio::FBiquadFilter due to performance issues") FBiquad
	{
	public:
		FBiquad()
			: A0(1.0f)
			, A1(0.0f)
			, A2(0.0f)
			, B1(0.0f)
			, B2(0.0f)
		{
			Reset();
		}

		inline float ProcessAudio(const float InSample)
		{
			// Use the biquad difference eq: y(n) = a0*x(n) + a1*x(n-1) + a2*x(n-2) - b1*y(n-1) - b2*y(n-2) 
			const float Output = A0 * InSample + A1 * X_Z1 + A2 * X_Z2 - B1 * Y_Z1 - B2 * Y_Z2;

			// Apply the z-transforms
			Y_Z2 = Y_Z1;
			Y_Z1 = Output;

			X_Z2 = X_Z1;
			X_Z1 = InSample;

			return Output;
		}

		// Reset the filter (flush delays)
		void Reset()
		{
			X_Z1 = 0.0f;
			X_Z2 = 0.0f;
			Y_Z1 = 0.0f;
			Y_Z2 = 0.0f;
		}

	public:
		// Biquad filter coefficients
		float A0;
		float A1;
		float A2;
		float B1;
		float B2;

	protected:
		float X_Z1; // previous inputs z-transforms
		float X_Z2;
		float Y_Z1; // prvious outputs z-transforms
		float Y_Z2;
	};
}
