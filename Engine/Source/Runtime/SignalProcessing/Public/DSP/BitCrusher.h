// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Osc.h"

namespace Audio
{
	// Bit crushing effect
	// https://en.wikipedia.org/wiki/Bitcrusher
	class FBitCrusher
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FBitCrusher();

		// Destructor
		SIGNALPROCESSING_API ~FBitCrusher();

		// Initialize the equalizer
		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumChannels);

		// The amount to reduce the sample rate of the audio stream.
		SIGNALPROCESSING_API void SetSampleRateCrush(const float InFrequency);

		// The amount to reduce the bit depth of the audio stream.
		SIGNALPROCESSING_API void SetBitDepthCrush(const float InBitDepth);

		// Processes audio
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame);
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer);

		// Returns the maximum value given bit depths will be clamped to.
		static SIGNALPROCESSING_API float GetMaxBitDepth();

	private:
		// i.e. 8 bit, etc. But can be float!
		float SampleRate;
		float BitDepth;
		float BitDelta;
		float ReciprocalBitDelta;

		// The current phase of the bit crusher
		float Phase;

		// The amount of phase to increment each sample
		float PhaseDelta;

		// Used to sample+hold the last output
		float LastOutput[2];

		int32 NumChannels;
	};

}
