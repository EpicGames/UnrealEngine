// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/AudioFFT.h"
#include "DSP/FFTAlgorithm.h"

namespace Audio
{	
	/** Settings for block correlator */
	struct FBlockCorrelatorSettings
	{
		/** Logarithm base 2 of the number of values in a block.*/
		int32 Log2NumValuesInBlock = 10;  // 2^10 = 1024

		/** Audio window type to apply to input. */
		EWindowType WindowType = EWindowType::Blackman;

		/** If true, autocorrelation values will be normalized to reduce bias introduced by block windowing. */
		bool bDoNormalize = true;
	};

	/** Correlation calculator which utilizes FFT to perform fast correlation calculations. */
	class FBlockCorrelator
	{
		public:
			/** Construct a block correlator with FBlockCorrelatorSettings */
			SIGNALPROCESSING_API FBlockCorrelator(const FBlockCorrelatorSettings& InSettings);

			/** Return settings used to construct block correlator. */
			SIGNALPROCESSING_API const FBlockCorrelatorSettings& GetSettings() const;

			/** Returns the number of input values to use when calling CrossCorrelate(...) or AutoCorrelate(...) */
			SIGNALPROCESSING_API int32 GetNumInputValues() const;

			/** Returns the number of output values to use when calling CrossCorrelate(...) or AutoCorrelate(...) */
			SIGNALPROCESSING_API int32 GetNumOutputValues() const;

			/** Cross correlate two input signals. 
			 *
			 * The Output array is filled with the cross correlation between InputA and InputB. Care should be taken 
			 * when interpreting the output as the elements are ordered so that the cross correlation value of zero
			 * lag is in the first element.
			 *
			 * CrossCorr[k] = Sum_n(InputA[n] * InputB[n + k])
			 *
			 * Then for M = `GetNumInputValues()` the Output array contains:
			 * Output = [CrossCorr[0], CrossCorr[1], ..., CrossCorr[M-1], CrossCorr[M], CrossCorr[-M + 1], CrossCorr[-M + 2], ..., CrossCorr[-2], CrossCorr[-1]]
			 *
			 * @param InputA - First input block with `GetNumInputValues()` elements. 
			 * @param InputB - Second input block with `GetNumInputValues()` elements. 
			 * @param Output - Output block with `GetNumOutputValues()` elements.
			 */
			SIGNALPROCESSING_API void CrossCorrelate(const FAlignedFloatBuffer& InputA, const FAlignedFloatBuffer& InputB, FAlignedFloatBuffer& Output);

			/** Autocorrelate an input signal. 
			 *
			 * The Output array is filled with the autocorrelation of the Input. Care should be taken when interpreting the 
			 * output as the elements are ordered so that the autocorrelation value of zero lag is in the first element.
			 *
			 * AutoCorr[k] = Sum_n(Input[n] * Input[n + k])
			 *
			 * Then for M = `GetNumInputValues()` the Output array contains:
			 * Output = [AutoCorr[0], AutoCorr[1], ..., AutoCorr[M-1], AutoCorr[M], AutoCorr[-M + 1], AutoCorr[-M + 2], ..., AutoCorr[-2], AutoCorr[-1]]
			 *
			 * @param Input - input block with `GetNumInputValues()` elements. 
			 * @param Output - Output block with `GetNumOutputValues()` elements.
			 */
			SIGNALPROCESSING_API void AutoCorrelate(const FAlignedFloatBuffer& Input, FAlignedFloatBuffer& Output);


		private:
			// No normalization or windowing.
			void CyclicCrossCorrelate(const FAlignedFloatBuffer& InputA, const FAlignedFloatBuffer& InputB, FAlignedFloatBuffer& Output);

			// No normalization or windowing.
			void CyclicAutoCorrelate(const FAlignedFloatBuffer& Input, FAlignedFloatBuffer& Output);

			void InitializeNormalizationBuffer();

			FBlockCorrelatorSettings Settings;

			int32 NumValuesInBlock;
			int32 NumValuesInFFTRealBuffer;
			int32 NumValuesInFFTComplexBuffer;

			TUniquePtr<IFFTAlgorithm> FFTAlgorithm;

			FWindow Window;
			FAlignedFloatBuffer NormalizationBuffer;

			FAlignedFloatBuffer WindowedBufferA;
			FAlignedFloatBuffer WindowedBufferB;

			FAlignedFloatBuffer ComplexBufferA;
			FAlignedFloatBuffer ComplexBufferB;

			FAlignedFloatBuffer ComplexCorrelationBuffer;

			FAlignedFloatBuffer FullOutputBuffer;
	};
}
