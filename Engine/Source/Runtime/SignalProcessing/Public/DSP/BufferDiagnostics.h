// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/List.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "HAL/PreprocessorHelpers.h"
#include "Math/Float32.h"
#include "Templates/Function.h"

namespace Audio
{
	// IEEE Standard 754 Floating Point Numbers
	// https://www.geeksforgeeks.org/ieee-standard-754-floating-point-numbers/

	/*
	* Denormalized.
	*/
	inline static bool IsDenormalized(const float InValue)
	{
		// If the exponent is all zeros, but the mantissa is not then the value is a denormalized number.
		// This means this number does not have an assumed leading one before the binary point.
		const FFloat32 F(InValue);
		return F.Components.Exponent == 0 && F.Components.Mantissa != 0;
	}

	SIGNALPROCESSING_API int32 FindDenormalized(TArrayView<const float> InBuffer);

	SIGNALPROCESSING_API bool ContainsDenormalized(TArrayView<const float> InBuffer);

	/*
	 * Infinity.
	*/
	inline static bool IsInfinity(const float InValue)
	{
		// The values + infinity and -infinity are denoted with an exponent of all ones and a mantissa of all zeros.
		// The sign bit distinguishes between negative infinity and positive infinity.
		const FFloat32 F(InValue);
		return F.Components.Exponent == 255 && F.Components.Mantissa == 0;
	}

	SIGNALPROCESSING_API int32 FindInfinity(TArrayView<const float> InBuffer);
	SIGNALPROCESSING_API bool ContainsInfinity(TArrayView<const float> InBuffer);

	/*
	* NaN (Not a number)
	*/
	SIGNALPROCESSING_API int32 FindNan(TArrayView<const float> InBuffer);

	SIGNALPROCESSING_API bool ContainsNan(TArrayView<const float> InBuffer);

	enum class ECheckBufferFlags : uint32
	{
		None = 0,
		Infinity = 1 << 1,
		Nan = 1 << 2,
		Denormalized = 1 << 3,
		All = Infinity | Nan | Denormalized
	};

	inline ECheckBufferFlags operator|(const ECheckBufferFlags A, const ECheckBufferFlags B)
	{
		return static_cast<ECheckBufferFlags>(static_cast<uint32>(A) | static_cast<uint32>(B));
	}

	inline ECheckBufferFlags& operator|=(ECheckBufferFlags& Out, const ECheckBufferFlags Other)
	{
		return Out = Out | Other;
	}

	inline ECheckBufferFlags operator&(const ECheckBufferFlags A, const ECheckBufferFlags B)
	{
		return static_cast<ECheckBufferFlags>(static_cast<uint32>(A) & static_cast<uint32>(B));
	}

	// ToString with deliminators i.e. "Nan|Infinity" etc.
	FString ToDelimitedString(const ECheckBufferFlags InEnum);

	/**
	 * Performs various tests on a audio buffer.
	 * @param InBuffer Buffer to examine
	 * @parm InFlags Bitfield of checks to perform
	 * @return true if check passes, false otherwise
	**/
	SIGNALPROCESSING_API bool CheckBuffer(TArrayView<const float> InBuffer, const ECheckBufferFlags InFlags, ECheckBufferFlags& OutFailedFlags);

	enum class EBufferCheckBehavior : uint8
	{
		Nothing,
		Ensure,
		Log,
		Break,
	};

	struct FCheckedBufferState
	{
		// Statics.
		static FCriticalSection ListCs;	// To protect linked-list.
		static FCheckedBufferState* Head;
		static void ForEach(TFunctionRef<void (FCheckedBufferState&)> InCmd);
		
		// Per check state.
		FCheckedBufferState* Next = nullptr;
		const TCHAR* Name = nullptr;
		int32 Line = 0;
		const TCHAR* File = nullptr;
		ECheckBufferFlags CheckFlags = ECheckBufferFlags::None;
		ECheckBufferFlags FailFlags = ECheckBufferFlags::None;
		EBufferCheckBehavior Behavior = EBufferCheckBehavior::Ensure;

		SIGNALPROCESSING_API FCheckedBufferState(const int32 InLine, const TCHAR* InFile, const TCHAR* InName,
			const ECheckBufferFlags InCheckFlags = ECheckBufferFlags::None,
			const EBufferCheckBehavior InCheckBehavior = EBufferCheckBehavior::Ensure);

		// The CHECK_AUDIO_BUFFER Macros calls these two member functions.
		SIGNALPROCESSING_API bool DoCheck(TArrayView<const float> InBuffer);
		SIGNALPROCESSING_API void FailedBufferCheckImpl(const TCHAR* InFormat, ...) const;
	};

	
}

#ifndef WITH_AUDIO_BUFFERDIAGNOSTICS
// By default enabled in all but ship
#define WITH_AUDIO_BUFFERDIAGNOSTICS (!UE_BUILD_SHIPPING)
#endif //WITH_AUDIO_BUFFERDIAGNOSTICS

#if WITH_AUDIO_BUFFERDIAGNOSTICS
#define AUDIO_CHECK_BUFFER(BUFFER)\
	static FCheckedBufferState UE_JOIN(BufferCheck, __LINE__)(__LINE__,TEXT(__FILE__),TEXT(UE_STRINGIZE(BUFFER)));\
	if (!UE_JOIN(BufferCheck, __LINE__).DoCheck(BUFFER) )\
	{\
		UE_JOIN(BufferCheck, __LINE__).FailedBufferCheckImpl(TEXT(""));\
	}

#define AUDIO_CHECK_BUFFER_NAMED(BUFFER,NAME)\
	static FCheckedBufferState UE_JOIN(BufferCheck, __LINE__)(__LINE__,TEXT(__FILE__),NAME);\
	if (!UE_JOIN(BufferCheck, __LINE__).DoCheck(BUFFER) )\
	{\
		UE_JOIN(BufferCheck, __LINE__).FailedBufferCheckImpl(TEXT(""));\
	}

#define AUDIO_CHECK_BUFFER_NAMED_MSG(BUFFER,NAME,MSG,...)\
	static FCheckedBufferState UE_JOIN(BufferCheck, __LINE__)(__LINE__,TEXT(__FILE__),NAME);\
	if (!UE_JOIN(BufferCheck, __LINE__).DoCheck(BUFFER))\
	{\
		UE_JOIN(BufferCheck, __LINE__).FailedBufferCheckImpl(MSG,__VA_ARGS__);\
	}
#else //WITH_AUDIO_BUFFERDIAGNOSTICS
	#define AUDIO_CHECK_BUFFER_NAMED(BUFFER,NAME)
	#define AUDIO_CHECK_BUFFER(BUFFER)
	#define AUDIO_CHECK_BUFFER_NAMED_MSG(BUFFER,NAME,MSG,...)
#endif //WITH_AUDIO_BUFFERDIAGNOSTICS
