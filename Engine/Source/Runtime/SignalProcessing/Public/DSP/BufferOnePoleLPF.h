// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "DSP/BufferVectorOperations.h"

namespace Audio
{

	// Simple 1-pole lowpass filter
	class FBufferOnePoleLPF
	{
	public:

		// Constructor 
		SIGNALPROCESSING_API FBufferOnePoleLPF(float InG = 0.0f);

		// Set the LPF gain coefficient
		SIGNALPROCESSING_API void SetG(float InG);

		// Resets the sample delay to 0
		SIGNALPROCESSING_API void Reset();

		// Resets the audio to silence.
		SIGNALPROCESSING_API void FlushAudio();

		// Sets the filter frequency using normalized frequency (between 0.0 and 1.0f or 0.0 hz and Nyquist Frequency in Hz) 
		SIGNALPROCESSING_API void SetFrequency(const float InFrequency);

		// Process InSamples and place filtered audio in OutSamples
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples);

	protected:
		float CutoffFrequency;

		// Filter coefficients
		float B1;
		float A0;

		// 1-sample delay
		float Z1;
	};

	

}
