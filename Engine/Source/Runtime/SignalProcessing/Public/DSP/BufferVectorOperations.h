// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AlignedBuffer.h"
#include "Containers/Array.h"
#include "Misc/CoreMiscDefines.h"

namespace Audio
{
	/** CHANNEL-SPECIFIC OPERATIONS */

	/** Takes a 2 channel interleaved buffer and applies Gains to it. Gains is expected to point to a 2 float long buffer.
	 *  StereoBuffer must have an even number of frames.
	 *  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void Apply2ChannelGain(FAlignedFloatBuffer& StereoBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Apply2ChannelGain(float* RESTRICT StereoBuffer, int32 NumSamples, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Apply2ChannelGain(FAlignedFloatBuffer& StereoBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void Apply2ChannelGain(float* RESTRICT StereoBuffer, int32 NumSamples, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 1 channel buffer and mixes it to a stereo buffer using Gains. Gains is expected to point to a 2 float long buffer.
	 *  these buffers must have an even number of frames.
	 *  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void MixMonoTo2ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void MixMonoTo2ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void MixMonoTo2ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void MixMonoTo2ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void MixMonoTo2ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer);
	SIGNALPROCESSING_API void MixMonoTo2ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames);

	/** Takes a 2 channel buffer and mixes it to an 2 channel interleaved buffer using Gains. Gains is expected to point to a 16 float long buffer.
	*  Output gains for the left input channel should be the first 8 values in Gains, and Output gains for the right input channel should be rest.
	*  NumFrames must be a multiple of 4.
	*  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void Mix2ChannelsTo2ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Mix2ChannelsTo2ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Mix2ChannelsTo2ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void Mix2ChannelsTo2ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 4 channel interleaved buffer and applies Gains to it. Gains is expected to point to a 2 float long buffer.
	 *  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void Apply4ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Apply4ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Apply4ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void Apply4ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 1 channel buffer and mixes it to an 8 channel interleaved buffer using Gains. Gains is expected to point to a 8 float long buffer.
	*  these buffers must have an even number of frames.
	*  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void MixMonoTo4ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void MixMonoTo4ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void MixMonoTo4ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void MixMonoTo4ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 2 channel buffer and mixes it to an 8 channel interleaved buffer using Gains. Gains is expected to point to a 16 float long buffer.
	*  Output gains for the left input channel should be the first 8 values in Gains, and Output gains for the right input channel should be rest.
	*  NumFrames must be a multiple of 4.
	*  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void Mix2ChannelsTo4ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Mix2ChannelsTo4ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Mix2ChannelsTo4ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void Mix2ChannelsTo4ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 6 channel interleaved buffer and applies Gains to it. Gains is expected to point to a 2 float long buffer.
	 *  InterleavedBuffer must have an even number of frames.
	*/
	SIGNALPROCESSING_API void Apply6ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Apply6ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Apply6ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void Apply6ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 1 channel buffer and mixes it to an 8 channel interleaved buffer using Gains. Gains is expected to point to a 8 float long buffer.
	 *  these buffers must have an even number of frames.
	 *  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void MixMonoTo6ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void MixMonoTo6ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void MixMonoTo6ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void MixMonoTo6ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 2 channel buffer and mixes it to an 8 channel interleaved buffer using Gains. Gains is expected to point to a 16 float long buffer.
	 *  Output gains for the left input channel should be the first 8 values in Gains, and Output gains for the right input channel should be rest.
	 *  NumFrames must be a multiple of 4.
	 *  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void Mix2ChannelsTo6ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Mix2ChannelsTo6ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Mix2ChannelsTo6ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void Mix2ChannelsTo6ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes an 8 channel interleaved buffer and applies Gains to it. Gains is expected to point to an 8 float long buffer. */
	SIGNALPROCESSING_API void Apply8ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Apply8ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Apply8ChannelGain(FAlignedFloatBuffer& InterleavedBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void Apply8ChannelGain(float* RESTRICT InterleavedBuffer, int32 NumSamples, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 1 channel buffer and mixes it to an 8 channel interleaved buffer using Gains. Gains is expected to point to a 8 float long buffer.
	 *  these buffers must have an even number of frames.
	 *  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void MixMonoTo8ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void MixMonoTo8ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void MixMonoTo8ChannelsFast(const FAlignedFloatBuffer& MonoBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void MixMonoTo8ChannelsFast(const float* RESTRICT MonoBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** Takes a 2 channel buffer and mixes it to an 8 channel interleaved buffer using Gains. Gains is expected to point to a 16 float long buffer.
	 *  Output gains for the left input channel should be the first 8 values in Gains, and Output gains for the right input channel should be rest.
	 *  these buffers must have an even number of frames.
	 *  If StartGains and EndGains are provided, this function will interpolate between the two across the buffer.
	*/
	SIGNALPROCESSING_API void Mix2ChannelsTo8ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Mix2ChannelsTo8ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void Mix2ChannelsTo8ChannelsFast(const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void Mix2ChannelsTo8ChannelsFast(const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/** This is a generalized operation that uses the channel gain matrix provided in Gains to mix an interleaved source buffer to the interleaved downmix buffer.
	 *  This operation is not explicitly vectorized and will almost always be slower than using one of the functions above.
	*/
	SIGNALPROCESSING_API void DownmixBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void DownmixBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void DownmixBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& DestinationBuffer, float* RESTRICT StartGains, const float* RESTRICT EndGains);
	SIGNALPROCESSING_API void DownmixBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const float* RESTRICT SourceBuffer, float* RESTRICT DestinationBuffer, int32 NumFrames, float* RESTRICT StartGains, const float* RESTRICT EndGains);

	/**
	 * This is similar to DownmixBuffer, except that it sums into DestinationBuffer rather than overwriting it.
	 */
	SIGNALPROCESSING_API void DownmixAndSumIntoBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const FAlignedFloatBuffer& SourceBuffer, FAlignedFloatBuffer& BufferToSumTo, const float* RESTRICT Gains);
	SIGNALPROCESSING_API void DownmixAndSumIntoBuffer(int32 NumSourceChannels, int32 NumDestinationChannels, const float* RESTRICT SourceBuffer, float* RESTRICT BufferToSumTo, int32 NumFrames, const float* RESTRICT Gains);

	/** Interleaves samples from two input buffers */
	SIGNALPROCESSING_API void BufferInterleave2ChannelFast(const FAlignedFloatBuffer& InBuffer1, const FAlignedFloatBuffer& InBuffer2, FAlignedFloatBuffer& OutBuffer);

	/** Interleaves samples from two input buffers */
	SIGNALPROCESSING_API void BufferInterleave2ChannelFast(const float* RESTRICT InBuffer1, const float* RESTRICT InBuffer2, float* RESTRICT OutBuffer, const int32 InNum);

	/** Deinterleaves samples from a 2 channel input buffer */
	SIGNALPROCESSING_API void BufferDeinterleave2ChannelFast(const FAlignedFloatBuffer& InBuffer, FAlignedFloatBuffer& OutBuffer1, FAlignedFloatBuffer& OutBuffer2);

	/** Deinterleaves samples from a 2 channel input buffer */
	SIGNALPROCESSING_API void BufferDeinterleave2ChannelFast(const float* RESTRICT InBuffer, float* RESTRICT OutBuffer1, float* RESTRICT OutBuffer2, const int32 InNum);

	/** Sums 2 channel interleaved input samples. OutSamples[n] = InSamples[2n] + InSamples[2n + 1] */
	SIGNALPROCESSING_API void BufferSum2ChannelToMonoFast(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples);

	/** Sums 2 channel interleaved input samples. OutSamples[n] = InSamples[2n] + InSamples[2n + 1] */
	SIGNALPROCESSING_API void BufferSum2ChannelToMonoFast(const float* RESTRICT InSamples, float* RESTRICT OutSamples, const int32 InNumFrames);

	/** Class which handles a vectorized interpolation of an entire buffer to the values of a target buffer */
	class FBufferLinearEase
	{
	public:
		SIGNALPROCESSING_API FBufferLinearEase();
		SIGNALPROCESSING_API FBufferLinearEase(const FAlignedFloatBuffer& InSourceValues, const FAlignedFloatBuffer& InTargetValues, int32 InLerpLength);
		SIGNALPROCESSING_API ~FBufferLinearEase();

		/** will cache SourceValues ptr and manually update SourceValues on Update() */
		SIGNALPROCESSING_API void Init(const FAlignedFloatBuffer& InSourceValues, const FAlignedFloatBuffer& InTargetValues, int32 InLerpLength);

		/** Performs Vectorized update of SourceValues float buffer. Returns true if interpolation is complete */
		SIGNALPROCESSING_API bool Update(FAlignedFloatBuffer& InSourceValues);

		/** Update overloaded to let you jump forward more than a single time-step */
		SIGNALPROCESSING_API bool Update(uint32 StepsToJumpForward, FAlignedFloatBuffer& InSourceValues);

		/** returns const reference to the deltas buffer for doing interpolation elsewhere */
		SIGNALPROCESSING_API const FAlignedFloatBuffer& GetDeltaBuffer();

	private:
		int32 BufferLength {0};
		int32 LerpLength {0};
		int32 CurrentLerpStep{0};
		FAlignedFloatBuffer DeltaBuffer;

	}; // class BufferLerper
}
