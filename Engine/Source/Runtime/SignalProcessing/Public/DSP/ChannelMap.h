// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"

namespace Audio
{
	// Maximum number of channels supported by channel maps (Corresponds to 7.1 format)
	static constexpr uint32 ChannelMapMaxNumChannels = 8;
	// Number of channels in a vorbis input (Corresponds to 5.1 with vorbis channel ordering).
	static constexpr uint32 ChannelMapVorbisNumChannels = 6;

	/** Denotes the order of gain coefficients for a channel map. */
	enum class EChannelMapOrder : uint8
	{
		// The channel map matrix is in output major order, meaning that adjacent
		// values increment the output channel index.
		//
		// G_In_Om is the gain for input channel (n) and output channel (m).
		// If there are (N) input channels and (M) output channel, then the channel
		// map will be in the form:
		// [ 
		// 		G_I0_O0, G_I0_O1, ... G_I0_OM, 
		// 		G_I1_O0, G_I1_O1, ... G_I0_OM, 
		// 		...
		// 		G_IN_O0, G_IN_O1, ... G_I0_OM, 
		// ]
		OutputMajorOrder,


		// The channel map matrix is in input major order, meaning that adjacent
		// values increment the input channel index.
		//
		// G_In_Om is the gain for input channel (n) and output channel (m).
		// If there are (N) input channels and (M) output channel, then the channel
		// map will be in the form:
		// [ 
		// 		G_I0_O0, G_I1_O0, ... G_IN_O0, 
		// 		G_I0_O1, G_I1_O1, ... G_IN_O1, 
		// 		...
		// 		G_I0_OM, G_I1_OM, ... G_IN_OM, 
		// ]
		InputMajorOrder
	};

	/** Denotes the upmix method for mixing a mono input into the front left
	 * and front right speakers. */
	enum class EChannelMapMonoUpmixMethod : uint8
	{
		// The mono channel is split 0.5 left/right
		Linear,

		// The mono channel is split 0.707 left/right
		EqualPower,

		// The mono channel is split 1.0 left/right
		FullVolume
	};

	/** Parameters for creating 2D channel maps. */
	struct FChannelMapParams
	{
		// Number of channels in the input audio.
		int32 NumInputChannels;
		// Number of channels in the output audio.
		int32 NumOutputChannels;
		// Order of gain coefficients in the output channel map. 
		EChannelMapOrder Order;
		// Method for upmixing mono audio (only used if NumInputChannels == 1)
		EChannelMapMonoUpmixMethod MonoUpmixMethod;
		// If true and (NumInputChannels == 1) and the output format has a center channel,
		// then the mono channel will be routed directly to the center channel.
		bool bIsCenterChannelOnly;
	};

	/** Create a 2D channel map.
	 *
	 * @param InParams - Configuration of channel map.
	 * @param OutChannelMap - Array to append with channel map.
	 *
	 * @return true on success, false on failure. 
	 */
	SIGNALPROCESSING_API bool Create2DChannelMap(const FChannelMapParams& InParams, TArray<float>& OutChannelMap);

	/** Create a 2D channel map for 5.1 vorbis input audio.
	 *
	 * @param InNumOutputChannels - Number of channels in the output audio.
	 * @param InOrder - Order of gian coefficients in the output channel map.
	 * @param OutChannelMap - Array to append with channel map.
	 *
	 * @return true on success, false on failure. 
	 */
	SIGNALPROCESSING_API bool CreateVorbis2DChannelMap(int32 InNumOutputChannels, EChannelMapOrder InOrder, TArray<float>& OutVorbisChannelMap);
}
