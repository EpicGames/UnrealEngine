// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Delay.h"
#include "DSP/LFO.h"

namespace Audio
{
	namespace EChorusDelays
	{
		enum Type
		{
			Left,
			Center,
			Right,
			NumDelayTypes
		};
	}

	class FChorus
	{
	public:
		SIGNALPROCESSING_API FChorus();
		SIGNALPROCESSING_API ~FChorus();

		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumChannels, const float InDelayLengthSec = 2.0f, const int32 InControlSamplePeriod = 256);

		inline int32 GetNumChannels() const
		{
			return NumChannels;
		}

		SIGNALPROCESSING_API void SetDepth(const EChorusDelays::Type InType, const float InDepth);
		SIGNALPROCESSING_API void SetFrequency(const EChorusDelays::Type InType, const float InFrequency);
		SIGNALPROCESSING_API void SetFeedback(const EChorusDelays::Type InType, const float InFeedback);
		SIGNALPROCESSING_API void SetWetLevel(const float InWetLevel);
		SIGNALPROCESSING_API void SetDryLevel(const float InDryLevel);
		SIGNALPROCESSING_API void SetSpread(const float InSpread);
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame);
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer);

	protected:
		FDelay Delays[EChorusDelays::NumDelayTypes];
		FLFO LFOs[EChorusDelays::NumDelayTypes];
		FLinearEase Depth[EChorusDelays::NumDelayTypes];
		float Feedback[EChorusDelays::NumDelayTypes];

		float MinDelayMsec;
		float MaxDelayMsec;
		float DelayRangeMsec;
		float Spread;
		float MaxFrequencySpread;
		float WetLevel;
		float DryLevel;
		int32 NumChannels;
	};


}
