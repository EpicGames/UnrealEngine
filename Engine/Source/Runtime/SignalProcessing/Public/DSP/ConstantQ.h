// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AlignedBuffer.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	/** Band normalization schemes */
	enum class EPseudoConstantQNormalization : uint8
	{
		EqualAmplitude,		//< No energy scaling. All bands weighted so max is 1.
		EqualEuclideanNorm, //< Scale energy by euclidean norm. Good when using magnitude spectrum.
		EqualEnergy			//< Scale energy by total energy. Good when using power spectrum.
	};

	/** Settings for Pseudo Constant Q Kernel generation. */
	struct FPseudoConstantQKernelSettings
	{
		int32 NumBands;									//< Total number of resulting constant Q bands.
		float NumBandsPerOctave;							//< Number of bands to space within an octave.
		float KernelLowestCenterFreq;					//< The starting frequency of the first band.
		float BandWidthStretch;							//< Stretching factor to control overlap of adjacent bands.
		EPseudoConstantQNormalization Normalization;	//< Normalization scheme for bands.

		/** Default settings for Pseudo Constant Q Kernel */
		FPseudoConstantQKernelSettings()
		:	NumBands(96)
		,	NumBandsPerOctave(12)
		,	KernelLowestCenterFreq(40.f)
		,	BandWidthStretch(1.0) 
		,	Normalization(EPseudoConstantQNormalization::EqualEnergy)
		{}
	};

	/** NewPseudoConstantQKernelTransform
	 *
	 *  This creates a new Pseudo Constant Q Kernel Transform. Pseudo Constant Q differs from standard Constant Q in
	 *  that it applies a window to an existing DFT output as opposed to using a bank of filters.  The use of
	 *  an FFT speeds up the calculation but introduces some limitations in bandwidth due to the granularity of the
	 *  DFT. 
	 *
	 *  The resulting kernel will transform an array with (InFFTSize / 2 + 1) elements into an array with 
	 *  InSettings.NumBands elements.
	 *
	 *  InSettings controls the number of bands and band properties.
	 *  InFFTSize controls the expected input size to the kernel.
	 *  InSampleRate describes the sampling rate of the analyzed audio.
	 *
	 *  Returns a TUniquePtr to a FContiguousSparse2DKernelTransform.
	 */
	SIGNALPROCESSING_API TUniquePtr<FContiguousSparse2DKernelTransform> NewPseudoConstantQKernelTransform(const FPseudoConstantQKernelSettings& InSettings, const int32 InFFTSize, const float InSampleRate);

	/** Settings for a single constant q band. */
	struct FPseudoConstantQBandSettings
	{
		/** Center frequency of band in Hz. */
		float CenterFreq;

		/** Width of band in Hz. */
		float BandWidth; 

		/** Size of FFT used. */
		int32 FFTSize; 

		/** Samplerate of audio. */
		float SampleRate; 

		/** Desired normalization. */
		EPseudoConstantQNormalization Normalization;
	};

	struct FPseudoConstantQ
	{
		// Returns the center frequency of a given CQT band.
		static SIGNALPROCESSING_API float GetConstantQCenterFrequency(const int32 InBandIndex, const float InBaseFrequency, const float InBandsPerOctave);

		// Returns the bandwidth for a given CQT band.
		static SIGNALPROCESSING_API float GetConstantQBandWidth(const float InBandCenter, const float InBandsPerOctave, const float InBandWidthStretch);

		// Fills an array with the spectrum weights to apply to an FFT magnitude or power spectrum output.
		static SIGNALPROCESSING_API void FillArrayWithConstantQBand(const FPseudoConstantQBandSettings& InSettings, FAlignedFloatBuffer& OutOffsetArray, int32& OutOffsetIndex);
	};
}
