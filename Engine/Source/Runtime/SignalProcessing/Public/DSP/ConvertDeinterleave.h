// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/ChannelMap.h"
#include "DSP/MultichannelBuffer.h"
#include "Containers/ArrayView.h"
#include "Templates/UniquePtr.h"

namespace Audio
{
	/** Parameters for creating a IConvertDeinterleave object. */
	struct FConvertDeinterleaveParams
	{
		// Number of channels in the input audio.
		int32 NumInputChannels = 0;

		// Number of channels in the output audio.
		int32 NumOutputChannels = 0;

		// Method for upmixing mono audio (only used if NumInputChannels == 1)
		EChannelMapMonoUpmixMethod MonoUpmixMethod = EChannelMapMonoUpmixMethod::EqualPower;
	};

	/** IConvertDeinterleave is an interface for transforming multichannel interleaved
	 * audio samples into multichannel deinterleaved samples. The channel count of 
	 * the input and output audio may differ. 
	 *
	 * The deinterleaving and channel format conversion operations are combined
	 * into this single object as both operations are often required for any given 
	 * source audio. 
	 */
	struct IConvertDeinterleave
	{
		virtual ~IConvertDeinterleave() = default;

		/** Deinterleave and convert the channel format of the input audio.
		 *
		 * @param InSamples - ArrayView of interleaved input samples. The number of 
		 *                    samples must be evenly divisible by the number of 
		 *                    input channels. 
		 * @param OutSamples - A multichannel buffer of samples generated by deinterleaving
		 *                     the input samples and possibly upmixing or downmixing
		 *                     them to the target channel count. 
		 */
		void ProcessAudio(TArrayView<const float> InSamples, FMultichannelBuffer& OutSamples) const
		{
			TArray<TArrayView<float>> OutMultiChannelAudio = MakeMultichannelBufferView(OutSamples);
			ProcessAudio(InSamples,OutMultiChannelAudio);
		}

		/** Deinterleave and convert the channel format of the input audio.
		 *
		 * @param InSamples - ArrayView of interleaved input samples. The number of 
		 *                    samples must be evenly divisible by the number of 
		 *                    input channels. 
		 * @param OutMultiChannelAudio - A multichannel buffer of samples generated by deinterleaving
		 *                     the input samples and possibly upmixing or downmixing
		 *                     them to the target channel count. 
		 */
		virtual void ProcessAudio(TArrayView<const float> InSamples, const TArray<TArrayView<float>>& OutMultiChannelAudio) const = 0;


		/** Create an IConvertDeinterleave object for a given number of input
		 * and output channels. 
		 * 
		 * @param InNumInputChannels - Number of input channels in incoming audio.
		 * @param InNumOutputChannels - Number of output channels in outgoing audio.
		 *
		 * @return A valid TUniquePtr<> on success, an invalid TUniquePtr<> on failure. 
		 */
		SIGNALPROCESSING_API static TUniquePtr<IConvertDeinterleave> Create(const FConvertDeinterleaveParams& InParams);
	};
}
