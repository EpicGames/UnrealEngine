// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

namespace Audio
{
	/** FConvolutionSettings
	 *
	 * Settings for creating an IConvolutionAlgorithm.
	 */
	struct FConvolutionSettings
	{
		/** If true, hardware accelerated algorithms are valid. */
		bool bEnableHardwareAcceleration = true;

		/** Defines size of audio processing block. */
		int32 BlockNumSamples = 256;

		/** Number of input audio channels to support. */
		int32 NumInputChannels = 0;

		/** Number of output audio channels to support. */
		int32 NumOutputChannels = 0;

		/** Number of impulse responses to support. */
		int32 NumImpulseResponses = 0;

		/** Maximum number of samples in an impulse responses. */
		int32 MaxNumImpulseResponseSamples = 0;

		bool operator==(const FConvolutionSettings& Other)
		{
			return bEnableHardwareAcceleration == Other.bEnableHardwareAcceleration
				&& BlockNumSamples == Other.BlockNumSamples
				&& NumInputChannels == Other.NumInputChannels
				&& NumOutputChannels == Other.NumOutputChannels
				&& NumImpulseResponses == Other.NumImpulseResponses
				&& MaxNumImpulseResponseSamples == Other.MaxNumImpulseResponseSamples;
		}

		bool operator!=(const FConvolutionSettings& Other)
		{
			return !(*this == Other);
		}
	};


	/** IConvolutionAlgorithm 
	 *
	 * Interface for Convolution algorithm.
	 */
	class IConvolutionAlgorithm 
	{
		public:

			/** virtual destructor for inheritance. */
			virtual ~IConvolutionAlgorithm() = default;

			/** Returns the number of samples in an audio block. */
			virtual int32 GetNumSamplesInBlock() const = 0;

			/** Returns number of audio inputs. */
			virtual int32 GetNumAudioInputs() const = 0;

			/** Returns number of audio outputs. */
			virtual int32 GetNumAudioOutputs() const = 0;

			/** Process one block of audio.
			 *
			 * InSamples is processed by the impulse responses. The output is placed in OutSamples.
			 *
			 * @params InSamples - A 2D array of input deinterleaved audio samples. InSamples[GetNumAudioInputs()][GetNumSamplesInBlock()]
			 * @params OutSamples - A 2D array of output deinterleaved audio samples. OutSamples[GetNumAudioOutputs()][GetNumSamplesInBlock()]
			 *
			 */
			virtual void ProcessAudioBlock(const float* const InSamples[], float* const OutSamples[]) = 0;

			/** Reset internal history buffers. */
			virtual void ResetAudioHistory() = 0;

			/** Maximum supported length of impulse response. */
			virtual int32 GetMaxNumImpulseResponseSamples() const = 0;


			/** Return the number of impulse responses. */
			virtual int32 GetNumImpulseResponses() const = 0;

			/** Return the number of samples in an impulse response. */
			virtual int32 GetNumImpulseResponseSamples(int32 InImpulseResponseIndex) const = 0;

			/** Set impulse response values. */
			virtual void SetImpulseResponse(int32 InImpulseResponseIndex, const float* InSamples, int32 NumSamples) = 0;

			/** Sets the gain between an audio input, impulse response and audio output.
			 *
			 * ([audio input] * [impulse response]) x gain = [audio output]
			 */
			virtual void SetMatrixGain(int32 InAudioInputIndex, int32 InImpulseResponseIndex, int32 InAudioOutputIndex, float InGain) = 0;

			/** Gets the gain between an audio input, impulse response and audio output.
			 *
			 * ([audio input] * [impulse response]) x gain = [audio output]
			 */
			virtual float GetMatrixGain(int32 InAudioInputIndex, int32 InImpulseResponseIndex, int32 InAudioOutputIndex) const = 0;
	};

	/** IConvolutionAlgorithmFactory
	 *
	 * Factory interface for creating convolution algorithms.
	 */
	class IConvolutionAlgorithmFactory : public IModularFeature
	{
		public:
			SIGNALPROCESSING_API virtual ~IConvolutionAlgorithmFactory();

			/** Name of modular feature for Convolution factory.  */
			static SIGNALPROCESSING_API const FName GetModularFeatureName();

			/** Name of this particular factory. */
			virtual const FName GetFactoryName() const = 0;

			/** If true, this implementation uses hardware acceleration. */
			virtual bool IsHardwareAccelerated() const = 0;

			/** Returns true if the input settings are supported by this factory. */
			virtual bool AreConvolutionSettingsSupported(const FConvolutionSettings& InSettings) const = 0;

			/** Creates a new Convolution algorithm. */
			virtual TUniquePtr<IConvolutionAlgorithm> NewConvolutionAlgorithm(const FConvolutionSettings& InSettings) = 0;
	};

	/** FConvolutionFactory
	 *
	 * FConvolutionFactory creates convolution algorithms. It will choose hardware accelerated versions when they are available. 
	 */
	class FConvolutionFactory
	{
		public:
			/** This denotes that no specific IConvolutionAlgorithmFactory is desired. */
			static SIGNALPROCESSING_API const FName AnyAlgorithmFactory;

			/** NewConvolutionAlgorithm
			 *
			 * Creates and returns a new ConvolutionAlgorithm. 
			 *
			 * @param InSettings - The settings used to create the Convolution algorithm.
			 * @param InAlgorithmFactoryName - If not equal to FConvolutionFactory::AnyAlgorithmFactory, will only uses Convolution algorithm facotry if IConvolutionAlgorithmFactory::GetFactoryName() equals InAlgorithmFactoryName.
			 * @return A TUniquePtr<IConvolutionAlgorithm> to the created Convolution. This pointer can be invalid if an error occured or the fft algorithm could not be created.
			 */
			static SIGNALPROCESSING_API TUniquePtr<IConvolutionAlgorithm> NewConvolutionAlgorithm(const FConvolutionSettings& InSettings, const FName& InAlgorithmFactoryName = AnyAlgorithmFactory);
	};
}
