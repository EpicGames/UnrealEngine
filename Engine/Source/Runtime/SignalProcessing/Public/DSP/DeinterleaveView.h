// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	/** TDeinterleaveView
	 *
	 *  TDeinterleaveView provides iterators to access deinterleaved arrays from an interleaved array.
	 */
	template <typename T>
	class TDeinterleaveView
	{
		friend class TChannelIterator;

	public:

		/** TChannel
		 *
		 *  TChannel provides a contiguous copy of an single channel from an interleaved array.
		 */
		template<typename InAllocator = FDefaultAllocator>
		class TChannel
		{
			public:
				TArray<T, InAllocator>& Values;
				const int32 ChannelIndex;

				TChannel(TArray<T, InAllocator>& InValues, const int32 InChannelIndex)
				:	Values(InValues)
				,	ChannelIndex(InChannelIndex)
				{}
		};

		/** TChannelIterator
		 *
		 *  TChannelIterator iterates over channels in an interleaved array and providing
		 *  contiguous arrays of a single channel.
		 */
		template<typename InAllocator = FDefaultAllocator>
		class TChannelIterator
		{
			TDeinterleaveView<T> DeinterleaveView;
			TArray<T, InAllocator>& ArrayToFill;
			int32 ChannelIndex;

		public:

			/** Denotes the end of a channel iterator */
			static const int32 ChannelIndexEnd = INDEX_NONE;

			/** TChannelIterator
			 *
			 *  TChannelIterator allows iteration over interleaved arrays and provides contiguous arrays of single channels.
			 *
			 *  InDeinterleaveView is the parent view provides the source interleaved array for this iterator.
			 *  InArrayToFill is the array which will be populated with contiguous elements for a channel.
			 *  InChannelIndex is the channel index which this iterator will point to.
			 */
			TChannelIterator(TDeinterleaveView<T> InDeinterleaveView, TArray<T, InAllocator>& InArrayToFill, int32 InChannelIndex)
			:	DeinterleaveView(InDeinterleaveView)
			,	ArrayToFill(InArrayToFill)
			,	ChannelIndex(InChannelIndex)
			{
				if (ChannelIndex >= DeinterleaveView.NumChannels)
				{
					// Channel index is more than number of total channels
					ChannelIndex = ChannelIndexEnd;
				}


			}

			/** Increment the iterator forward by one channel */
			TChannelIterator& operator++()
			{
				if (ChannelIndex != ChannelIndexEnd)
				{
					ChannelIndex++;
					if (ChannelIndex >= DeinterleaveView.NumChannels)
					{
						ChannelIndex = ChannelIndexEnd;
					}
				}
				return *this;
			}

			/** Check equality between iterators */
			bool operator!=(const TChannelIterator& Other) const
			{
				return Other.ChannelIndex != ChannelIndex;
			}

			/** Get the current channel index */
			int32 GetChannelIndex() const
			{
				return ChannelIndex;
			}

			/** Dereference the iterator, returns a TChannel object */
			TChannel<InAllocator> operator*()
			{
				if (ChannelIndex == ChannelIndexEnd)
				{
					ArrayToFill.Reset(DeinterleaveView.NumElementsPerChannel);
				}
				else
				{
					// prepare the array
					ArrayToFill.Reset(DeinterleaveView.NumElementsPerChannel);
					if (DeinterleaveView.NumElementsPerChannel > 0)
					{
						ArrayToFill.AddUninitialized(DeinterleaveView.NumElementsPerChannel);
					}

					// Fill array with deinterleave data
					T* ArrayToFillData = ArrayToFill.GetData();
					const T* InterleavedData = DeinterleaveView.InterleavedArray.GetData();

					int32 InterleavedPos = ChannelIndex;
					for (int32 OutPos = 0; OutPos < DeinterleaveView.NumElementsPerChannel; OutPos++, InterleavedPos += DeinterleaveView.NumChannels)
					{
						ArrayToFillData[OutPos] = InterleavedData[InterleavedPos];
					}
				}


				// Create and return channel
				return TChannel<InAllocator>(ArrayToFill, ChannelIndex);
			}
		};

		/** TDeinterleave constructor.
		 *
		 *  InInterleavedArray is the interleaved array to be deinterleaved.
		 *  InNumChannels is the number of channels in the deinterleaved array.
		 */
		TDeinterleaveView(TArrayView<const T> InInterleavedArray, int32 InNumChannels)
		:	InterleavedArray(InInterleavedArray)
		,	NumChannels(InNumChannels)
		,	NumElementsPerChannel(0)
		{
			check(InterleavedArray.Num() % NumChannels == 0);

			NumElementsPerChannel = InterleavedArray.Num() / NumChannels;
		}

		/** Return an STL iterator to the first channel. It fills the given array with channel elements */
		template <typename InAllocator = FDefaultAllocator>
		TChannelIterator<InAllocator> begin(TArray<T, InAllocator>& InArrayToFill) const
		{
			return TChannelIterator<InAllocator>(*this, InArrayToFill, 0);
		}

		/** Return an STL iterator to the end. */
		template <typename InAllocator = FDefaultAllocator>
		TChannelIterator<InAllocator> end(TArray<T, InAllocator>& InArrayToFill) const
		{
			return TChannelIterator<InAllocator>(*this, InArrayToFill, TChannelIterator<InAllocator>::ChannelIndexEnd);
		}

	private:
		// view to interleaved data
		TArrayView<const T> InterleavedArray;

		int32 NumChannels;
		int32 NumElementsPerChannel;
	};

	/** TAutoDeinterleaveView
	 *
	 *  TAutoDeinterlaveView provides a STL like iterators which exposes contiguous channel arrays from interleaved arrays.  As opposed to TDeinterleaveView, this class can be used in range based for loops, but only one iterator is valid at a time since they all share the same InArrayToFill.
	 *
	 *  Example:
	 *
	 * 	TArray<float> ArrayToFill;
	 *  for (auto Channel : TAudoDeineterleaveView(InterleavedArray, ArrayToFill, 2))
	 *  {
	 *  	DoSomethingWithAudio(Channel.Values, Channel.ChannelIndex);
	 *  }
	 */
	template <typename T, typename InAllocator = FDefaultAllocator> 
	class TAutoDeinterleaveView : public TDeinterleaveView<T>
	{
		TArray<T, InAllocator>& ArrayToFill;

		typedef typename TDeinterleaveView<T>::template TChannelIterator<InAllocator> TAutoChannelIterator;

	public:
		/** TAutoDeinterleaveView Constructor.
		 *
		 *  InInterleavedArray is the interleaved array to be deinterleaved.
		 *  InArrayToFill is the array which will be populated with contiguous elements for a channel.
		 *  InNumChannels is the number of channels in the deinterleaved array.
		 */
		TAutoDeinterleaveView(TArrayView<const T> InInterleavedArray, TArray<T, InAllocator>& InArrayToFill, int32 InNumChannels)
		:	TDeinterleaveView<T>(InInterleavedArray, InNumChannels)
		,	ArrayToFill(InArrayToFill)
		{}

		/** Return an STL iterator to the first channel. */
		TAutoChannelIterator begin() 
		{
			return TDeinterleaveView<T>::template begin<InAllocator>(ArrayToFill);
		}

		/** Return an STL iterator to the end. */
		TAutoChannelIterator end()
		{
			return TDeinterleaveView<T>::template end<InAllocator>(ArrayToFill);
		}
	};
}

