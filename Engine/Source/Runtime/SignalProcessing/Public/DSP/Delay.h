// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/AlignedBuffer.h"
#include "DSP/Dsp.h"

namespace Audio
{
	// Circular Buffer Delay Line
	class FDelay
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FDelay();

		// Virtual Destructor
		SIGNALPROCESSING_API virtual ~FDelay();

		// Initialization of the delay with given sample rate and max buffer size in samples.
		// (calls Reset())
		SIGNALPROCESSING_API void Init(const float InSampleRate, const float InBufferLengthSec = 2.0f);

		// Resets the delay line state, flushes buffer and resets read/write pointers.
		// (called by Init())
		SIGNALPROCESSING_API void Reset();
		SIGNALPROCESSING_API void ResetWithFade();
		

		// Sets the delay line length. Will clamp to within range of the max initialized delay line length (won't resize).
		SIGNALPROCESSING_API void SetDelayMsec(const float InDelayMsec);

		// Same as SetDelayMsec, except in samples.
		SIGNALPROCESSING_API void SetDelaySamples(const float InDelaySamples);

		// Sets the delay line length but using the internal easing function for smooth delay line interpolation.
		SIGNALPROCESSING_API void SetEasedDelayMsec(const float InDelayMsec, const bool bIsInit = false);

		// Sets the easing factor for the delay line's internal exponential interpolator.
		SIGNALPROCESSING_API void SetEaseFactor(const float InEaseFactor);

		// Sets the output attenuation in DB
		SIGNALPROCESSING_API void SetOutputAttenuationDB(const float InDelayAttenDB);

		// Returns the current delay line length (in samples).
		float GetDelayLengthSamples() const { return DelayInSamples; }

		// Reads the delay line at current read index without writing or incrementing read/write pointers.
		SIGNALPROCESSING_API float Read() const;

		// Reads the delay line at an arbitrary time in Msec without writing or incrementing read/write pointers.
		SIGNALPROCESSING_API float ReadDelayAt(const float InReadMsec) const;

		// Write the input and increment read/write pointers
		SIGNALPROCESSING_API void WriteDelayAndInc(const float InDelayInput);

		// Process audio in the delay line, return the delayed value
		SIGNALPROCESSING_API virtual float ProcessAudioSample(const float InAudio);

		SIGNALPROCESSING_API virtual void ProcessAudioBuffer(const float* InAudio, int32 InNumSamples, float* OutAudio);

	protected:
		// Updates delay line based on any recent changes to settings
		SIGNALPROCESSING_API void Update(bool bForce = false);

		// Updates delay line based on any recent changes to settings
		SIGNALPROCESSING_API void ResizeIfNeeded(const int32 InNewNumSamples);

		// Pointer to the circular buffer of audio.
		FAlignedFloatBuffer AudioBuffer;

		// Max length of buffer (in samples)
		int32 AudioBufferSize;

		// Read index for circular buffer.
		int32 ReadIndex;

		// Write index for circular buffer.
		int32 WriteIndex;

		// Sample rate
		float SampleRate;

		// Delay in samples; float supports fractional delay
		float DelayInSamples;

		// Upper limit of where we will resize the array
		int32 MaxBufferLengthSamples;

		// Eased delay in msec
		FExponentialEase EaseDelayMsec;

		// Output attenuation value.
		float OutputAttenuation;


		// Attenuation in decibel
		float OutputAttenuationDB;

	private:

		// Used to do a quick fade-in of input after a call to "ResetWithFade()"
		float InputAttenuation = 1.f;
		float InputFadeGainStep = 0.f;

		void WriteToBuffer(const float InInput);
		void AdvanceIndices();
	};

}
