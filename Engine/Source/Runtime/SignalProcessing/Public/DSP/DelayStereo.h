// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Delay.h"
#include "DSP/Filter.h"

namespace Audio
{
	// The different stereo delay modes
	namespace EStereoDelayMode
	{
		enum Type
		{
			Normal,
			Cross,
			PingPong,

			NumStereoDelayModes
		};
	}

	class FDelayStereo
	{
	public:
		SIGNALPROCESSING_API FDelayStereo();
		SIGNALPROCESSING_API ~FDelayStereo();

		// Initializes the stereo delay with given sample rate and default max delay length
		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumChannels, const float InDelayLengthSec = 2.0f);

		// Resets the stereo delay state
		SIGNALPROCESSING_API void Reset();

		// Process a single frame of audio
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame);

		// Process a buffer of audio
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer);

		// Sets which delay stereo mode to use
		SIGNALPROCESSING_API void SetMode(const EStereoDelayMode::Type InMode);

		// Gets the current stereo dealy mode
		EStereoDelayMode::Type GetMode() const { return DelayMode; }

		// Sets the delay time in msec. 
		SIGNALPROCESSING_API void SetDelayTimeMsec(const float InDelayTimeMsec);

		// Sets the feedback amount
		SIGNALPROCESSING_API void SetFeedback(const float InFeedback);

		// Sets the delay ratio (scales difference between left and right stereo delays)
		SIGNALPROCESSING_API void SetDelayRatio(const float InDelayRatio);

		// Sets the amount of the effect to mix in the output
		SIGNALPROCESSING_API void SetWetLevel(const float InWetLevel);

		// Sets the amount of the effect to mix in the output
		SIGNALPROCESSING_API void SetDryLevel(const float InDryLevel);

		// Sets whether or not the filter is enabled
		SIGNALPROCESSING_API void SetFilterEnabled(bool bInEnabled);

		// Sets the filter settings
		SIGNALPROCESSING_API void SetFilterSettings(EBiquadFilter::Type InFilterType, const float InCutoffFrequency, const float InQ);

	protected:
		// Updates the delays based on recent parameters
		SIGNALPROCESSING_API void UpdateDelays();

		// Delay lines per channel
		TArray<FDelay> Delays;

		// Biquad filter per channel to feed the delay output through
		TArray<FBiquadFilter> BiquadFilters;

		// What mode the stereo delay is in
		EStereoDelayMode::Type DelayMode  = EStereoDelayMode::Normal;

		// Amount of delay time in msec	
		float DelayTimeMsec = 0.0f;

		// How much delay feedback to use
		float Feedback = 0.0f;

		// How much to shift the delays from each other
		float DelayRatio = 0.0f;

		// The amount of wet level on the output
		float WetLevel = 0.0f;

		// The amount of dry level on the output
		float DryLevel = 1.0f;

		// Filter data
		float FilterFreq = 20000.0f;
		float FilterQ = 2.0f;
		EBiquadFilter::Type FilterType = EBiquadFilter::Lowpass;

		// The number of channels to use (will sum mono)
		int32 NumChannels = 0;

		// If the delay has started processing yet
		bool bIsInit = true;

		bool bIsFilterEnabled = false;
	};

}
