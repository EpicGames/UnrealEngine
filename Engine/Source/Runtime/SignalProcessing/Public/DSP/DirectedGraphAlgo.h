// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Graph/DirectedGraphUtils.h"

namespace Audio
{

	/** A pair of int32s represent a directed edge. The first value represents 
	 * the source vertex, and the second value represents the destination 
	 * vertex.
	 */
	UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
	typedef TTuple<int32, int32> FDirectedEdge;

	/** A strongly connected component contains a subgraph of strongly connected
	 * vertices and their corresponding edges.
	 */
	struct UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace") FStronglyConnectedComponent
	{
		/** Vertices in the strongly connected component. */
		TArray<int32> Vertices;

		/** Edges in the strongly connected component. */
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArray<FDirectedEdge> Edges;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	/** An element in a directed tree with references to children of a vertex. */
	UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
	typedef UE::MathCore::Graph::FDirectedTreeElement FDirectedTreeElement;

	/** A directed tree graph represenation. */
	UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	typedef TMap<int32, FDirectedTreeElement> FDirectedTree;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	struct FDirectedGraphAlgo
	{
		/** Build a directed tree from an array of edges.
		 *
		 * @parma InEdges - An array of directed eges.
		 * @param OutTree - A tree structure built from the edges.
		 */
		UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SIGNALPROCESSING_API void BuildDirectedTree(TArrayView<const FDirectedEdge> InEdges, FDirectedTree& OutTree);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/** Build a transpose directed tree from an array of edges. 
		 *
		 * The transpose of a directed graph is created by reversing each edge.
		 *
		 * @parma InEdges - An array of directed eges.
		 * @param OutTree - A tree structure built from the reversed edges.
		 */
		UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SIGNALPROCESSING_API void BuildTransposeDirectedTree(TArrayView<const FDirectedEdge> InEdges, FDirectedTree& OutTree);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/** Traverse a tree in a depth first ordering.
		 *
		 * @param InInitialVertex - The starting vertex for traversal.
		 * @param InTree          - The tree structure to traverse. 
		 * @param InVisitFunc     - A function that is called for each vertex that 
		 *                          is visited. If this function returns true, 
		 *                          traversal is continued in a depth first 
		 *                          manner. If this function returns false, the 
		 *                          children of the current vertex are not visited.
		 */
		UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SIGNALPROCESSING_API void DepthFirstTraversal(int32 InInitialVertex, const FDirectedTree& InTree, TFunctionRef<bool (int32)> InVisitFunc);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/** Traverse a tree in a breadth first ordering.
		 *
		 * @param InInitialVertex - The starting vertex for traversal.
		 * @param InTree          - The tree structure to traverse. 
		 * @param InVisitFunc     - A function that is called for each vertex that 
		 *                          is visited. If this function returns true, 
		 *                          the children of this vertex will be visited 
		 *                          in the future.  If this function returns false, 
		 *                          then the children of the current vertex will 
		 *                          not be visited.
		 */
		UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SIGNALPROCESSING_API void BreadthFirstTraversal(int32 InInitialVertex, const FDirectedTree& InTree, TFunctionRef<bool (int32)> InVisitFunc);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/** Sort vertices topologically using a depth first sorting algorithm.
		 *
		 * @param InUniqueVertices - An array of vertices to sort.
		 * @param InUniqueEdges - An array of edges describing dependencies.
		 * @param OutVertexOrder - An array where ordered vertices are placed.
		 *
		 * @return True if sorting was successful. False otherwise.
		 */
		UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SIGNALPROCESSING_API bool DepthFirstTopologicalSort(TArrayView<const int32> InUniqueVertices, TArrayView<const FDirectedEdge> InUniqueEdges, TArray<int32>& OutVertexOrder);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/** Sort vertices topologically using a Kahn's sorting algorithm.
		 *
		 * @param InUniqueVertices - An array of vertices to sort.
		 * @param InUniqueEdges - An array of edges describing dependencies.
		 * @param OutVertexOrder - An array where ordered vertices are placed.
		 *
		 * @return True if sorting was successful. False otherwise.
		 */
		UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SIGNALPROCESSING_API bool KahnTopologicalSort(TArrayView<const int32> InUniqueVertices, TArrayView<const FDirectedEdge> InUniqueEdges, TArray<int32>& OutVertexOrder);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS


		/** Find strongly connected components given a set of edges using Tarjan
		 * algorithm.
		 *
		 * @param InEdges - Edges in the directed graph.
		 * @param OutComponents - Strongly connected components found in the graph are be added to this array.
		 * @param bExcludeSingleVertex - If true, single vertices are not be returned as strongly connected components. If false, single vertices may be returned as strongly connected components. 
		 *
		 * @return True if one or more strongly connected components are added to OutComponents. False otherwise. 
		 */
		UE_DEPRECATED(5.5, "This has moved to the UE::MathCore::Graph namespace")
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SIGNALPROCESSING_API bool TarjanStronglyConnectedComponents(const TSet<FDirectedEdge>& InEdges, TArray<FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex = true);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
}
