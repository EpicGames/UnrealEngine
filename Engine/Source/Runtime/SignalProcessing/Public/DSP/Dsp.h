// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DSP/AlignedBuffer.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMath.h"
#include "Misc/ScopeLock.h"
#include "SignalProcessingModule.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/IsIntegral.h"
#include "Templates/IsSigned.h"

// Macros which can be enabled to cause DSP sample checking
#if 0
#define CHECK_SAMPLE(VALUE) 
#define CHECK_SAMPLE2(VALUE)
#else
#define CHECK_SAMPLE(VALUE)  Audio::CheckSample(VALUE)
#define CHECK_SAMPLE2(VALUE)  Audio::CheckSample(VALUE)
#endif

namespace Audio
{
	// Utility to check for sample clipping. Put breakpoint in conditional to find 
	// DSP code that's not behaving correctly
	inline void CheckSample(float InSample, float Threshold = 0.001f)
	{	
		if (InSample > Threshold || InSample < -Threshold)
		{
			UE_LOGF(LogSignalProcessing, Log, "SampleValue Was %.2f", InSample);
		}
	}

	// Clamps floats to 0 if they are in sub-normal range
	UE_DEPRECATED(5.5, "Audio code relies on denormals flush to zero floating point mode from now on. Please use FScopedFTZFloatMode instead of this API")
	inline float UnderflowClamp(const float InValue)
	{
		if (InValue > -FLT_MIN && InValue < FLT_MIN)
		{
			return 0.0f;
		}
		return InValue;
	}

	// Function converts linear scale volume to decibels
	inline float ConvertToDecibels(const float InLinear, const float InFloor = UE_SMALL_NUMBER)
	{
		return 20.0f * FMath::LogX(10.0f, FMath::Max(InLinear, InFloor));
	}

	// Function converts decibel to linear scale
	inline float ConvertToLinear(const float InDecibels)
	{
		return FMath::Pow(10.0f, InDecibels / 20.0f);
	}

	// Given a velocity value [0,127], return the linear gain
	inline float GetGainFromVelocity(const float InVelocity)
	{
		if (InVelocity == 0.0f)
		{
			return 0.0f;
		}
		return (InVelocity * InVelocity) / (127.0f * 127.0f);
	}

	// Low precision, high performance approximation of sine using parabolic polynomial approx
	// Valid on interval [-PI, PI]
	inline float FastSin(const float X)
	{
		return (4.0f * X) / UE_PI * (1.0f - FMath::Abs(X) / UE_PI);
	}

	// Slightly higher precision, high performance approximation of sine using parabolic polynomial approx
	inline float FastSin2(const float X)
	{
		float X2 = FastSin(X);
		X2 = 0.225f * (X2* FMath::Abs(X2) - X2) + X2;
		return X2;
	}

	// Valid on interval [-PI, PI] 
	// Sine approximation using Bhaskara I technique discovered in 7th century. 
	// https://en.wikipedia.org/wiki/Bh%C4%81skara_I
	inline float FastSin3(const float X)
	{
		const float AbsX = FMath::Abs(X);
		const float Numerator = 16.0f * X * (UE_PI - AbsX);
		const float Denominator = 5.0f * UE_PI * UE_PI - 4.0f * AbsX * (UE_PI - AbsX);
		return Numerator / Denominator;
	}
	
	/**
	* Generates a sine wave in the given buffer with the given frequency.
	* 
	* Uses the method of generating a 2D point on the unit circle and multiplying
	* with a 2d rotation matrix to advance the phase. Very accurate and very fast.
	* 
	* Use this unless you're changing frequency every few samples on very tiny
	* blocks (single digit sample count)
	* 
	* Distortion vs directly calling FMath::Sin() is on the order of -100db, and is roughly
	* 10x faster (for constant frequency).
	* 
	* Frequency changes are not interpolated across the buffer, so dramatic changes
	* could introduce aliasing due to a discontinuous derivative in the output.
	* If you're changing frequencies by a lot, constantly, you should probably 
	* use VectorSinCos in a loop.
	* 
	* Usage:
	* 
	* constexpr const int32 ChunkSampleCount = 480;
	* alignas (sizeof(VectorRegister4Float)) float ChunkBuffer[ChunkSampleCount];
	* FSinOscBufferGenerator Generator;
	* Generator.GenerateBuffer(48000, 400, ChunkBuffer, ChunkSampleCount);
	* 
	* ChunkBuffer now has 10ms of sine tone at 400hz. The next call will
	* continue the sine tone where it left off.
	*
	*/
	class FSinOsc2DRotation
	{
	public:
		FSinOsc2DRotation(float InStartingPhaseRadians = 0.f)
		{
			LastPhasePerSample = -1;
			LastPhase = InStartingPhaseRadians;
			QuadDxVec = VectorZeroFloat();
			QuadDyVec = VectorZeroFloat();
		}

		/**
		* Generates the sine tone, continuing from the last phase.
		* 
		* @param SampleRate the sample rate the buffer will be played at
		* @param ClampedFrequency the frequency of the tone to emit, clamped to nyquist (SampleRate/2)
		* @param Buffer the output buffer
		* @param BufferSampleCount the number of samples in the buffer. Generally, this should be %4 granularity.
		* 
		*/
		void GenerateBuffer(float SampleRate, float ClampedFrequency, float* Buffer, int32 BufferSampleCount)
		{
			// Regenerate our vector rotation components if our changes.
			const float PhasePerSample = (ClampedFrequency * (2 * UE_PI)) / (SampleRate);
			if (LastPhasePerSample != PhasePerSample)
			{
				float QuadDx = FMath::Cos(PhasePerSample * 4);
				float QuadDy = FMath::Sin(PhasePerSample * 4);

				QuadDxVec = VectorLoadFloat1(&QuadDx);
				QuadDyVec = VectorLoadFloat1(&QuadDy);

				LastPhasePerSample = PhasePerSample;
			}

			float* Write = Buffer;

			// The rotation matrix drifts, so we resync every so often
			while (BufferSampleCount)
			{
				// The concept here is that cos/sin are points on a unit circle,
				// so an oscilator is just a rotation of that point.
				//
				// To avoid drift, we resync off of an accurate sin/cos every evaluation
				// then do a 2d rotation in the actual loop.
				//
				// we store 4 points at once to use SIMD, so each rotation is actually
				// 4x the phase delta.
				//
				alignas(16) float PhaseSource[4];
				PhaseSource[0] = LastPhase + 0 * PhasePerSample;
				PhaseSource[1] = LastPhase + 1 * PhasePerSample;
				PhaseSource[2] = LastPhase + 2 * PhasePerSample;
				PhaseSource[3] = LastPhase + 3 * PhasePerSample;

				VectorRegister4Float PhaseVec = VectorLoad(PhaseSource);
				VectorRegister4Float XVector, YVector;

				// We need an accurate representation of the delta
				// vectors since we are integrating it
				VectorSinCos(&YVector, &XVector, &PhaseVec);

				// Copy to local (the compiler actually didn't do this!)
				VectorRegister4Float LocalDxVec = QuadDxVec;
				VectorRegister4Float LocalDyVec = QuadDyVec;
				
				int32 BlockSampleCount = BufferSampleCount;
				if (BlockSampleCount > 480)
					BlockSampleCount = 480;

				// Unrolling this didn't seem to really help perf wise.
				int32 Block4 = BlockSampleCount >> 2;
				while (Block4)
				{
					VectorStore(YVector, Write);

					// 2D rotation matrix.
					VectorRegister4Float NewX = VectorSubtract(VectorMultiply(LocalDxVec, XVector), VectorMultiply(LocalDyVec, YVector));
					VectorRegister4Float NewY = VectorAdd(VectorMultiply(LocalDyVec, XVector), VectorMultiply(LocalDxVec, YVector));

					XVector = NewX;
					YVector = NewY;

					Write += 4;
					Block4--;
				}

				constexpr int32 SIMD_MASK = 0x00000003;
				if (BlockSampleCount & SIMD_MASK)
				{
					// We've actually already calculated the next quad - it's in YVector
					alignas(16) float YFloats[4];
					VectorStore(YVector, YFloats);

					int32 Remn = BlockSampleCount & SIMD_MASK;
					for (int32 i = 0; i < Remn; i++)
					{
						Write[i] = YFloats[i];
					}
				}

				// Advance phase, range reduce, and store.
				float PhaseInRadians = LastPhase + (float)BlockSampleCount * PhasePerSample;
				PhaseInRadians -= FMath::FloorToFloat(PhaseInRadians / (2 * UE_PI)) * (2 * UE_PI);
				LastPhase = PhaseInRadians;

				BufferSampleCount -= BlockSampleCount;
			} // end while BufferSampleCount
		} // end GenerateBuffer

	private:
		VectorRegister4Float QuadDxVec, QuadDyVec;
		float LastPhasePerSample;
		float LastPhase;
	}; // end FSinOsc2DRotation

	// Fast tanh based on pade approximation
	inline float FastTanh(float X)
	{
		if (X < -3) return -1.0f;
		if (X > 3) return 1.0f;
		const float InputSquared = X*X;
		return X*(27.0f + InputSquared) / (27.0f + 9.0f * InputSquared);
	}

	// Based on sin parabolic approximation
	inline float FastTan(float X)
	{
		const float Num = X * (1.0f - FMath::Abs(X) / UE_PI);
		const float Den = (X + 0.5f * UE_PI) * (1.0f - FMath::Abs(X + 0.5f * UE_PI) / UE_PI);
		return Num / Den;
	}

	// Gets polar value from unipolar
	inline float GetBipolar(const float X)
	{
		return 2.0f * X - 1.0f;
	}

	// Converts bipolar value to unipolar
	inline float GetUnipolar(const float X)
	{
		return 0.5f * X + 0.5f;
	}

	// Converts in place a buffer into Unipolar (0..1)
	// This is a Vector op version of the GetUnipolar function. (0.5f * X + 0.5f).
	inline void ConvertBipolarBufferToUnipolar(float* InAlignedBuffer, int32 NumSamples)
	{
		// Make sure buffers are aligned and we can do a whole number of loops.
		check(NumSamples % 4 == 0);

		const VectorRegister4Float Half = VectorSetFloat1(0.5f);

		// Process buffer 1 vector (4 floats) at a time.
		for(int32 i = NumSamples / 4; i; --i, InAlignedBuffer += 4)
		{
			VectorRegister4Float V = VectorLoad(InAlignedBuffer);
			V = VectorMultiply(V, Half);
			V = VectorAdd(V, Half);
			VectorStore(V, InAlignedBuffer);
		}
	}

	inline bool IsAudioBufferSilent(const float* AudioBuffer, const int32 NumSamples)
	{
		bool bIsSilent = true;

		int32 Index = 0;

#if PLATFORM_ENABLE_VECTORINTRINSICS
		const int32 SimdNum = NumSamples & 0xFFFFFFF0;
		for (; Index < SimdNum; Index += 16)
		{
			const VectorRegister4x4Float Samples = VectorLoad16(&AudioBuffer[Index]);

			if (VectorAnyGreaterThan(VectorAbs(Samples.val[0]), GlobalVectorConstants::SmallNumber) ||
				VectorAnyGreaterThan(VectorAbs(Samples.val[1]), GlobalVectorConstants::SmallNumber) ||
				VectorAnyGreaterThan(VectorAbs(Samples.val[2]), GlobalVectorConstants::SmallNumber) ||
				VectorAnyGreaterThan(VectorAbs(Samples.val[3]), GlobalVectorConstants::SmallNumber))
			{
				bIsSilent = false;
				Index = INT_MAX;
				break;
			}
		}
#endif

		// Finish to the end of the buffer or check each sample if vector intrinsics are disabled
		for (; Index < NumSamples; ++Index)
		{
			// As soon as we hit a non-silent sample, we're not silent
			if (FMath::Abs(AudioBuffer[Index]) > UE_SMALL_NUMBER)
			{
				bIsSilent = false;
				break;
			}
		}

		return bIsSilent;
	}

	// Using midi tuning standard, compute frequency in hz from midi value
	inline float GetFrequencyFromMidi(const float InMidiNote)
	{
		return 440.0f * FMath::Pow(2.0f, (InMidiNote - 69.0f) / 12.0f);
	}

	// Returns the log frequency of the input value. Maps linear domain and range values to log output (good for linear slider controlling frequency)
	inline float GetLogFrequencyClamped(const float InValue, const FVector2D& Domain, const FVector2D& Range)
	{
		// Check if equal as well as less than to avoid round error in case where at edges.
		if (InValue <= Domain.X)
		{
			return UE_REAL_TO_FLOAT(Range.X);
		}

		if (InValue >= Domain.Y)
		{
			return UE_REAL_TO_FLOAT(Range.Y);
		}

		//Handle edge case of NaN
		if (FMath::IsNaN(InValue))
		{
			return UE_REAL_TO_FLOAT(Range.X);
		}

		const FVector2D RangeLog(FMath::Max(FMath::Loge(Range.X), UE_SMALL_NUMBER), FMath::Min(FMath::Loge(Range.Y), UE_BIG_NUMBER));
		const float FreqLinear = (float)FMath::GetMappedRangeValueUnclamped(Domain, RangeLog, (FVector2D::FReal)InValue);
		return FMath::Exp(FreqLinear);
	}

	// Returns the linear frequency of the input value. Maps log domain and range values to linear output (good for linear slider representation/visualization of log frequency). Reverse of GetLogFrequencyClamped.
	inline float GetLinearFrequencyClamped(const float InFrequencyValue, const FVector2D& Domain, const FVector2D& Range)
	{
		// Check if equal as well as less than to avoid round error in case where at edges.
		if (InFrequencyValue <= Range.X)
		{
			return UE_REAL_TO_FLOAT(Domain.X);
		}

		if (InFrequencyValue >= Range.Y)
		{
			return UE_REAL_TO_FLOAT(Domain.Y);
		}

		//Handle edge case of NaN
		if (FMath::IsNaN(InFrequencyValue))
		{
			return UE_REAL_TO_FLOAT(Domain.X);
		}

		const FVector2D RangeLog(FMath::Max(FMath::Loge(Range.X), UE_SMALL_NUMBER), FMath::Min(FMath::Loge(Range.Y), UE_BIG_NUMBER));
		const FVector2D::FReal FrequencyLog = FMath::Loge(InFrequencyValue);
		return UE_REAL_TO_FLOAT(FMath::GetMappedRangeValueUnclamped(RangeLog, Domain, FrequencyLog));
	}

	// Using midi tuning standard, compute midi from frequency in hz
	inline float GetMidiFromFrequency(const float InFrequency)
	{
		return 69.0f + 12.0f * FMath::LogX(2.0f, InFrequency / 440.0f);
	}

	// Using passed in tuning frequency (standard is 440), compute midi from frequency in hz
	inline float GetMidiFromFrequency(const float InFrequency, const float TuningFrequency)
	{
		if (TuningFrequency <= 0.f)
		{
			return 0.f;
		}
		return 69.0f + 12.0f * FMath::LogX(2.0f, InFrequency / TuningFrequency);
	}

	// Return a pitch scale factor based on the difference between a base midi note and a target midi note. Useful for samplers.
	inline float GetPitchScaleFromMIDINote(int32 BaseMidiNote, int32 TargetMidiNote)
	{
		const float BaseFrequency = GetFrequencyFromMidi(FMath::Clamp((float)BaseMidiNote, 0.0f, 127.0f));
		const float TargetFrequency = 440.0f * FMath::Pow(2.0f, ((float)TargetMidiNote - 69.0f) / 12.0f);
		const float PitchScale = TargetFrequency / BaseFrequency;
		return PitchScale;
	}

	// Returns the frequency multiplier to scale a base frequency given the input semitones
	inline float GetFrequencyMultiplier(const float InPitchSemitones)
	{
		if (InPitchSemitones == 0.0f)
		{
			return 1.0f;

		}
		return FMath::Pow(2.0f, InPitchSemitones / 12.0f);
	}

	// Returns the number of semitones relative to a base frequency given the input frequency multiplier
	inline float GetSemitones(const float InMultiplier)
	{
		if (InMultiplier <= 0.0f)
		{
			return 12.0f * FMath::Log2(UE_SMALL_NUMBER);
		}
		return 12.0f * FMath::Log2(InMultiplier);
	}

	// Calculates equal power stereo pan using sinusoidal-panning law and cheap approximation for sin
	// InLinear pan is [-1.0, 1.0] so it can be modulated by a bipolar LFO
	inline void GetStereoPan(const float InLinearPan, float& OutLeft, float& OutRight)
	{
		const float LeftPhase = 0.5f * UE_PI * (0.5f * (InLinearPan + 1.0f) + 1.0f);
		const float RightPhase = 0.25f * UE_PI * (InLinearPan + 1.0f);
		OutLeft = FMath::Clamp(FastSin(LeftPhase), 0.0f, 1.0f);
		OutRight = FMath::Clamp(FastSin(RightPhase), 0.0f, 1.0f);
	}
 
	// Encodes a stereo Left/Right signal into a stereo Mid/Side signal 
	inline void EncodeMidSide(float& LeftChannel, float& RightChannel)
	{
		const float Temp = (LeftChannel - RightChannel);
		//Output
		LeftChannel = (LeftChannel + RightChannel);
		RightChannel = Temp;
	}

	// Encodes a stereo Left/Right signal into a stereo Mid/Side signal
	SIGNALPROCESSING_API void EncodeMidSide(
		const FAlignedFloatBuffer& InLeftChannel,
		const FAlignedFloatBuffer& InRightChannel,
		FAlignedFloatBuffer& OutMidChannel,
		FAlignedFloatBuffer& OutSideChannel);
	

	// Decodes a stereo Mid/Side signal into a stereo Left/Right signal
	inline void DecodeMidSide(float& MidChannel, float& SideChannel)
	{
		const float Temp = (MidChannel - SideChannel) * 0.5f;
		//Output
		MidChannel = (MidChannel + SideChannel) * 0.5f;
		SideChannel = Temp;
	}

	// Decodes a stereo Mid/Side signal into a stereo Left/Right signal
	SIGNALPROCESSING_API void DecodeMidSide(
		const FAlignedFloatBuffer& InMidChannel,
		const FAlignedFloatBuffer& InSideChannel,
		FAlignedFloatBuffer& OutLeftChannel,
		FAlignedFloatBuffer& OutRightChannel);

	// Helper function to get bandwidth from Q
	inline float GetBandwidthFromQ(const float InQ)
	{
		// make sure Q is not 0.0f, clamp to slightly positive
		const float Q = FMath::Max(UE_KINDA_SMALL_NUMBER, InQ);
		const float Arg = 0.5f * ((1.0f / Q) + FMath::Sqrt(1.0f / (Q*Q) + 4.0f));
		const float OutBandwidth = 2.0f * FMath::LogX(2.0f, Arg);
		return OutBandwidth;
	}

	// Helper function get Q from bandwidth
	inline float GetQFromBandwidth(const float InBandwidth)
	{
		const float InBandwidthClamped = FMath::Max(UE_KINDA_SMALL_NUMBER, InBandwidth);
		const float Temp = FMath::Pow(2.0f, InBandwidthClamped);
		const float OutQ = FMath::Sqrt(Temp) / (Temp - 1.0f);
		return OutQ;
	}

	// Given three values, determine peak location and value of quadratic fitted to the data.
	//
	// @param InValues - An array of 3 values with the maximum value located in InValues[1].
	// @param OutPeakLoc - The peak location relative to InValues[1].
	// @param OutPeakValue - The value of the peak at the peak location.
	//
	// @returns True if a peak was found, false if the values do not represent a peak.
	inline bool QuadraticPeakInterpolation(const float InValues[3], float& OutPeakLoc, float& OutPeakValue)
	{
		float Denom = InValues[0] - 2.f * InValues[1] + InValues[2];

		if (Denom >= 0.f)
		{
			// This is not a peak.
			return false;
		}

		float Tmp = InValues[0] - InValues[2];

		OutPeakLoc = 0.5f * Tmp / Denom;

		if ((OutPeakLoc > 0.5f) || (OutPeakLoc < -0.5f))
		{
			// This is not a peak.
			return false;
		}

		OutPeakValue = InValues[1] - 0.25f * Tmp * OutPeakLoc;

		return true;
	}

	// Polynomial interpolation using lagrange polynomials. 
	// https://en.wikipedia.org/wiki/Lagrange_polynomial
	inline float LagrangianInterpolation(const TArray<FVector2D> Points, const float Alpha)
	{
		float Lagrangian = 1.0f;
		float Output = 0.0f;

		const int32 NumPoints = Points.Num();
		for (int32 i = 0; i < NumPoints; ++i)
		{
			Lagrangian = 1.0f;
			for (int32 j = 0; j < NumPoints; ++j)
			{
				if (i != j)
				{
					float Denom = UE_REAL_TO_FLOAT(Points[i].X - Points[j].X);
					if (FMath::Abs(Denom) < UE_SMALL_NUMBER)
					{
						Denom = UE_SMALL_NUMBER;
					}
					Lagrangian *= (Alpha - UE_REAL_TO_FLOAT(Points[j].X)) / Denom;
				}
			}
			Output += Lagrangian * UE_REAL_TO_FLOAT(Points[i].Y);
		}
		return Output;
	}

	// Simple exponential easing class. Useful for cheaply and smoothly interpolating parameters.
	class FExponentialEase
	{
	public:
		FExponentialEase(float InInitValue = 0.0f, float InEaseFactor = 0.001f, float InThreshold = UE_KINDA_SMALL_NUMBER)
			: CurrentValue(InInitValue)
			, Threshold(InThreshold)
			, TargetValue(InInitValue)
			, EaseFactor(InEaseFactor)
			, OneMinusEase(1.0f - InEaseFactor)
			, EaseTimesTarget(EaseFactor * InInitValue)
		{
		}

		void Init(float InInitValue, float InEaseFactor = 0.001f)
		{
			CurrentValue = InInitValue;
			TargetValue = InInitValue;
			EaseFactor = InEaseFactor;

			OneMinusEase = 1.0f - EaseFactor;
			EaseTimesTarget = TargetValue * EaseFactor;
		}

		bool IsDone() const
		{
			return FMath::Abs(TargetValue - CurrentValue) < Threshold;
		}

		float GetNextValue()
		{
			if (IsDone())
			{
				return CurrentValue;
			}

			// Micro-optimization,
			// But since GetNextValue(NumTicksToJumpAhead) does this work in a tight loop (non-vectorizable), might as well
			/*
			return CurrentValue = CurrentValue + (TargetValue - CurrentValue) * EaseFactor;
								= CurrentValue + EaseFactor*TargetValue - EaseFactor*CurrentValue
								= (CurrentValue - EaseFactor*CurrentValue) + EaseFactor*TargetValue
								= (1 - EaseFactor)*CurrentValue + EaseFactor*TargetValue
			*/
			return CurrentValue = OneMinusEase * CurrentValue + EaseTimesTarget;
		}

		// same as GetValue(), but overloaded to jump forward by NumTicksToJumpAhead timesteps
		// (before getting the value)
		float GetNextValue(uint32 NumTicksToJumpAhead)
		{
			while (NumTicksToJumpAhead && !IsDone())
			{
				CurrentValue = OneMinusEase * CurrentValue + EaseTimesTarget;
				--NumTicksToJumpAhead;
			}

			return CurrentValue;
		}

		float PeekCurrentValue() const
		{
			return CurrentValue;
		}

		void SetEaseFactor(const float InEaseFactor)
		{
			EaseFactor = InEaseFactor;
			OneMinusEase = 1.0f - EaseFactor;
		}

		void operator=(const float& InValue)
		{
			SetValue(InValue);
		}

		void SetValue(const float InValue, const bool bIsInit = false)
		{
			TargetValue = InValue;
			EaseTimesTarget = EaseFactor * TargetValue;
			if (bIsInit)
			{
				CurrentValue = TargetValue;
			}
		}

		// This is a method for getting the factor to use for a given tau and sample rate.
		// Tau here is defined as the time it takes the interpolator to be within 1/e of it's destination.
		static float GetFactorForTau(float InTau, float InSampleRate)
		{
			return 1.0f - FMath::Exp(-1.0f / (InTau * InSampleRate));
		}

	private:

		// Current value of the exponential ease
		float CurrentValue;

		// Threshold to use to evaluate if the ease is done
		float Threshold;

		// Target value
		float TargetValue;

		// Percentage to move toward target value from current value each tick
		float EaseFactor;

		// 1.0f - EaseFactor
		float OneMinusEase;

		// EaseFactor * TargetValue
		float EaseTimesTarget;
	};
	
	// Simple easing function used to help interpolate params
	class FLinearEase 
	{
	public:
		FLinearEase()
			: StartValue(0.0f)
			, CurrentValue(0.0f)
			, DeltaValue(0.0f)
			, SampleRate(44100.0f)
			, DurationTicks(0)
			, DefaultDurationTicks(0)
			, CurrentTick(0)
			, bIsInit(true)
		{
		}

		~FLinearEase() = default;

		bool IsDone() const
		{
			return CurrentTick >= DurationTicks;
		}

		void Init(float InSampleRate)
		{
			SampleRate = InSampleRate;
			bIsInit = true;
		}

		void SetValueRange(const float Start, const float End, const float InTimeSec)
		{
			StartValue = Start;
			CurrentValue = Start;
			SetValue(End, InTimeSec);
		}

		float GetNextValue()
		{
			if (IsDone())
			{
				return CurrentValue;
			}

			CurrentValue = DeltaValue * (float)CurrentTick / (float)DurationTicks + StartValue;

			++CurrentTick;
			return CurrentValue;
		}

		// same as GetValue(), but overloaded to increment Current Tick by NumTicksToJumpAhead
		// (before getting the value)
		float GetNextValue(int32 NumTicksToJumpAhead)
		{
			if (IsDone())
			{
				return CurrentValue;
			}

			CurrentTick = FMath::Min(CurrentTick + NumTicksToJumpAhead, DurationTicks);
			CurrentValue = DeltaValue * (float)CurrentTick / (float)DurationTicks + StartValue;

			return CurrentValue;
		}

		float PeekCurrentValue() const
		{
			return CurrentValue;
		}
		 
		// Updates the target value without changing the duration or tick data.
		// Sets the state as if the new value was the target value all along
		void SetValueInterrupt(const float InValue)
		{
			if (IsDone())
			{
				CurrentValue = InValue;
			}
			else
			{
				DurationTicks = DurationTicks - CurrentTick;
				CurrentTick = 0;
				DeltaValue = InValue - CurrentValue;
				StartValue = CurrentValue;
			}
		}

		void SetValue(const float InValue, float InTimeSec = 0.0f)
		{
			if (bIsInit)
			{
				bIsInit = false;
				DurationTicks = 0;
			}
			else
			{
				DurationTicks = (int32)(SampleRate * InTimeSec);
			}
			CurrentTick = 0;

			if (DurationTicks == 0)
			{
				CurrentValue = InValue;			
			}
			else
			{
				DeltaValue = InValue - CurrentValue;
				StartValue = CurrentValue;
			}
		}

	private:
		float StartValue;
		float CurrentValue;
		float DeltaValue;
		float SampleRate;
		int32 DurationTicks;
		int32 DefaultDurationTicks;
		int32 CurrentTick;
		bool bIsInit;
	};

	// Simple parameter object which uses critical section to write to and read from data
	template<typename T>
	class TParams
	{
	public:
		TParams()
			: bChanged(false)
		{}

		// Sets the params
		void SetParams(const T& InParams)
		{
			FScopeLock Lock(&CritSect);
			bChanged = true;
			CurrentParams = InParams;
		}

		// Returns a copy of the params safely if they've changed since last time this was called
		bool GetParams(T* OutParamsCopy)
		{
			FScopeLock Lock(&CritSect);
			if (bChanged)
			{
				bChanged = false;
				*OutParamsCopy = CurrentParams;
				return true;
			}
			return false;
		}

		void CopyParams(T& OutParamsCopy) const
		{
			FScopeLock Lock(&CritSect);
			OutParamsCopy = CurrentParams;
		}

		bool bChanged;
		T CurrentParams;
		mutable FCriticalSection CritSect;
	};

	template <typename SampleType>
	struct DisjointedArrayView
	{
		DisjointedArrayView(TArrayView<SampleType> && InFirstBuffer, TArrayView<SampleType> && InSecondBuffer)
		: FirstBuffer(MoveTemp(InFirstBuffer))
		, SecondBuffer(MoveTemp(InSecondBuffer))
		{}

		template <typename OtherSampleType>
		DisjointedArrayView<OtherSampleType> SplitOtherToMatch(OtherSampleType* Other, int32 InNum) const
		{
			ensure(InNum == Num());
			const int32 FirstChunkNum = FirstNum();

			return DisjointedArrayView<OtherSampleType>(
				TArrayView<OtherSampleType>(Other, FirstChunkNum)
				, TArrayView<OtherSampleType>(Other + FirstChunkNum, InNum - FirstChunkNum)
			);
		}

		int32 CopyIntoBuffer(SampleType* InDestination, int32 InNumSamples)
		{
			check(InNumSamples >= Num());
			const int32 FirstCopySize = FirstNum() * sizeof(SampleType);
			const int32 SecondCopySize = SecondNum() * sizeof(SampleType);

			FMemory::Memcpy(InDestination, FirstBuffer.GetData(), FirstCopySize);

			if (SecondCopySize)
			{
				FMemory::Memcpy(InDestination + FirstNum(), SecondBuffer.GetData(), SecondCopySize);
			}

			return Num();
		}

		int32 FirstNum() const { return FirstBuffer.Num(); }
		int32 SecondNum() const { return SecondBuffer.Num(); }
		int32 Num() const { return FirstBuffer.Num() + SecondBuffer.Num(); }

		// data:
		TArrayView<SampleType> FirstBuffer;
		TArrayView<SampleType> SecondBuffer;

	}; // struct DisjointedArrayView

	/**
	 * Basic implementation of a circular buffer built for pushing and popping arbitrary amounts of data at once.
	 * Designed to be thread safe for SPSC; However, if Push() and Pop() are both trying to access an overlapping area of the buffer,
	 * One of the calls will be truncated. Thus, it is advised that you use a high enough capacity that the producer and consumer are never in contention.
	 */
	template <typename SampleType, size_t Alignment = 16>
	class TCircularAudioBuffer
	{
	private:

		TArray<SampleType, TAlignedHeapAllocator<Alignment>> InternalBuffer;
		uint32 Capacity;
		FThreadSafeCounter ReadCounter;
		FThreadSafeCounter WriteCounter;

	public:
		TCircularAudioBuffer()
		{
			SetCapacity(0);
		}

		TCircularAudioBuffer(const TCircularAudioBuffer<SampleType, Alignment>& InOther)
		{
			*this = InOther;
		}

		TCircularAudioBuffer& operator=(const TCircularAudioBuffer<SampleType, Alignment>& InOther)
		{
			InternalBuffer = InOther.InternalBuffer;
			Capacity = InOther.Capacity;
			ReadCounter.Set(InOther.ReadCounter.GetValue());
			WriteCounter.Set(InOther.WriteCounter.GetValue());

			return *this;
		}


		TCircularAudioBuffer(uint32 InCapacity)
		{
			SetCapacity(InCapacity);
		}

		void Reset(uint32 InCapacity = 0)
		{
			SetCapacity(InCapacity);
		}

		void Empty()
		{
			ReadCounter.Set(0);
			WriteCounter.Set(0);
			InternalBuffer.Empty();
		}

		void SetCapacity(uint32 InCapacity)
		{
			checkf(InCapacity < (uint32)TNumericLimits<int32>::Max(), TEXT("Max capacity for this buffer is 2,147,483,647 samples. Otherwise our index arithmetic will not work."));
			Capacity = InCapacity + 1;
			ReadCounter.Set(0);
			WriteCounter.Set(0);
			InternalBuffer.Reset();
			InternalBuffer.AddZeroed(Capacity);
		}

		/** Reserve capacity.
		 *
		 * @param InMinimumCapacity - Minimum capacity of circular buffer.
		 * @param bRetainExistingSamples - If true, existing samples in the buffer will be retained. If false, they are discarded.
		 */
		void Reserve(uint32 InMinimumCapacity, bool bRetainExistingSamples)
		{
			if (Capacity <= InMinimumCapacity)
			{
				uint32 NewCapacity = InMinimumCapacity + 1;

				checkf(NewCapacity < (uint32)TNumericLimits<int32>::Max(), TEXT("Max capacity overflow. Requested %d. Maximum allowed %d"), NewCapacity, TNumericLimits<int32>::Max());

				uint32 NumToAdd = NewCapacity - Capacity;
				InternalBuffer.AddZeroed(NumToAdd);
				Capacity = NewCapacity;
			}

			if (!bRetainExistingSamples)
			{
				ReadCounter.Set(0);
				WriteCounter.Set(0);
			}
		}

		/** Push an array of values into circular buffer. */
		int32 Push(TArrayView<const SampleType> InBuffer)
		{
			return Push(InBuffer.GetData(), InBuffer.Num());
		}

		// Pushes some amount of samples into this circular buffer.
		// Returns the amount of samples written.
		// This can only be used for trivially copyable types.
		int32 Push(const SampleType* InBuffer, uint32 NumSamples)
		{
			SampleType* DestBuffer = InternalBuffer.GetData();
			const uint32 ReadIndex = ReadCounter.GetValue();
			const uint32 WriteIndex = WriteCounter.GetValue();

			int32 NumToCopy = FMath::Min<int32>(NumSamples, Remainder());
			const int32 NumToWrite = FMath::Min<int32>(NumToCopy, Capacity - WriteIndex);

			FMemory::Memcpy(&DestBuffer[WriteIndex], InBuffer, NumToWrite * sizeof(SampleType));
			FMemory::Memcpy(&DestBuffer[0], &InBuffer[NumToWrite], (NumToCopy - NumToWrite) * sizeof(SampleType));

			WriteCounter.Set((WriteIndex + NumToCopy) % Capacity);

			return NumToCopy;
		}

		// Pushes some amount of zeros into the circular buffer.
		// Useful when acting as a blocked, mono/interleaved delay line
		int32 PushZeros(uint32 NumSamplesOfZeros)
		{
			SampleType* DestBuffer = InternalBuffer.GetData();
			const uint32 ReadIndex = ReadCounter.GetValue();
			const uint32 WriteIndex = WriteCounter.GetValue();

			int32 NumToZeroEnd = FMath::Min<int32>(NumSamplesOfZeros, Remainder());
			const int32 NumToZeroBegin = FMath::Min<int32>(NumToZeroEnd, Capacity - WriteIndex);

			FMemory::Memzero(&DestBuffer[WriteIndex], NumToZeroBegin * sizeof(SampleType));
			FMemory::Memzero(&DestBuffer[0], (NumToZeroEnd - NumToZeroBegin) * sizeof(SampleType));

			WriteCounter.Set((WriteIndex + NumToZeroEnd) % Capacity);

			return NumToZeroEnd;
		}

		// Push a single sample onto this buffer.
		// Returns false if the buffer is full.
		bool Push(const SampleType& InElement)
		{
			if (Remainder() == 0)
			{
				return false;
			}
			else
			{
				SampleType* DestBuffer = InternalBuffer.GetData();
				const uint32 ReadIndex = ReadCounter.GetValue();
				const uint32 WriteIndex = WriteCounter.GetValue();

				DestBuffer[WriteIndex] = InElement;

				WriteCounter.Set((WriteIndex + 1) % Capacity);
				return true;
			}
		}

		bool Push(SampleType && InElement)
		{
			if (Remainder() == 0)
			{
				return false;
			}
			else
			{
				SampleType* DestBuffer = InternalBuffer.GetData();
				const uint32 ReadIndex = ReadCounter.GetValue();
				const uint32 WriteIndex = WriteCounter.GetValue();

				DestBuffer[WriteIndex] = MoveTemp(InElement);

				WriteCounter.Set((WriteIndex + 1) % Capacity);
				return true;
			}
		}

		// Same as Pop(), but does not increment the read counter.
		int32 Peek(SampleType* OutBuffer, uint32 NumSamples) const
		{
			const SampleType* SrcBuffer = InternalBuffer.GetData();
			const uint32 ReadIndex = ReadCounter.GetValue();
			const uint32 WriteIndex = WriteCounter.GetValue();

			int32 NumToCopy = FMath::Min<int32>(NumSamples, Num());

			const int32 NumRead = FMath::Min<int32>(NumToCopy, Capacity - ReadIndex);
			FMemory::Memcpy(OutBuffer, &SrcBuffer[ReadIndex], NumRead * sizeof(SampleType));
				
			FMemory::Memcpy(&OutBuffer[NumRead], &SrcBuffer[0], (NumToCopy - NumRead) * sizeof(SampleType));

			check(NumSamples < ((uint32)TNumericLimits<int32>::Max()));

			return NumToCopy;
		}

		// same Peek(), but provides a (possibly) disjointed view of the memory in-place
		// Push calls while the returned view is being accessed is undefined behavior
		DisjointedArrayView <const SampleType> PeekInPlace(uint32 NumSamples) const
		{
			const SampleType* SrcBuffer = InternalBuffer.GetData();
			const uint32 ReadIndex = ReadCounter.GetValue();
			const uint32 WriteIndex = WriteCounter.GetValue();

			int32 NumToView = FMath::Min<int32>(NumSamples, Num());
			const int32 NumRead = FMath::Min<int32>(NumToView, Capacity - ReadIndex);
			check(NumSamples < ((uint32)TNumericLimits<int32>::Max()));

			return DisjointedArrayView < const SampleType > (
				TArrayView<const SampleType>(SrcBuffer + ReadIndex, NumRead)
				, TArrayView<const SampleType>(SrcBuffer, (NumToView - NumRead))
			);
		}

		// Peeks a single element.
		// returns false if the element is empty.
		bool Peek(SampleType& OutElement) const
		{
			if (Num() == 0)
			{
				return false;
			}
			else
			{
				SampleType* SrcBuffer = InternalBuffer.GetData();
				const uint32 ReadIndex = ReadCounter.GetValue();
				
				OutElement = SrcBuffer[ReadIndex];

				return true;
			}
		}

		// Pops some amount of samples into this circular buffer.
		// Returns the amount of samples read.
		int32 Pop(SampleType* OutBuffer, uint32 NumSamples)
		{
			int32 NumSamplesRead = Peek(OutBuffer, NumSamples);
			check(NumSamples < ((uint32)TNumericLimits<int32>::Max()));

			ReadCounter.Set((ReadCounter.GetValue() + NumSamplesRead) % Capacity);

			return NumSamplesRead;
		}

		// Same as Pop(), but provides a (possibly) disjinted view of memory in-place
		// Push calls while the returned view is being accessed is undefined behavior
		DisjointedArrayView<const SampleType> PopInPlace(uint32 NumSamples)
		{
			check(NumSamples < ((uint32)TNumericLimits<int32>::Max()));

			DisjointedArrayView<const SampleType> View = PeekInPlace(NumSamples);
			const int32 NumSamplesRead = View.Num();
			ReadCounter.Set((ReadCounter.GetValue() + NumSamplesRead) % Capacity);

			return View;
		}

		// Pops some amount of samples into this circular buffer.
		// Returns the amount of samples read.
		int32 Pop(uint32 NumSamples)
		{
			check(NumSamples < ((uint32)TNumericLimits<int32>::Max()));

			int32 NumSamplesRead = FMath::Min<int32>(NumSamples, Num());

			ReadCounter.Set((ReadCounter.GetValue() + NumSamplesRead) % Capacity);

			return NumSamplesRead;
		}

		// Pops a single element.
		// Will assert if the buffer is empty. Please check Num() > 0 before calling.
		SampleType Pop()
		{
			// Calling this when the buffer is empty is considered a fatal error.
			check(Num() > 0);

			SampleType* SrcBuffer = InternalBuffer.GetData();
			const uint32 ReadIndex = ReadCounter.GetValue();

			SampleType PoppedValue = MoveTempIfPossible(InternalBuffer[ReadIndex]);
			ReadCounter.Set((ReadCounter.GetValue() + 1) % Capacity);
			return PoppedValue;
		}

		// When called, seeks the read or write cursor to only retain either the NumSamples latest data
		// (if bRetainOldestSamples is false) or the NumSamples oldest data (if bRetainOldestSamples is true)
		// in the buffer. Cannot be used to increase the capacity of this buffer.
		void SetNum(uint32 NumSamples, bool bRetainOldestSamples = false)
		{
			check(NumSamples < Capacity);

			if (bRetainOldestSamples)
			{
				WriteCounter.Set((ReadCounter.GetValue() + NumSamples) % Capacity);
			}
			else
			{
				int64 ReadCounterNum = ((int32)WriteCounter.GetValue()) - ((int32) NumSamples);
				if (ReadCounterNum < 0)
				{
					ReadCounterNum = Capacity + ReadCounterNum;
				}

				ReadCounter.Set(ReadCounterNum);
			}
		}

		// Get number of samples that can be popped off of the buffer.
		uint32 Num() const
		{
			const int32 ReadIndex = ReadCounter.GetValue();
			const int32 WriteIndex = WriteCounter.GetValue();

			if (WriteIndex >= ReadIndex)
			{
				return WriteIndex - ReadIndex;
			}
			else
			{
				return Capacity - ReadIndex + WriteIndex;
			}
		}

		// Get the current capacity of the buffer
		uint32 GetCapacity() const
		{
			return Capacity;
		}

		// Get number of samples that can be pushed onto the buffer before it is full.
		uint32 Remainder() const
		{
			return Capacity - Num() - 1;
		}
	};

	/**
	 * This allows us to write a compile time exponent of a number.
	 */
	template <int Base, int Exp>
	struct TGetPower
	{
		static_assert(Exp >= 0, "TGetPower only supports positive exponents.");
		static const int64 Value = Base * TGetPower<Base, Exp - 1>::Value;
	};

	template <int Base>
	struct TGetPower<Base, 0>
	{
		static const int64 Value = 1;
	};

	/**
	 * TSample<SampleType, Q>
	 * Variant type to simplify converting and performing operations on fixed precision and floating point samples.
	 */
	template <typename SampleType, uint32 Q = (sizeof(SampleType) * 8 - 1)>
	class TSample
	{
		SampleType Sample;

		template <typename SampleTypeToCheck>
		static void CheckValidityOfSampleType()
		{
			constexpr bool bIsTypeValid = !!(TIsFloatingPoint<SampleTypeToCheck>::Value || TIsIntegral<SampleTypeToCheck>::Value);
			static_assert(bIsTypeValid, "Invalid sample type! TSampleRef only supports float or integer values.");
		}

		template <typename SampleTypeToCheck, uint32 QToCheck>
		static void CheckValidityOfQ()
		{
			// If this is a fixed-precision value, our Q offset must be less than how many bits we have.
			constexpr bool bIsTypeValid = !!(TIsFloatingPoint<SampleTypeToCheck>::Value || ((sizeof(SampleTypeToCheck) * 8) > QToCheck));
			static_assert(bIsTypeValid, "Invalid value for Q! TSampleRef only supports float or int types. For int types, Q must be smaller than the number of bits in the int type.");
		}

		// This is the number used to convert from float to our fixed precision value.
		static constexpr float QFactor = TGetPower<2, Q>::Value - 1;

		// for fixed precision types, the max and min values that we can represent are calculated here:
		static constexpr float MaxValue = TGetPower<2, (sizeof(SampleType) * 8 - Q)>::Value;
		static constexpr float MinValue = !!TIsSigned<SampleType>::Value ? (-1.0f * MaxValue) : 0.0f;

	public:

		TSample(SampleType& InSample)
			: Sample(InSample)
		{
			CheckValidityOfQ<SampleType, Q>();
			CheckValidityOfSampleType<SampleType>();
		}

		template<typename ReturnType = float>
		ReturnType AsFloat() const
		{
			static_assert(TIsFloatingPoint<ReturnType>::Value, "Return type for AsFloat() must be a floating point type.");

			if (TIsFloatingPoint<SampleType>::Value)
			{
				return static_cast<ReturnType>(Sample);
			}
			else if (TIsIntegral<SampleType>::Value)
			{
				// Cast from fixed to float.
				return static_cast<ReturnType>(Sample) / QFactor;
			}
			else
			{
				checkNoEntry();
				return static_cast<ReturnType>(Sample);
			}
		}

		template<typename ReturnType, uint32 ReturnQ = (sizeof(SampleType) * 8 - 1)>
		ReturnType AsFixedPrecisionInt()
		{
			static_assert(TIsIntegral<ReturnType>::Value, "This function must be called with an integer type as ReturnType.");
			CheckValidityOfQ<ReturnType, ReturnQ>();

			if (TIsIntegral<SampleType>::Value)
			{
				if (Q > ReturnQ)
				{
					return Sample << (Q - ReturnQ);
				}
				else if (Q < ReturnQ)
				{
					return Sample >> (ReturnQ - Q);
				}
				else
				{
					return Sample;
				}
			}
			else if (TIsFloatingPoint<SampleType>::Value)
			{
				static constexpr float ReturnQFactor = TGetPower<2, ReturnQ>::Value - 1;
				return (ReturnType)(Sample * ReturnQFactor);
			}
		}

		template <typename OtherSampleType>
		TSample<SampleType, Q>& operator =(const OtherSampleType InSample)
		{
			CheckValidityOfSampleType<OtherSampleType>();

			if constexpr (std::is_same_v<SampleType, OtherSampleType>)
			{
				Sample = InSample;
				return *this;
			}
			else if (TIsIntegral<OtherSampleType>::Value && TIsFloatingPoint<SampleType>::Value)
			{
				// Cast from Q15 to float.
				Sample = ((SampleType)InSample) / QFactor;
				return *this;
			}
			else if (TIsFloatingPoint<OtherSampleType>::Value && TIsIntegral<SampleType>::Value)
			{
				// cast from float to Q15.
				Sample = static_cast<SampleType>(InSample * QFactor);
				return *this;
			}
			else
			{
				checkNoEntry();
				return *this;
			}
		}

		template <typename OtherSampleType>
		friend TSample<SampleType, Q> operator *(const TSample<SampleType, Q>& LHS, const OtherSampleType& RHS)
		{
			CheckValidityOfSampleType<OtherSampleType>();

			// Float case:
			if (TIsFloatingPoint<SampleType>::Value)
			{
				if (TIsFloatingPoint<OtherSampleType>::Value)
				{
					// float * float.
					return LHS.Sample * RHS;
				}
				else if (TIsIntegral<OtherSampleType>::Value)
				{
					// float * Q.
					SampleType FloatRHS = ((SampleType)RHS) / QFactor;
					return LHS.Sample * FloatRHS;
				}
				else
				{
					checkNoEntry();
					return LHS.Sample;
				}

			}
			// Q Case
			else if (TIsIntegral<SampleType>::Value)
			{
				if (TIsFloatingPoint<OtherSampleType>::Value)
				{
					// fixed * float.
					OtherSampleType FloatLHS = ((OtherSampleType)LHS.Sample) / QFactor;
					OtherSampleType Result = FMath::Clamp(FloatLHS * RHS, MinValue, MaxValue);
					return static_cast<SampleType>(Result * QFactor);
				}
				else if (TIsIntegral<OtherSampleType>::Value)
				{
					// Q * Q.
					float FloatLHS = ((float)LHS.Sample) / QFactor;
					float FloatRHS = ((float)RHS) / QFactor;
					float Result = FMath::Clamp(FloatLHS * FloatRHS, MinValue, MaxValue);
					return static_cast<OtherSampleType>(Result * QFactor);
				}
				else
				{
					checkNoEntry();
					return LHS.Sample;
				}
			}
			else
			{
				checkNoEntry();
				return LHS.Sample;
			}
		}
	};


	/**
	 * TSampleRef<SampleType, Q>
	 * Ref version of TSample. Useful for converting between fixed and float precisions.
	 * Example usage:
	 * int16 FixedPrecisionSample;
	 * TSampleRef<int16, 15> SampleRef(FixedPrecisionSample);
	 * 
	 * // Set the sample value directly:
	 * SampleRef = 0.5f;
	 * 
	 * // Or multiply the the sample:
	 * SampleRef *= 0.5f;
	 *
	 * bool bThisCodeWorks = FixedPrecisionSample == TNumericLimits<int16>::Max() / 4;
	 */
	template <typename SampleType, uint32 Q = (sizeof(SampleType) * 8 - 1)>
	class TSampleRef
	{
		SampleType& Sample;

		template <typename SampleTypeToCheck>
		static void CheckValidityOfSampleType()
		{
			constexpr bool bIsTypeValid = !!(TIsFloatingPoint<SampleTypeToCheck>::Value || TIsIntegral<SampleTypeToCheck>::Value);
			static_assert(bIsTypeValid, "Invalid sample type! TSampleRef only supports float or integer values.");
		}

		template <typename SampleTypeToCheck, uint32 QToCheck>
		static void CheckValidityOfQ()
		{
			// If this is a fixed-precision value, our Q offset must be less than how many bits we have.
			constexpr bool bIsTypeValid = !!(TIsFloatingPoint<SampleTypeToCheck>::Value || (sizeof(SampleTypeToCheck) * 8) > QToCheck);
			static_assert(bIsTypeValid, "Invalid value for Q! TSampleRef only supports float or int types. For int types, Q must be smaller than the number of bits in the int type.");
		}

		// This is the number used to convert from float to our fixed precision value.
		static constexpr float QFactor = TGetPower<2, Q>::Value - 1;

		// for fixed precision types, the max and min values that we can represent are calculated here:
		static constexpr float MaxValue = TGetPower<2, (sizeof(SampleType) * 8 - Q)>::Value;
		static constexpr float MinValue = !!TIsSigned<SampleType>::Value ? (-1.0f * MaxValue) : 0.0f;

	public:

		TSampleRef(SampleType& InSample)
			: Sample(InSample)
		{
			CheckValidityOfQ<SampleType, Q>();
			CheckValidityOfSampleType<SampleType>();
		}

		template<typename ReturnType = float>
		ReturnType AsFloat() const
		{
			static_assert(TIsFloatingPoint<ReturnType>::Value, "Return type for AsFloat() must be a floating point type.");

			if (TIsFloatingPoint<SampleType>::Value)
			{
				return static_cast<ReturnType>(Sample);
			}
			else if (TIsIntegral<SampleType>::Value)
			{
				// Cast from fixed to float.
				return static_cast<ReturnType>(Sample) / QFactor;
			}
			else
			{
				checkNoEntry();
				return static_cast<ReturnType>(Sample);
			}
		}

		template<typename ReturnType, uint32 ReturnQ = (sizeof(SampleType) * 8 - 1)>
		ReturnType AsFixedPrecisionInt()
		{
			static_assert(TIsIntegral<ReturnType>::Value, "This function must be called with an integer type as ReturnType.");
			
			CheckValidityOfQ<ReturnType, ReturnQ>();

			if (TIsIntegral<SampleType>::Value)
			{
				if (Q > ReturnQ)
				{
					return Sample << (Q - ReturnQ);
				}
				else if (Q < ReturnQ)
				{
					return Sample >> (ReturnQ - Q);
				}
				else
				{
					return Sample;
				}
			}
			else if (TIsFloatingPoint<SampleType>::Value)
			{
				static constexpr SampleType ReturnQFactor = TGetPower<2, ReturnQ>::Value - 1;
				return (ReturnType) (Sample * ReturnQFactor);
			}
		}

		template <typename OtherSampleType>
		TSampleRef<SampleType, Q>& operator =(const OtherSampleType InSample)
		{
			CheckValidityOfSampleType<OtherSampleType>();

			if constexpr (std::is_same_v<SampleType, OtherSampleType>)
			{
				Sample = InSample;
				return *this;
			}
			else if (TIsIntegral<OtherSampleType>::Value && TIsFloatingPoint<SampleType>::Value)
			{
				// Cast from fixed to float.
				Sample = ((SampleType)InSample) / QFactor;
				return *this;
			}
			else if (TIsFloatingPoint<OtherSampleType>::Value && TIsIntegral<SampleType>::Value)
			{
				// cast from float to fixed.
				Sample = (SampleType)(InSample * QFactor);
				return *this;
			}
			else
			{
				checkNoEntry();
				return *this;
			}
		}

		template <typename OtherSampleType>
		friend SampleType operator *(const TSampleRef<SampleType>& LHS, const OtherSampleType& RHS)
		{
			CheckValidityOfSampleType<OtherSampleType>();

			// Float case:
			if (TIsFloatingPoint<SampleType>::Value)
			{
				if (TIsFloatingPoint<OtherSampleType>::Value)
				{
					// float * float.
					return LHS.Sample * RHS;
				}
				else if (TIsIntegral<OtherSampleType>::Value)
				{
					// float * fixed.
					SampleType FloatRHS = ((SampleType)RHS) / QFactor;
					return LHS.Sample * FloatRHS;
				}
				else
				{
					checkNoEntry();
					return LHS.Sample;
				}

			}
			// Fixed Precision Case
			else if (TIsIntegral<SampleType>::Value)
			{
				if (TIsFloatingPoint<OtherSampleType>::Value)
				{
					// fixed * float.
					OtherSampleType FloatLHS = ((OtherSampleType)LHS.Sample) / QFactor;
					OtherSampleType Result = FMath::Clamp(FloatLHS * RHS, MinValue, MaxValue);
					return static_cast<SampleType>(Result * QFactor);
				}
				else if (TIsIntegral<OtherSampleType>::Value)
				{
					// fixed * fixed.
					float FloatLHS = ((float)LHS.Sample) / QFactor;
					float FloatRHS = ((float)RHS) / QFactor;
					float Result = FMath::Clamp(FloatLHS * FloatRHS, MinValue, MaxValue);
					
					return static_cast<SampleType>(Result * QFactor);
				}
				else
				{
					checkNoEntry();
					return LHS.Sample;
				}
			}
			else
			{
				checkNoEntry();
				return LHS.Sample;
			}
		}
	};
}

