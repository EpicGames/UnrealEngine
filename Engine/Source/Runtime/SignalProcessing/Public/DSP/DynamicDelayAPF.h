// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IntegerDelay.h"
#include "BufferVectorOperations.h"
#include "AllPassFractionalDelay.h"
#include "LinearInterpFractionalDelay.h"

namespace Audio
{
	// All Pass Filter with a long fractional delay which can be set per a sample. This filter is specifically designed for 
	// reverb applications where filter delay lines are long.
	class FDynamicDelayAPF
	{
	public:
		// InG is the filter coefficient used in the long delay all pass filter.
		// InMinDelay is the minimum allowable delay of the all pass filter.
		// InMaxDelay is the maximum allowable delay of the all pass filter.
		// InMaxNumInternalBufferSamples is the maximum internal block size used internally. 
		// InSampleRate is the current rendering sample rate. Used to convert parameter ease time from seconds to samples.
		SIGNALPROCESSING_API FDynamicDelayAPF(float InG, int32 InMinDelay, int32 InMaxDelay, int32 InMaxNumInternalBufferSamples, float InSampleRate);

		// Destructor
		SIGNALPROCESSING_API ~FDynamicDelayAPF();

		// Set the APF feedback/feedforward gain coefficient
		void SetG(float InG)
		{
			G.SetValue(InG, EaseTimeInSec);
		}

		// Processes InSamples through the all pass filter and populates OutSamples with the filter output.
		// InDelays denotes the per-sample delay of the allpass. It must have an equal number of elements as InSamples
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, const FAlignedFloatBuffer& InDelays, FAlignedFloatBuffer& OutSamples);

		// Zeros the internal delay line.
		SIGNALPROCESSING_API void Reset();

		void SetEaseTimeInSec(float InEaseTimeInSec) { EaseTimeInSec = InEaseTimeInSec; }

	protected:

		// Process one block of audio.
		SIGNALPROCESSING_API void ProcessAudioBlock(const float* InSamples, const FAlignedFloatBuffer& InFractionalDelays, const int32 InNum, float* OutSamples);

	private:
		// Feedback/Feedforward gain coefficient
		FLinearEase G;
		float EaseTimeInSec;
		int32 MinDelay;
		int32 MaxDelay;
		int32 NumDelaySamples;
		int32 NumInternalBufferSamples;

		// Buffers for block processing
		FAlignedFloatBuffer FractionalDelays;
		FAlignedFloatBuffer DelayLineInput;
		FAlignedFloatBuffer WorkBufferA;
		FAlignedFloatBuffer WorkBufferB;

		// Delay line memory.
		TUniquePtr<FAlignedBlockBuffer> IntegerDelayLine;
		TUniquePtr<FLinearInterpFractionalDelay> FractionalDelayLine;
	};

}
