// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/EnvelopeFollower.h"

namespace Audio
{
	enum EDynamicFilterType
	{
		Bell,
		LowShelf,
		HighShelf,
	};

	// combination of two state-variable filters, the first filtering a key signal, which drives gain in the second
	class FDynamicStateVariableFilter
	{
	public:
		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumChannels);
		SIGNALPROCESSING_API void Reset();

		SIGNALPROCESSING_API void ProcessAudio(const float* InSamples, float* OutSamples, const int32 InNumSamples);
		SIGNALPROCESSING_API void ProcessAudio(const float* InSamples, float* OutSamples, const float* KeySamples, const int32 InNumSamples);

		SIGNALPROCESSING_API void SetFrequency(const float InFrequency);
		SIGNALPROCESSING_API void SetQ(const float InQ);
		SIGNALPROCESSING_API void SetGain(const float InGain);
		SIGNALPROCESSING_API void SetFilterType(const EDynamicFilterType InFilterType);

		SIGNALPROCESSING_API void SetAnalog(const bool bInAnalog);
		SIGNALPROCESSING_API void SetAttackTime(const float InAttackTime);
		SIGNALPROCESSING_API void SetReleaseTime(const float InReleaseTime);
		SIGNALPROCESSING_API void SetThreshold(const float InThresholdDb);
		SIGNALPROCESSING_API void SetEnvMode(const EPeakMode::Type InMode);
		SIGNALPROCESSING_API void SetDynamicRange(const float InDynamicRange);
		SIGNALPROCESSING_API void SetRatio(const float InRatio);
		SIGNALPROCESSING_API void SetKnee(const float InKnee);

		SIGNALPROCESSING_API float GetGainReduction();

	protected:
		float SampleRate = 48000.f;
		float OneOverSampleRate = 1.f / 48000.f;
		int32 NumChannels = 0;

		// human readable parameters
		EDynamicFilterType FilterType = EDynamicFilterType::Bell;

		float Frequency = 1000.f;
		float Q = 1.f;
		float OneOverQ = 1.f;
		float ThresholdDb = -12.f;
		float DynamicRangeDb = 0.f;
		float Ratio = 1.5f;
		float GainDb = 1.f;
		float Knee = 12.f;
		float HalfKnee = 6.f;

		// cached result of envelope follower
		float ScaledGainLinear = 1.f;

		// Cached Gain ratio, only needs to be updated when Ratio is changed 
		float SlopeFactor = 0.f;

		// whether coefficients need updating
		bool bNeedsUpdate = true;

		// state-variable coefficient formulas for the included filter modes can be found here:
		// https://cytomic.com/files/dsp/SvfLinearTrapOptimised2.pdf
		struct FSVFCoefficients
		{
			// TanOmega
			float G = 0.f;

			// Feedback / 1/Q
			float K = 0.f;

			// Node Scalars
			float A0 = 1.f;
			float A1 = 0.f;
			float A2 = 0.f;
			float A3 = 0.f;

			// Output Scalars
			float M0 = 1.f;
			float M1 = 0.f;
			float M2 = 0.f;
		};

		struct FSVFState
		{
			// Nodes
			float V1 = 0.f;
			float V2 = 0.f;
			float V3 = 0.f;

			// Delays
			float Z1 = 0.f;
			float Z2 = 0.f;

			void ProcessSample(const FSVFCoefficients& Coeffs, const float InSample);
		};

		FInlineEnvelopeFollower Envelope;

		FSVFState KeyFilterState;
		// One per channel
		TArray<FSVFState> FilterState;

		FSVFCoefficients KeyVars;
		
		FSVFCoefficients OutputVars;
		
		// Points in the knee used for lagrangian interpolation
		TArray<FVector2D> KneePoints;
		
		SIGNALPROCESSING_API void ProcessBell(const float* InSamples, float* OutSamples, const float* KeySamples, const int32 InNumSamples);
		SIGNALPROCESSING_API void ProcessLowShelf(const float* InSamples, float* OutSamples, const float* KeySamples, const int32 InNumSamples);
		SIGNALPROCESSING_API void ProcessHighShelf(const float* InSamples, float* OutSamples, const float* KeySamples, const int32 InNumSamples);
		
		SIGNALPROCESSING_API void UpdateSettings();
		SIGNALPROCESSING_API void UpdatePreFilterVariables();
		SIGNALPROCESSING_API void UpdatePostFilterVariables();
		
		SIGNALPROCESSING_API float CalcGain(const float KeySample);
	};
}
