// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/EnvelopeFollower.h"
#include "DSP/IntegerDelay.h"
#include "Filter.h"

namespace Audio
{
	// What mode the compressor is in
	namespace EDynamicsProcessingMode
	{
		enum Type
		{
			Compressor,
			Limiter,
			Expander,
			Gate,
			UpwardsCompressor,
			Count
		};
	}

	enum class EDynamicsProcessorChannelLinkMode : uint8
	{
		Disabled,
		Average,
		Peak,
		Count
	};

	// Dynamic range compressor
	// https://en.wikipedia.org/wiki/Dynamic_range_compression
	class FDynamicsProcessor
	{
	public:
		SIGNALPROCESSING_API FDynamicsProcessor();
		SIGNALPROCESSING_API ~FDynamicsProcessor();

		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumChannels = 2);

		SIGNALPROCESSING_API int32 GetNumChannels() const;
		SIGNALPROCESSING_API int32 GetKeyNumChannels() const;
		SIGNALPROCESSING_API float GetMaxLookaheadMsec() const;

		SIGNALPROCESSING_API void SetLookaheadMsec(const float InLookAheadMsec);
		SIGNALPROCESSING_API void SetAttackTime(const float InAttackTimeMsec);
		SIGNALPROCESSING_API void SetReleaseTime(const float InReleaseTimeMsec);
		SIGNALPROCESSING_API void SetThreshold(const float InThresholdDb);
		SIGNALPROCESSING_API void SetRatio(const float InCompressionRatio);
		SIGNALPROCESSING_API void SetKneeBandwidth(const float InKneeBandwidthDb);
		SIGNALPROCESSING_API void SetInputGain(const float InInputGainDb);
		SIGNALPROCESSING_API void SetKeyAudition(const bool InAuditionEnabled);
		SIGNALPROCESSING_API void SetKeyGain(const float InKeyGain);
		SIGNALPROCESSING_API void SetKeyHighshelfCutoffFrequency(const float InCutoffFreq);
		SIGNALPROCESSING_API void SetKeyHighshelfEnabled(const bool bInEnabled);
		SIGNALPROCESSING_API void SetKeyHighshelfGain(const float InGainDb);
		SIGNALPROCESSING_API void SetKeyLowshelfCutoffFrequency(const float InCutoffFreq);
		SIGNALPROCESSING_API void SetKeyLowshelfEnabled(const bool bInEnabled);
		SIGNALPROCESSING_API void SetKeyLowshelfGain(const float InGainDb);
		SIGNALPROCESSING_API void SetKeyNumChannels(const int32 InNumChannels);
		SIGNALPROCESSING_API void SetNumChannels(const int32 InNumChannels);
		SIGNALPROCESSING_API void SetOutputGain(const float InOutputGainDb);
		SIGNALPROCESSING_API void SetChannelLinkMode(const EDynamicsProcessorChannelLinkMode InLinkMode);
		SIGNALPROCESSING_API void SetAnalogMode(const bool bInIsAnalogMode);
		SIGNALPROCESSING_API void SetPeakMode(const EPeakMode::Type InEnvelopeFollowerModeType);
		SIGNALPROCESSING_API void SetProcessingMode(const EDynamicsProcessingMode::Type ProcessingMode);

		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame, const float* InKeyFrame);
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame, const float* InKeyFrame, float* OutGain);
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer, const float* InKeyBuffer = nullptr, float* OutEnvelope = nullptr);
		 
		// For single channels of audio OR non-interleaved blocks of multichannel audio.
		SIGNALPROCESSING_API void ProcessAudio(const float* const* const InBuffers, const int32 InNumFrames, float* const* OutBuffers, const float* const* const InKeyBuffers, float* const* OutEnvelopes);

	protected:
		SIGNALPROCESSING_API float ComputeGain(const float InEnvFollowerDb);
		SIGNALPROCESSING_API void ComputeGains(float* InEnvFollowerDbOutGain, const int32 InNumSamples);

		// Process key frame, returning true if should continue processing
		// (Returns false in audition mode and writes straight to output).
		SIGNALPROCESSING_API bool ProcessKeyFrame(const float* InKeyFrame, float* OutFrame, bool bKeyIsInput);

		SIGNALPROCESSING_API bool IsInProcessingThreshold(const float InEnvFollowerDb) const;

		// (Optional) Low-pass filter for input signal
		FBiquadFilter InputLowshelfFilter;

		// (Optional) High-pass filter for input signal
		FBiquadFilter InputHighshelfFilter;

		EDynamicsProcessingMode::Type ProcessingMode;

		float SlopeFactor;

		// Peak mode of envelope followers
		EPeakMode::Type EnvelopeFollowerPeakMode;

		// Lookahead delay lines
		TArray<FIntegerDelay> LookaheadDelay;

		// Envelope followers
		TArray<FInlineEnvelopeFollower> EnvFollower;

		// Channel values of cached detector sample
		TArray<float> DetectorOuts;

		// Channel values of cached gain sample
		TArray<float> Gain;

		// How far ahead to look in the audio
		float LookaheadDelayMsec;

		// The period of which the compressor decreases gain to the level determined by the compression ratio
		float AttackTimeMsec;

		// The period of which the compressor increases gain to 0 dB once level has fallen below the threshold
		float ReleaseTimeMsec;

		// Amplitude threshold above which gain will be reduced
		float ThresholdDb;

		// Amount of gain reduction
		float Ratio;

		// Defines how hard or soft the gain reduction blends from no gain reduction to gain reduction (determined by the ratio)
		float HalfKneeBandwidthDb;

		// Amount of input gain
		float InputGain;

		// Amount of output gain
		float OutputGain;

		// Gain of key detector signal in dB
		float KeyGain;

		// Sample rate of both key and input (must match)
		float SampleRate;

		// Whether or not input channels are linked, and if so, how to calculate gain
		EDynamicsProcessorChannelLinkMode LinkMode;

		// Whether or not we're in analog mode
		bool bIsAnalogMode;

		// Whether or not to bypass processor and only output key modulator
		bool bKeyAuditionEnabled;

		// Whether or not key high-pass filter is enabled
		bool bKeyHighshelfEnabled;

		// Whether or not key low-pass filter is enabled
		bool bKeyLowshelfEnabled;

		static constexpr float UpwardsCompressionMaxGain = 36.0f;

		static constexpr float MaxLookaheadMsec = 100.0f;

	private:

		// OutBuffers is also used a temporary buffer for processing the Key. 
		// Thus, OutBuffers must supply enough valid memory to support it's use as a 
		// temporary buffer and needs to have Max(NumInputChannels, NumKeyChannels) 
		// buffers available.  
		void ProcessAudioDeinterleaveInternal(const float* const* const InBuffers, const int32 InNumFrames, float* const* OutBuffers, const float* const* const InKeyBuffers, float* const* OutEnvelopes);

		int32 GetNumDelayFrames() const;

		void CalculateSlope();

		void CalculateKnee();

		// Points in the knee used for lagrangian interpolation
		struct FKneePoint
		{
			float X{ 0.0f };
			float Y{ 0.0f };
		};
		TArray<FKneePoint> KneePoints;

		// For optimized LagrangianInterpolation on blocks...
		float Denominator0Minus1;

		// For optimized LagrangianInterpolation on blocks...
		float Denominator1Minus0;

	};

}
