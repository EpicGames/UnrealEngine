// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Filter.h"

namespace Audio
{
	// Equalizer filter
	// An equalizer is a cascaded (serial) band of parametric EQs
	// This filter allows for setting each band with variable Bandwidth/Q, Frequency, and Gain
	class FEqualizer
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FEqualizer();

		// Destructor
		SIGNALPROCESSING_API ~FEqualizer();

		// Initialize the equalizer
		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumBands, const int32 InNumChannels);

		// Sets whether or not the band is enabled
		SIGNALPROCESSING_API void SetBandEnabled(const int32 InBand, const bool bEnabled);

		// Sets all params of the band at once
		SIGNALPROCESSING_API void SetBandParams(const int32 InBand, const float InFrequency, const float InBandwidth, const float InGainDB);

		// Sets the band frequency
		SIGNALPROCESSING_API void SetBandFrequency(const int32 InBand, const float InFrequency);

		// Sets the band resonance (use alternatively to bandwidth)
		SIGNALPROCESSING_API void SetBandBandwidth(const int32 InBand, const float InBandwidth);

		// Sets the band gain in decibels
		SIGNALPROCESSING_API void SetBandGainDB(const int32 InBand, const float InGainDB);

		// Processes the audio frame (audio frame must have channels equal to that used during initialization)
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InAudio, float* OutAudio);

	private:

		// The number of channels in the equalizer
		int32 NumChannels;

		// The array of biquad filters
		TArray<FBiquadFilter> FilterBands;
		
		// Temporary array for processing audio.
		TArray<float> WorkBuffer;
	};

}
