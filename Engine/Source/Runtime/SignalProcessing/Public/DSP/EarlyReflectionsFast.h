// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/IntegerDelay.h"
#include "DSP/LongDelayAPF.h"
#include "DSP/BufferOnePoleLPF.h"
#include "DSP/FeedbackDelayNetwork.h"

namespace Audio
{
	struct FEarlyReflectionsFastSettings
	{
		// Early reflections gain
		float Gain;

		// Delay between input signal and early reflections
		float PreDelayMsec;

		// Input sample bandwidth before entering early reflections
		float Bandwidth;

		// Early reflections decay (lower value is longer)
		float Decay;

		// Early reflection high frequency absorption factor
		float Absorption;

		SIGNALPROCESSING_API FEarlyReflectionsFastSettings();

		SIGNALPROCESSING_API bool operator==(const FEarlyReflectionsFastSettings& Other) const;

		SIGNALPROCESSING_API bool operator!=(const FEarlyReflectionsFastSettings& Other) const;
	};

	// Basic implementation of early reflections using a predelay, low pass filter and feedback delay network (FDN). The FDN
	// utilizes four delay lines where each delay line consists of an all pass filter and low pass filter to control diffusion
	// and absorption.
	class FEarlyReflectionsFast
	{
	public:
		
		// Limits for settings
		static SIGNALPROCESSING_API const float MaxGain;
		static SIGNALPROCESSING_API const float MinGain;
		static SIGNALPROCESSING_API const float MaxPreDelay;
		static SIGNALPROCESSING_API const float MinPreDelay;
		static SIGNALPROCESSING_API const float MaxBandwidth;
		static SIGNALPROCESSING_API const float MinBandwidth;
		static SIGNALPROCESSING_API const float MaxDecay;
		static SIGNALPROCESSING_API const float MinDecay;
		static SIGNALPROCESSING_API const float MaxAbsorption;
		static SIGNALPROCESSING_API const float MinAbsorption;
		
		static SIGNALPROCESSING_API const FEarlyReflectionsFastSettings DefaultSettings;
		
		// InMaxNumInternalBufferSamples sets the maximum possible samples in an internal buffer.
		SIGNALPROCESSING_API FEarlyReflectionsFast(float InSampleRate, int32 InMaxNumInternalBufferSamples, const FEarlyReflectionsFastSettings& InSettings=DefaultSettings);
		SIGNALPROCESSING_API ~FEarlyReflectionsFast();

		// Sets the reverb settings, clamps, applies, and updates
		SIGNALPROCESSING_API void SetSettings(const FEarlyReflectionsFastSettings& InSettings);

		// Process the single audio frame
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, const int32 InNumChannels, FAlignedFloatBuffer& OutLeftSamples, FAlignedFloatBuffer& OutRightSamples);

		// Silence internal audio.
		SIGNALPROCESSING_API void FlushAudio();

		// Clamps settings to acceptable values. 
		static SIGNALPROCESSING_API void ClampSettings(FEarlyReflectionsFastSettings& InOutSettings);

	private:
		void ApplySettings();

		FEarlyReflectionsFastSettings Settings;
		float SampleRate;

		FAlignedFloatBuffer LeftInputBuffer;
		FAlignedFloatBuffer LeftWorkBufferA;
		FAlignedFloatBuffer LeftWorkBufferB;
		FAlignedFloatBuffer RightInputBuffer;
		FAlignedFloatBuffer RightWorkBufferA;
		FAlignedFloatBuffer RightWorkBufferB;

		FFDNCoefficients LeftCoefficients;
		FFDNCoefficients RightCoefficients;

		FFeedbackDelayNetwork LeftFDN;
		FFeedbackDelayNetwork RightFDN;

		FIntegerDelay LeftPreDelay;
		FIntegerDelay RightPreDelay;

		FBufferOnePoleLPF LeftInputLPF;
		FBufferOnePoleLPF RightInputLPF;
	};
}
