// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "Interfaces/IAudioFormat.h"

namespace Audio
{
	/**
	* IAudioEncoder
	* Interface used to implement a runtime audio encoder.
	*/
	class IAudioEncoder
	{
	public:
		/**
		 * Must be constructed with AudioBufferSlack.
		 * Optionally, DataBufferSlack can be called with a different value
		 * depending on how often PushAudio, EncodeIfPossible, or PopData will be called.
		 */
		SIGNALPROCESSING_API IAudioEncoder(uint32 AudioBufferSlack, uint32 DataBufferSlack = 4096);
		SIGNALPROCESSING_API virtual ~IAudioEncoder();

		/**
		 * Call this function when audio is available. 
		 * If you would like to handle compression on a separate thread or call,
		 * Call this function with bEncodeIfPossible set to false.
		 * Returns true if audio was successfully pushed, false, if the internal buffer is full.
		 */
		SIGNALPROCESSING_API bool PushAudio(const float* InBuffer, int32 NumSamples, bool bEncodeIfPossible = true);

		/**
		 * Pop compressed data. If you are using this encoder for streaming over network,
		 * use the size returned by GetCompressedPacketSize().
		 * Returns number of bytes written.
		 */
		SIGNALPROCESSING_API int32 PopData(uint8* OutData, int32 DataSize);

		/**
		 * Used for internet streaming. Should return the amount of bytes required for a self contained packet.
		 */
		virtual int32 GetCompressedPacketSize() const = 0;

		/**
		 * If you'd like to run audio encoding on a separate thread, use this call.
		 * Otherwise, ensure that bEncodeIfPossible is set to true when you call PushAudio.
		 */
		SIGNALPROCESSING_API bool EncodeIfPossible();

		/**
		 * Call this once you are finished pushing audio.
		 * Returns the amount of bytes left to pop if positive, and if negative indicates a failiure.
		 */
		SIGNALPROCESSING_API int64 Finalize();

	protected:
		/**
		 * Should be called in the constructor of any implementation of IAudioEncoder.
		 * Calls StartFile if necessary.
		 */
		SIGNALPROCESSING_API void Init(const FSoundQualityInfo& InQualityInfo);

		/**
		 * How many samples of decoded audio that are required for a single compression operation.
		 */
		virtual int64 SamplesRequiredPerEncode() const = 0;

		/**
		 * This should be overridden to, give the properties in InQualityInfo, write a header into
		 * OutFileStart.
		 * returns true on success, or false on failure.
		 */
		virtual bool StartFile(const FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutFileStart) = 0;

		/**
		 * Override this to compress InAudio to OutBytes.
		 * Returns true on success, false on failure.
		 */
		virtual bool EncodeChunk(const TArray<float>& InAudio, TArray<uint8>& OutBytes) = 0;

		/**
		* Override this to compress InAudio to OutBytes.
		* Returns true on success, false on failure.
		*/
		virtual bool EndFile(TArray<uint8>& OutBytes) = 0;

	private:
		SIGNALPROCESSING_API IAudioEncoder();

		TArray<float> CurrentAudioBuffer;
		TArray<uint8> CurrentCompressedBuffer;

		TCircularAudioBuffer<float> UncompressedAudioBuffer;
		TCircularAudioBuffer<uint8> CompressedDataBuffer;
	};
}

