// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Encoders/IAudioEncoder.h"

#if PLATFORM_SUPPORTS_VORBIS_ENCODE

struct FOggVorbisEncoderPrivateState;

class FOggVorbisEncoder : public Audio::IAudioEncoder
{
public:
	SIGNALPROCESSING_API FOggVorbisEncoder(const FSoundQualityInfo& InInfo, int32 AverageBufferCallbackSize);

	// From IAudioEncoder: returns 0, since Ogg Vorbis is not built for networked streaming.
	SIGNALPROCESSING_API virtual int32 GetCompressedPacketSize() const override;

protected:

	// From IAudioEncoder:
	SIGNALPROCESSING_API virtual int64 SamplesRequiredPerEncode() const override;
	SIGNALPROCESSING_API virtual bool StartFile(const FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutFileStart) override;
	SIGNALPROCESSING_API virtual bool EncodeChunk(const TArray<float>& InAudio, TArray<uint8>& OutBytes) override;
	SIGNALPROCESSING_API virtual bool EndFile(TArray<uint8>& OutBytes) override;

private:
	FOggVorbisEncoder();

	int32 NumChannels;

	// Private, uniquely owned state.
	// This must be a raw pointer because it has a non-default destructor that isn't public.
	FOggVorbisEncoderPrivateState* PrivateState;
};

#endif // PLATFORM_SUPPORTS_VORBIS_ENCODE
