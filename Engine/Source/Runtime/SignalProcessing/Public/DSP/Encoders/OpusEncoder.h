// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Encoders/IAudioEncoder.h"

#if !PLATFORM_TVOS

class FOpusEncoderPrivateState;
class FOggEncapsulator;

// Possible frame sizes to use for the encoder.
enum class EOpusFrameSizes : uint8
{
	Min, // 2.5 milliseconds
	Small, // 5 milliseconds
	MediumLow, // 10 milliseconds
	MediumHigh, // 20 milliseconds
	High, // 40 milliseconds
	Max, // 60 milliseconds
};

enum class EOpusMode : uint8
{
	File, // Use this when encoding a .opus file. Pushes the Opus frames into Ogg packets.
	AudioStream, // Use this for general music and non-speech streaming applications.
	VoiceStream // Use this for Voice-specific applications.
};

class FOpusEncoder : public Audio::IAudioEncoder
{
public:
	FOpusEncoder(const FSoundQualityInfo& InInfo, int32 AverageBufferCallbackSize, EOpusFrameSizes InFrameSize = EOpusFrameSizes::MediumLow, EOpusMode InMode = EOpusMode::File);
	~FOpusEncoder();

	virtual int32 GetCompressedPacketSize() const override;

protected:
	virtual int64 SamplesRequiredPerEncode() const override;
	virtual bool StartFile(const FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutFileStart) override;
	virtual bool EncodeChunk(const TArray<float>& InAudio, TArray<uint8>& OutBytes) override;
	virtual bool EndFile(TArray<uint8>& OutBytes) override;

private:
	FOpusEncoder();

	int32 GetNumSamplesForEncode(EOpusFrameSizes InFrameSize) const;
	int32 GetNumSamplesForPreskip();

	int32 LastValidFrameSize;
	int32 NumChannels;
	int32 SampleRate;
	int32 UncompressedFrameSize;

	// Private state so that we don't have a public dependency on opus libraries.
	// Uniquely owned by this instance. Only a raw pointer because the destructor is not accessible.
	FOpusEncoderPrivateState* PrivateOpusState;

	// Private state. Only used if we are generating a .opus file, which are ogg encapsulations of an opus stream.
	FOggEncapsulator* PrivateOggEncapsulator;

	// Used for .opus files only:
	uint32 GranulePos;
	uint32 PacketIndex;
};

#endif // !PLATFORM_TVOS
