// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Encoders/IAudioEncoder.h"

class FWavEncoder : public Audio::IAudioEncoder
{
public:
	FWavEncoder(const FSoundQualityInfo& InInfo, int32 AudioCallbackSize);

	virtual int32 GetCompressedPacketSize() const override;

protected:
	virtual int64 SamplesRequiredPerEncode() const override;
	virtual bool StartFile(const FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutFileStart) override;
	virtual bool EncodeChunk(const TArray<float>& InAudio, TArray<uint8>& OutBytes) override;
	virtual bool EndFile(TArray<uint8>& OutBytes) override;

private:
	int32 CallbackSize;

	FWavEncoder();

};