// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Osc.h"
#include "DSP/ModulationMatrix.h"
#include "DSP/BufferVectorOperations.h"

namespace Audio
{
	// Envelope class generates ADSR style envelope
	class FEnvelope
	{
	public:
		// States for the envelope state machine
		enum class EEnvelopeState
		{
			Off,
			Attack,
			Decay,
			Sustain,
			Release,
			Shutdown
		};
		
		SIGNALPROCESSING_API FEnvelope();
		SIGNALPROCESSING_API virtual ~FEnvelope();

		// Initialize the envelope with the given sample rate
		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InVoiceId = 0, FModulationMatrix* InModMatrix = nullptr, const bool bInSimulateAnalog = true);

		// Sets the envelope mode 
		SIGNALPROCESSING_API void SetSimulateAnalog(const bool bInSimulatingAnalog);

		// Sets whether the envelope is in legato mode. Legato mode doesn't restart the envelope if it's already playing
		void SetLegato(const bool bInLegatoMode) { bIsLegatoMode = bInLegatoMode; }

		// Sets whether or not the envelope is zero'd when reset
		void SetRetrigger(const bool bInRetrigger) { bIsRetriggerMode = bInRetrigger; }
		bool IsRetrigger() const { return bIsRetriggerMode; }

		// Start the envelope, puts envelope in attack state
		SIGNALPROCESSING_API virtual void Start();

		// For truly legato envelope logic, we need to know the new sustain gain (if its being changed).
		SIGNALPROCESSING_API virtual void StartLegato(const float InNewDepth);
		virtual void StartLegato() { StartLegato(Depth); }

		// Stop the envelope, puts in the release state. Can optionally force to off state.
		SIGNALPROCESSING_API virtual void Stop();

		// Puts envelope into shutdown mode, which is a much faster cutoff than release, but avoids pops
		SIGNALPROCESSING_API virtual void Shutdown();

		// Kills the envelope (will cause discontinuity)
		SIGNALPROCESSING_API virtual void Kill();

		// Queries if the envelope has finished
		SIGNALPROCESSING_API virtual bool IsDone() const;

		// Resets the envelope
		SIGNALPROCESSING_API virtual void Reset();

		// Update the state of the envelope
		SIGNALPROCESSING_API virtual void Update();

		// Generate the next output value of the envelope.
		// Optionally outputs the bias output (i.e. -1.0 to 1.0)
		SIGNALPROCESSING_API virtual float Generate(float* BiasedOutput = nullptr);

		SIGNALPROCESSING_API virtual EEnvelopeState GetState() const;

		// Sets the envelope attack time in msec
		SIGNALPROCESSING_API virtual void SetAttackTime(const float InAttackTimeMsec);

		// Sets the envelope decay time in msec
		SIGNALPROCESSING_API virtual void SetDecayTime(const float InDecayTimeMsec);

		// Sets the envelope sustain gain in linear gain values
		SIGNALPROCESSING_API virtual void SetSustainGain(const float InSustainGain);

		// Sets the envelope release time in msec
		SIGNALPROCESSING_API virtual void SetReleaseTime(const float InReleaseTimeMsec);

		// Inverts the value of envelope output
		SIGNALPROCESSING_API virtual void SetInvert(const bool bInInvert);

		// Inverts the value of the biased envelope output
		SIGNALPROCESSING_API virtual void SetBiasInvert(const bool bInBiasInvert);

		// Sets the envelope depth.
		SIGNALPROCESSING_API virtual void SetDepth(const float InDepth);

		// Sets the depth of the bias output. 
		SIGNALPROCESSING_API virtual void SetBiasDepth(const float InDepth);

		// Get the envelope's patch nodes
		const FPatchSource GetModSourceEnv() const { return EnvSource; }
		const FPatchSource GetModSourceBiasEnv() const { return BiasedEnvSource; }

	protected:
		struct FEnvData
		{
			float Coefficient;
			float Offset;
			float TCO;
			float TimeSamples;

			FEnvData()
				: Coefficient(0.0f)
				, Offset(0.0f)
				, TCO(0.0f)
				, TimeSamples(0.0f)
			{}
		};

		// The current envelope value, used to compute exponential envelope curves
		int32 VoiceId;
		float CurrentEnvelopeValue;
		float CurrentEnvelopeBiasValue;
		float SampleRate;
		float AttackTimeMSec;
		float DecayTimeMsec;
		float SustainGain;
		float ReleaseTimeMsec;
		float ShutdownTimeMsec;
		float ShutdownDelta;
		float Depth;
		float BiasDepth;
		float OutputGain;

		FEnvData AttackData;
		FEnvData DecayData;
		FEnvData ReleaseData;

		EEnvelopeState CurrentState;

		// Mod matrix
		FModulationMatrix* ModMatrix;

		FPatchSource EnvSource;
		FPatchSource BiasedEnvSource;

		// Whether or not we want to simulate analog behavior
		uint8 bIsSimulatingAnalog:1;

		// Whether or not the envelope is reset back to attack when started
		uint8 bIsLegatoMode:1;

		// Whether or not the envelope value is set to zero when finished
		uint8 bIsRetriggerMode:1;

		// Whether or not this envelope has changed and needs to have values recomputed
		uint8 bChanged:1;

		// Inverts the output
		uint8 bInvert:1;

		// Bias output inversions
		uint8 bBiasInvert:1;

		// tracks if the current envelope was started with sustain at 0.0
		// (avoids bug where sustain being turned up during decay phase makes note hang)
		uint8 bCurrentCycleIsADOnly:1;
	};

	// sample accurate attack-decay style envelope generator
	class FADEnvelope
	{
	public:
		FADEnvelope() {}
		~FADEnvelope() = default;

		SIGNALPROCESSING_API void Init(int32 InSampleRate);

		SIGNALPROCESSING_API void SetAttackTimeSeconds(float InAttackTimeSeconds);
		SIGNALPROCESSING_API void SetDecayTimeSeconds(float InReleaseTimeSeconds);
		SIGNALPROCESSING_API void SetAttackCurveFactor(float InAttackCurve);
		SIGNALPROCESSING_API void SetDecayCurveFactor(float InDecayCurve);

		void SetLooping(bool bInIsLooping) { bIsLooping = bInIsLooping; }
		bool IsLooping() const { return bIsLooping; }

		// Call function to trigger a new attack-phase of the envelope generator
		SIGNALPROCESSING_API void Attack();

		// Generates an output audio buffer (used for audio-rate envelopes)
		SIGNALPROCESSING_API void GetNextEnvelopeOut(int32 StartFrame, int32 EndFrame, TArray<int32>& OutFinishedFrames, Audio::AlignedFloatBuffer& OutEnvelope);

		// Generates a single float value (used for control-rate envelopes)
		SIGNALPROCESSING_API void GetNextEnvelopeOut(int32 StartFrame, int32 EndFrame, TArray<int32>& OutFinishedFrames, float& OutEnvelope);
		SIGNALPROCESSING_API bool GetNextEnvelopeOut(float& OutEnvelope);

	private:
		int32 CurrentSampleIndex = INDEX_NONE;
		float StartingEnvelopeValue = 0.0f;
		float CurrentEnvelopeValue = 0.0f;
		float AttackTimeSeconds = 0.0f;
		float DecayTimeSeconds = 0.1f;
		int32 AttackSampleCount = 0;
		int32 DecaySampleCount = 0;
		float AttackCurveFactor = 1.0f;
		float DecayCurveFactor = 1.0f;
		float SampleRate = 0.0f;
		bool bIsLooping = false;

	};
}
