// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DSP/AlignedBuffer.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/Dsp.h"
#include "HAL/Platform.h"

namespace Audio
{
	// Different modes for the envelope follower
	namespace EPeakMode
	{
		enum Type
		{
			MeanSquared,
			RootMeanSquared,
			Peak,
			TruePeak,
			Count
		};
	}


	/** Conversion between attack/release time and attack/release sample counts. */
	class FAttackRelease
	{
		// see https://en.wikipedia.org/wiki/RC_time_constant
		// Time constants indicate how quickly the envelope follower responds to changes in input
		static constexpr float AnalogTimeConstant = 1.00239343f;
		static constexpr float DigitalTimeConstant = 4.60517019f;

	public:
		/** Construct an FAttackRelease object.
		 *
		 * @param InSampleRate - The number of frames per a second.
		 * @param InAttackTimeMsec - The desired attack time in milliseconds.
		 * @param InReleaseTimeMsec - The desired release time in milliseconds.
		 * @param bInIsAnalog - Whether to model analog RC circuits or use digital models.
		 */
		SIGNALPROCESSING_API FAttackRelease(float InSampleRate, float InAttackTimeMsec, float InReleaseTimeMsec, bool bInIsAnalog);

		SIGNALPROCESSING_API void SetAnalog(bool bInIsAnalog);
		SIGNALPROCESSING_API void SetAttackTime(float InAttackTimeMsec);
		SIGNALPROCESSING_API void SetReleaseTime(float InReleaseTimeMsec);

		/** Get whether set to analog or digital time constant. (True is analog, false is digital) */
		inline bool GetAnalog() const { return bIsAnalog; }

		/** Get the attack time in samples. */
		inline float GetAttackTimeSamples() const { return AttackTimeSamples; }

		/** Get the release time in samples. */
		inline float GetReleaseTimeSamples() const { return ReleaseTimeSamples; }

		/** Get the attack time in milliseconds. */
		inline float GetAttackTimeMsec() const { return AttackTimeMsec; }

		/** Get the release time in milliseconds. */
		inline float GetReleaseTimeMsec() const { return ReleaseTimeMsec; }

		/** Get the sample rate. */
		inline float GetSampleRate() const { return SampleRate; }

	protected:

		SIGNALPROCESSING_API void SetSampleRate(float InSampleRate);

	private:

		float SampleRate;
		float AttackTimeSamples;
		float AttackTimeMsec;
		float ReleaseTimeSamples;
		float ReleaseTimeMsec;
		bool bIsAnalog;
	};
	

	/** Smooths signals using attack and release settings. */
	class FAttackReleaseSmoother : public FAttackRelease
	{
	public:
		/** Construct an FAttackReleaseSmoother.
		 *
		 * @param InSampleRate - The number of samples per a second of incoming data.
		 * @param InNumChannels - The number of input channels. 
		 * @param InAttackTimeMsec - The desired attack time in milliseconds.
		 * @param InReleaseTimeMsec - The desired release time in milliseconds.
		 * @param bInIsAnalog - Whether to model analog RC circuits or use digital models.
		 */
		SIGNALPROCESSING_API FAttackReleaseSmoother(float InSampleRate, int32 InNumChannels, float InAttackTimeMsec, float InReleaseTimeMsec, bool bInIsAnalog);

		/** Process channel interleaved data.
		 *
		 * @param InBuffer - Interleaved channel data. 
		 * @param InNumFrames - Number of frames in InBuffer.
		 */
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, int32 InNumFrames);

		/** Process channel interleaved data.
		 *
		 * @param InBuffer - Interleaved input data. 
		 * @param InNumFrames - Number of frames in InBuffer.
		 * @param OutBuffer - Output interleaved envelope data. Should be same size as InBuffer.
		 */
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer);

		/** Retrieve the final values of the envelope for each channel. */
		SIGNALPROCESSING_API const TArray<float>& GetEnvelopeValues() const;

		/** Set the number of input channels */
		SIGNALPROCESSING_API void SetNumChannels(int32 InNumChannels);

		SIGNALPROCESSING_API void Reset();

	private:
		int32 NumChannels;
		TArray<float> PriorEnvelopeValues;
	};

	/** Compute mean squared using FIR method.  */
	class FMeanSquaredFIR
	{
		static constexpr int32 DefaultHistoryCapacity = 16384;

	public:

		/** Construct an FMeanSquaredFIR
		 *
		 * @parma InSampleRate - Number of frames per a second.
		 * @param InNumChannels - Number of channels per a frame.
		 * @param InWindowTimeMsec - Number of milliseconds per a mean squared window.
		 */
		SIGNALPROCESSING_API FMeanSquaredFIR(float InSampleRate, int32 InNumChannels, float InWindowTimeMsec);

		/** Set the size of the analysis window. */
		SIGNALPROCESSING_API void SetWindowSize(float InWindowTimeMsec);

		/** Set the number of input channels. */
		SIGNALPROCESSING_API void SetNumChannels(int32 InNumChannels);

		SIGNALPROCESSING_API void Reset();

		/** Calculate mean squared per sample.
		 *
		 * @param InBuffer - Interleaved input data. 
		 * @param InNumFrames - Number of frames in InBuffer.
		 * @param OutBuffer - Output interleaved data. Should be same size as InBuffer.
		 */
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer);

	private:
		float SampleRate;
		int32 NumChannels;
		int32 WindowTimeFrames;
		int32 WindowTimeSamples;
		float NormFactor;

		TArray<float> ChannelValues;
		TCircularAudioBuffer<float> HistorySquared;
		FAlignedFloatBuffer SquaredHistoryBuffer;
		FAlignedFloatBuffer SquaredInputBuffer;
	};

	/** Compute mean squared using IIR method.  */
	class FMeanSquaredIIR
	{
	public:
		/** Construct an FMeanSquaredIIR
		 *
		 * @parma InSampleRate - Number of frames per a second.
		 * @param InNumChannels - Number of channels per a frame.
		 * @param InWindowTimeMsec - Number of milliseconds per a mean squared window.
		 */
		SIGNALPROCESSING_API FMeanSquaredIIR(float InSampleRate, int32 InNumChannels, float InWindowTimeMsec);

		/** Set the size of the analysis window. */
		SIGNALPROCESSING_API void SetWindowSize(float InWindowTimeMsec);

		/** Set the number of input channels. */
		SIGNALPROCESSING_API void SetNumChannels(int32 InNumChannels);

		SIGNALPROCESSING_API void Reset();

		/** Calculate mean squared per sample.
		 *
		 * @param InBuffer - Interleaved input data. 
		 * @param InNumFrames - Number of frames in InBuffer.
		 * @param OutBuffer - Output interleaved data. Should be same size as InBuffer.
		 */
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer);

	private:
		float SampleRate;
		int32 NumChannels;
		float Alpha;
		float Beta;

		TArray<float> ChannelValues;
	};

	/** Compute True-Peak as defined by ITU-R BS.1770-5 */
	class FTruePeak
	{
	public:
		FTruePeak(int32 InNumChannels)
			: NumChannels(InNumChannels)
		{
			if (InNumChannels > 0)
			{
				Reset();
			}
		}

		void SetNumChannels(int32 InNumChannels)
		{
			if (NumChannels != InNumChannels)
			{
				NumChannels = InNumChannels;
				Reset();
			}
		}

		SIGNALPROCESSING_API void Reset();
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer);

	private:
		int32 NumChannels = 0;
		TArray<float> AllChannelSignalHistories;
		uint32 HistoryPosition = 0;
	};

	struct FEnvelopeFollowerInitParams
	{
		/** Number of frames per a second. */
		float SampleRate = 48000.f;
		/** Number of channels per a frame. */
		int32 NumChannels = 1;
		/** The desired attack time in milliseconds. */
		float AttackTimeMsec = 10.0f;
		/** The desired release time in milliseconds. */
		float ReleaseTimeMsec = 100.0f;
		/** Technique for measuring amplitude of input signal. */
		EPeakMode::Type Mode = EPeakMode::Peak;
		/** Whether to model analog RC circuits or use digital models. */
		bool bIsAnalog = true;
		/** Number of milliseconds per a (root) mean squared window. */
		float AnalysisWindowMsec=5.f;
	};

	/** A simple utility that returns a smoothed value given audio input using 
	 * an RC circuit.  Used for following the envelope of an audio stream.
	 */
	class FEnvelopeFollower
	{
	public:
		/** Construct an envelope follower. */
		SIGNALPROCESSING_API FEnvelopeFollower();

		/** Construct an envelope follower. */
		SIGNALPROCESSING_API FEnvelopeFollower(const FEnvelopeFollowerInitParams& InParams);

		/** Initialize the envelope follower. */
		SIGNALPROCESSING_API void Init(const FEnvelopeFollowerInitParams& InParams);

		/** Returns the number of channels per an input frame */
		SIGNALPROCESSING_API int32 GetNumChannels() const;

		/** Returns the number of frames per a second set on initialization */
		SIGNALPROCESSING_API float GetSampleRate() const;

		/** Returns the envelope follower attack time (how fast the envelope responds to input) */
		SIGNALPROCESSING_API float GetAttackTimeMsec() const;

		/** Returns the envelope follower release time (how slow the envelope dampens from input) */
		SIGNALPROCESSING_API float GetReleaseTimeMsec() const;

		/** Returns whether or not to use analog or digital time constants */
		SIGNALPROCESSING_API bool GetAnalog() const;

		/** Returns the input mode of the envelope follower */
		SIGNALPROCESSING_API EPeakMode::Type GetMode() const;

		/** Set the number of channels per an input frame. */
		SIGNALPROCESSING_API void SetNumChannels(int32 InNumChannels);

		/** Sets whether or not to use analog or digital time constants */
		SIGNALPROCESSING_API void SetAnalog(bool bInIsAnalog);

		/** Sets the envelope follower attack time (how fast the envelope responds to input) */
		SIGNALPROCESSING_API void SetAttackTime(float InAttackTimeMsec);

		/** Sets the envelope follower release time (how slow the envelope dampens from input) */
		SIGNALPROCESSING_API void SetReleaseTime(float InReleaseTimeMsec);

		/** Sets the input mode of the envelope follower */
		SIGNALPROCESSING_API void SetMode(EPeakMode::Type InMode);

		/** Calculate envelope per sample.
		 *
		 * @param InBuffer - Interleaved input data. 
		 * @param InNumFrames - Number of frames in InBuffer.
		 * @param OutBuffer - Output interleaved data. Should be same size as InBuffer.
		 */
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, int32 InNumFrames, float* OutBuffer);

		/** Calculate envelope
		 *
		 * @param InBuffer - Interleaved input data. 
		 * @param InNumFrames - Number of frames in InBuffer.
		 */
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, int32 InNumFrames);
		
		/** Retrieve the final values of the envelope for each channel. */
		SIGNALPROCESSING_API const TArray<float>& GetEnvelopeValues() const;

		/** Resets the state of the envelope follower */
		SIGNALPROCESSING_API void Reset();

	private:
		void ProcessWorkBuffer(const float* InBuffer, int32 InNumFrames);

		FAlignedFloatBuffer WorkBuffer;
		FMeanSquaredIIR MeanSquaredProcessor; 
		FTruePeak TruePeakProcessor;
		FAttackReleaseSmoother Smoother;

		int32 NumChannels;
		EPeakMode::Type EnvMode;
	};

	struct FInlineEnvelopeFollowerInitParams
	{
		/** Number of frames per a second. */
		float SampleRate = 48000.f;
		/** The desired attack time in milliseconds. */
		float AttackTimeMsec = 10.0f;
		/** The desired release time in milliseconds. */
		float ReleaseTimeMsec = 100.0f;
		/** Technique for measuring amplitude of input signal. */
		EPeakMode::Type Mode = EPeakMode::Peak;
		/** Whether to model analog RC circuits or use digital models. */
		bool bIsAnalog = true;
		/** Number of milliseconds per a (root) mean squared window. */
		float AnalysisWindowMsec=5.f;
	};

	/** FInlineEnvelopeFollower is useful for low sample rate use cases and where
	 * samples are only available one at a time. This class is inlined because
	 * there are situations where it is needed in a CPU intensive situations.
	 */
	class FInlineEnvelopeFollower : public FAttackRelease
	{

	public:

		/** Construct an envelope follower. */
		FInlineEnvelopeFollower(const FInlineEnvelopeFollowerInitParams& InParams)
		: FAttackRelease(InParams.SampleRate, InParams.AttackTimeMsec, InParams.ReleaseTimeMsec, InParams.bIsAnalog)
		, Value(0.f)
		, Mode((InParams.Mode != EPeakMode::TruePeak) ? InParams.Mode : EPeakMode::Peak)
		, AnalysisValue(0.f)
		, AnalysisWindowMsec(5.f)
		{
			ensure(InParams.Mode != EPeakMode::TruePeak);
			SetAnalysisWindow(InParams.AnalysisWindowMsec);
		}

		FInlineEnvelopeFollower()
		: FInlineEnvelopeFollower(FInlineEnvelopeFollowerInitParams{})
		{
		}

		/** Initialize an envelope follower.  */
		void Init(const FInlineEnvelopeFollowerInitParams& InParams)
		{
			ensure(InParams.Mode != EPeakMode::TruePeak);
			SetSampleRate(InParams.SampleRate);
			SetAttackTime(InParams.AttackTimeMsec);
			SetReleaseTime(InParams.ReleaseTimeMsec);
			SetMode((InParams.Mode != EPeakMode::TruePeak) ? InParams.Mode : EPeakMode::Peak);
			SetAnalog(InParams.bIsAnalog);
			SetAnalysisWindow(InParams.AnalysisWindowMsec);
		}

		/* Sets the input analysis mode of the envelope follower */
		void SetMode(EPeakMode::Type InMode)
		{
			if (ensure(InMode != EPeakMode::TruePeak))
			{
				Mode = InMode;
			}
		}

		/** Set the analysis window size (for MeanSquared and RootMeanSquared). */
		void SetAnalysisWindow(float InAnalysisWindowMsec)
		{
			if (ensure(InAnalysisWindowMsec > 0.f))
			{
				AnalysisWindowMsec = InAnalysisWindowMsec;
				InitMeanSquaredCoefficients(AnalysisWindowMsec, GetSampleRate());
			}
		}

		/** Process a single sample and return the envelope value. */
		inline float ProcessSample(float InSample)
		{
			float NormSample = NormalizeSample(InSample);

			float Diff = Value - NormSample;
			if (Value <= NormSample)
			{
				Value = (GetAttackTimeSamples() * Diff) + NormSample;
			}
			else
			{
				Value = (GetReleaseTimeSamples() * Diff) + NormSample;
			}

			return Value;
		}

		inline void ProcessBuffer(const float* InSamples, const int32 InNumSamples, float* OutSamples)
		{
			check(InSamples != OutSamples);

			NormalizeSamples(InSamples, InNumSamples, OutSamples);

			for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
			{
				float NormSample = OutSamples[SampleIndex];
				float Diff = Value - NormSample;
				if (Value <= NormSample)
				{
					Value = (GetAttackTimeSamples() * Diff) + NormSample;
				}
				else
				{
					Value = (GetReleaseTimeSamples() * Diff) + NormSample;
				}
				OutSamples[SampleIndex] = Value;
			}
		}

		void Reset()
		{
			Value = 0.f;
		}

		/** Return the most recent envelope value. */
		inline float GetValue() const
		{
			return Value;
		}

	private:
		inline float NormalizeSample(float InSample)
		{
			switch (Mode)
			{
				case EPeakMode::Peak:
					return FMath::Abs(InSample);
					
				case EPeakMode::MeanSquared:

					{
						float SampleSquared = InSample * InSample;
						AnalysisValue = AnalysisFilterBeta * SampleSquared + AnalysisFilterAlpha * AnalysisValue;
						return AnalysisValue;
					}

				case EPeakMode::RootMeanSquared:

					{
						float SampleSquared = InSample * InSample;
						AnalysisValue = AnalysisFilterBeta * SampleSquared + AnalysisFilterAlpha * AnalysisValue;
						return FMath::Sqrt(AnalysisValue);
					}

				case EPeakMode::TruePeak:
				default:
				{
					checkNoEntry();
				}
			}

			return InSample;
		}

		inline void NormalizeSamples(const float* InSamples, const int32 InNumSamples, float* OutSamples)
		{
			TArrayView<const float> In = TArrayView<const float>(InSamples, InNumSamples);
			TArrayView<float> Out = TArrayView<float>(OutSamples, InNumSamples);
			switch (Mode)
			{
			case EPeakMode::Peak:
				ArrayAbs(In, Out);
				return;

			case EPeakMode::MeanSquared:
			{
				ArraySquare(In, Out);
				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
				{
					AnalysisValue = AnalysisFilterBeta * OutSamples[SampleIndex] + AnalysisFilterAlpha * AnalysisValue;
					OutSamples[SampleIndex] = AnalysisValue;
				}
				return;
			}

			case EPeakMode::RootMeanSquared:

			{
				ArraySquare(In, Out);
				for (int32 SampleIndex = 0; SampleIndex < InNumSamples; ++SampleIndex)
				{
					AnalysisValue = AnalysisFilterBeta * OutSamples[SampleIndex] + AnalysisFilterAlpha * AnalysisValue;
					OutSamples[SampleIndex] = AnalysisValue;
				}
				ArraySqrtInPlace(Out);
				return;
			}

			case EPeakMode::TruePeak:
			default:
			{
				checkNoEntry();
			}
			}
		}

		void InitMeanSquaredCoefficients(float InWinMsec, float InSampleRate)
		{
			if (ensure((InSampleRate > 0.f) && (InWinMsec > 0.f)))
			{
				AnalysisFilterAlpha = FMath::Exp(-1000.f / (InSampleRate * InWinMsec));
				AnalysisFilterBeta = 1.f - AnalysisFilterAlpha;
			}
		}

		float Value = 0.f;
		EPeakMode::Type Mode = EPeakMode::Peak;
		float AnalysisValue = 0.f;
		float AnalysisWindowMsec = 10.f;
		float AnalysisFilterAlpha = 0.f;
		float AnalysisFilterBeta = 1.f;
	};
}
