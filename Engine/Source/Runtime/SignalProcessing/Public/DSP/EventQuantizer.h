// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	enum class EEventQuantization : uint8
	{
		None,
		Bars8,
		Bars4,
		Bars2,
		Bar,
		HalfNote,
		HalfNoteTriplet,
		QuarterNote, 
		QuarterNoteTriplet,
		EighthNote, 
		EighthNoteTriplet,
		SixteenthNote, 
		SixteenthNoteTriplet,
		ThirtySecondNote, 

		Count,
	};

	struct FEventQuantizationSettings
	{
		// The sample rate (in frames per second) of the output audio buffer
		uint32 SampleRate;

		// Number of channels in the output audio buffer
		uint32 NumChannels;

		// The beats per minute
		float BeatsPerMinute;

		// The beats per bar
		uint32 BeatsPerBar;

		// What the "global" quantization setting is
		EEventQuantization GlobalQuantization;

		// The beat division (must be power of 2)
		uint16 BeatDivision;

		FEventQuantizationSettings()
			: SampleRate(0.0f)
			, NumChannels(2)
			, BeatsPerMinute(90.0f)
			, BeatsPerBar(4)
			, GlobalQuantization(EEventQuantization::Bar)
			, BeatDivision(4)
		{}
	};

	// Event listener interface. Event listeners make callbacks to objects that are interested in specific events.
	// Can be hooked to non-audio systems to trigger BP delegates, animation triggering, etc.
	// Note: this event listener is called from the audio render thread (or async audio rendering/generation task).
	class IQuantizedEventListener
	{
	public:
		// Callback for when a specific event type occurs. Listeners register themselves for specific events.
		// But will get the same callback called. I.e. same object can register for multiple events.
		// @param EventQuantizationType			The event type that just happened
		// @param Bars							The number of bars in that this event happened
		// @param Beat							The beat within the bar that the event happened (can be fractional for sub-division events)
		virtual void OnEvent(EEventQuantization EventQuantizationType, int32 Bars, float Beat) = 0;
	};

	// Class which handles the details of event quantization.
	class FEventQuantizer
	{
	public:
		SIGNALPROCESSING_API FEventQuantizer();
		SIGNALPROCESSING_API ~FEventQuantizer();

		// Sets the quantization settings for the event quantizer
		SIGNALPROCESSING_API void SetQuantizationSettings(const FEventQuantizationSettings& QuantizationSettings);
		const FEventQuantizationSettings& GetQuantizationSettings() const { return QuantizationSettings; }

		// Allows continuous control over BPM for the event quantizer
		SIGNALPROCESSING_API void SetBPM(const float InBPM);
		float GetBPM() const { return QuantizationSettings.BeatsPerMinute; }

		// Set the beat division
		SIGNALPROCESSING_API void SetBeatDivision(const uint16 InBeatDivision);
		uint16 GetBeatDivision() const { return QuantizationSettings.BeatDivision; }

		SIGNALPROCESSING_API float GetPlaybacktimeSeconds() const;

		SIGNALPROCESSING_API uint32 GetDurationInFrames(int32 NumBars, float NumBeats) const;

		// Called to perform notifications for any events which happen in the next given number of frames.
		// This function should be called in an audio buffer render callback.
		SIGNALPROCESSING_API void NotifyEvents(int32 NumFrames);

		// Enqueues a quantized event
		SIGNALPROCESSING_API void EnqueueEvent(EEventQuantization InQuantization, TFunction<void(uint32 NumFramesOffset)> Lambda);

		// Register event listener for specific events
		SIGNALPROCESSING_API void RegisterListenerForEvent(IQuantizedEventListener* InListener, EEventQuantization InQuantization);

		// Unregisters the event listener for all quantization events
		SIGNALPROCESSING_API void UnregisterListenerForEvent(IQuantizedEventListener* InListener);

		// Unregister the event listener for specific quantization event
		SIGNALPROCESSING_API void UnregisterListenerForEvent(IQuantizedEventListener* InListener, EEventQuantization InQuantization);

	private:

		SIGNALPROCESSING_API void SetQuantizationSettingsInternal(const FEventQuantizationSettings& QuantizationSettings);
		SIGNALPROCESSING_API void SetBPMInternal(const float InBPM);
		SIGNALPROCESSING_API void ResetEventState();

		// Struct for defining and hold quantization timing state
		struct FEventQuantizationState
		{
			// The current frame count of this quantization type
			uint32 FrameCount;

			// The frame duration of this quantization type (how many frames for the event to reoccur)
			uint32 EventFrameDuration;

			// Array of events queued for this state
			TArray<TFunction<void(uint32 NumFramesOffset)>> QueuedEvents;

			// Event listeners for this specific event
			TArray<IQuantizedEventListener*> EventListeners;

			FEventQuantizationState()
				: FrameCount(0)
				, EventFrameDuration(INDEX_NONE)
			{}
		};

		SIGNALPROCESSING_API void NotifyEventForState(FEventQuantizationState& State, EEventQuantization Type, bool bIsQuantizationEvent, int32 NumFrames);

		// The frame count of the whole event quantizer since it started. Passed to listeners.
		uint32 FrameCount;
		uint32 NumFramesPerBar;
		uint32 NumFramesPerBeat;

		// Scratch buffer for copied events
		TArray<TFunction<void(uint32 NumFramesOffset)>> CopiedEvents;

		// The quantization settings of the event quantizer
		FEventQuantizationSettings QuantizationSettings;
		EEventQuantization EventQuantizationForSettingsChange;

		// Array of quantization states, one for each quantization type
		// Zero'th index is reserved for pending quantization events
		FEventQuantizationState EventQuantizationStates[(int32)EEventQuantization::Count];

		// Quantization state used for quantizing BPM changes
		FEventQuantizationState BPMQuantizationState;

		// Whether or not we've set the quantization settings
		bool bQuantizationSettingsSet;

		// Whether or not to reset pending event states. Set after quantization settings have changed.
		bool bResetEventState;
	};



}
