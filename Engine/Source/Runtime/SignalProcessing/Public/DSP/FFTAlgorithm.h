// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

namespace Audio
{
	/** EFFTScaling
	 *
	 * EFFTScaling describes how transformed data is scaled. IFFTAlgorithm implementations
	 * must return this value to describe how to achieve equal energy between input data and 
	 * transformed data. It is assumed that applying the logical inverse of the scaling will
	 * produce equal energy.
	 */
	enum class EFFTScaling : uint8
	{
		/** No scaling needed to maintain equal energy. */
		None,

		/** Output is effectively multiplied by FFTSize. To maintain equal energy, output must be divided by the FFTSize. */
		MultipliedByFFTSize,

		/** Output is effectively multiplied by the square root of the FFTSize. To maintain equal energy, output must be divided by the square root of the FFTSize. */
		MultipliedBySqrtFFTSize,

		/** Output is effectively divided by the square root of the FFTSize. To maintain equal energy, output must be multiplied by the square root of the FFTSize. */
		DividedByFFTSize,

		/** Output is effectively divided by the square root of the FFTSize. To maintain equal energy, output must be multiplied by the square root of the FFTSize. */
		DividedBySqrtFFTSize,
	};

	/** FFFTSettings
	 *
	 * Settings for creating an IFFTAlgorithm.
	 */
	struct FFFTSettings
	{
		/** Log2Size of FFT algorithm. */
		int32 Log2Size;

		/** If true, the algorithm expects input & ouput arrays to be 128bit aligned. */
		bool bArrays128BitAligned;

		/** If true, hardware accelerated algorithms are valid. */
		bool bEnableHardwareAcceleration;
	};

	/** IFFTAlgorithm 
	 *
	 * Interface for FFT algorithm.
	 */
	class IFFTAlgorithm 
	{
		public:
			/** virtual destructor for inheritance. */
			SIGNALPROCESSING_API virtual ~IFFTAlgorithm();

			/** Number of elements in FFT */
			virtual int32 Size() const = 0;

			/** Number of floats expected in input. */
			int32 NumInputFloats() const { return Size(); };

			/** Number of floats produced in output. */
			int32 NumOutputFloats() const { return Size() + 2; }

			/** Scaling applied when performing forward FFT. */
			virtual EFFTScaling ForwardScaling() const = 0;

			/** Scaling applied when performing inverse FFT. */
			virtual EFFTScaling InverseScaling() const = 0;

			
			/** ForwardRealToComplex
			 *
			 * Perform the forward FFT on real data and produce complex data.
			 *
			 * @param InReal - Array of floats to input into fourier transform. Must have NumInputFloats() elements.
			 * @param OutComplex - Array of floats to store output of fourier transform. Must have NumOutputFloats() elements which represent (NumOutputFloats() / 2) complex numbers in interleaved format.
			 */
			virtual void ForwardRealToComplex(const float* RESTRICT InReal, float* RESTRICT OutComplex) = 0;

			/** InverseComplexToReal
			 *
			 * Perform the inverse FFT on complex data and produce real data.
			 *
			 * @param InComplex - Array of floats to input into inverse fourier transform. Must have NumOutputFloats() elements which represent complex numbers in interleaved format.
			 * @param OutReal - Array of floats to store output of inverse fourier transform. Must have NumInputFloats() elements.
			 */
			virtual void InverseComplexToReal(const float* RESTRICT InComplex, float* RESTRICT OutReal) = 0;

			/** BatchForwardRealToComplex
			 *
			 * Perform the forward FFT on real data and produce complex data.
			 *
			 * @param InCount - The number of transforms to compute.
			 * @param InReal - Array of array of floats to input into fourier transform. Must have InCount elements, each containing NumInputFloats() elements.
			 * @param OutComplex - Array of array of floats to store output of fourier transform. Must have InCount elements, each with NumOutputFloats()elements which represent complex numbers in interleaved format.
			 */
			virtual void BatchForwardRealToComplex(int32 InCount, const float* const RESTRICT InReal[], float* RESTRICT OutComplex[]) = 0;

			/** BatchInverseComplexToReal
			 *
			 * Perform the inverse FFT on complex data and produce real data.
			 *
			 * @param InCount - The number of transforms to compute.
			 * @param InComplex - Array of array of floats to input into inverse fourier transform. Must have InCount elements, each with NumOutputFloats() elements which represent complex numbers in interleaved format.
			 * @param OutReal - Array of array of floats to store output of inverse fourier transform. Must have InCount elements, each with NumInputFloats() elements.
			 */
			virtual void BatchInverseComplexToReal(int32 InCount, const float* const RESTRICT InComplex[], float* RESTRICT OutReal[]) = 0;
	};

	/** IFFTAlgorithmFactory
	 *
	 * Factory interface for creating fft algorithms.
	 */
	class IFFTAlgorithmFactory : public IModularFeature
	{
		public:
			virtual ~IFFTAlgorithmFactory() = default;

			/** Name of modular feature for FFT factory.  */
			static SIGNALPROCESSING_API const FName GetModularFeatureName();

			/** Name of this particular factory. */
			virtual FName GetFactoryName() const = 0;

			/** If true, this implementation uses hardware acceleration. */
			virtual bool IsHardwareAccelerated() const = 0;

			/** If true, this implementation requires input and output arrays to be 128 bit aligned. */
			virtual bool Expects128BitAlignedArrays() const = 0;

			/** Returns true if the input settings are supported by this factory. */
			virtual bool AreFFTSettingsSupported(const FFFTSettings& InSettings) const = 0;

			/** Creates a new FFT algorithm. */
			virtual TUniquePtr<IFFTAlgorithm> NewFFTAlgorithm(const FFFTSettings& InSettings) = 0;
	};

	/** FFFTFactory
	 *
	 * FFFTFactory creates fft algorithms. It will choose hardware accelerated versions of the FFT when they are available. 
	 */
	class FFFTFactory
	{
		public:
			/** This denotes that no specific IFFTAlgorithmFactory is desired. */
			static SIGNALPROCESSING_API const FName AnyAlgorithmFactory;

			/** NewFFTAlgorithm
			 *
			 * Creates and returns a new FFTAlgorithm. 
			 *
			 * @param InSettings - The settings used to create the FFT algorithm.
			 * @param InAlgorithmFactoryName - If not equal to FFFTFactory::AnyAlgorithmFactory, will only uses FFT algorithm facotry if IFFTAlgorithmFactory::GetFactoryName() equals InAlgorithmFactoryName.
			 * @return A TUniquePtr<IFFTAlgorithm> to the created FFT. This pointer can be invalid if an error occured or the fft algorithm could not be created.
			 */
			static SIGNALPROCESSING_API TUniquePtr<IFFTAlgorithm> NewFFTAlgorithm(const FFFTSettings& InSettings, const FName& InAlgorithmFactoryName = AnyAlgorithmFactory);

			/** AreFFTSettingsSupported
			 *
			 * Returns true if a valid FFT algorithm can be created which satifies the given settings.
			 *
			 * @param InSettings - Settings describing desired FFT algorithm.
			 * @param InAlgorithmFactoryName - If not equal to FFFTFactory::AnyAlgorithmFactory, will only uses FFT algorithm facotry if IFFTAlgorithmFactory::GetFactoryName() equals InAlgorithmFactoryName.
			 * @return True if settings are supported. Otherwise false.
			 */
			static SIGNALPROCESSING_API bool AreFFTSettingsSupported(const FFFTSettings& InSettings, const FName& InAlgorithmFactoryName = AnyAlgorithmFactory);
	};
}
