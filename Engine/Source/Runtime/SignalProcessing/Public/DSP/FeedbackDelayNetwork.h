// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/IntegerDelay.h"
#include "DSP/LongDelayAPF.h"
#include "DSP/BufferOnePoleLPF.h"

namespace Audio
{
	// Delay settings of 4 channel feedback delay network
	struct FFDNDelaySettings
	{
		// Number of samples for delay line 0
		int32 APF0DelayNumSamples;
		// Number of samples for delay line 1
		int32 APF1DelayNumSamples;
		// Number of samples for delay line 2
		int32 APF2DelayNumSamples;
		// Number of samples for delay line 3
		int32 APF3DelayNumSamples;
		
		// Default delay line settings for left channel.
		static SIGNALPROCESSING_API FFDNDelaySettings DefaultLeftDelays(float InSampleRate);
		// Default delay line settings for right channel. 
		static SIGNALPROCESSING_API FFDNDelaySettings DefaultRightDelays(float InSampleRate);
	};

	// Filter coefficients of 4 channel feedback delay network.
	struct FFDNCoefficients
	{
		// Sample multiplier of input samples before they enter the delay line.
		float InputScale;
		// All pass filter coefficients. One coefficient for each filter.
		float APFG[4];
		// Low pass filter feed back coefficients. One coefficient for each filter.
		float LPFA[4];
		// Low pass filter feed forward coefficients. One coefficient for each filter.
		float LPFB[4];
		// Feedback multiplier of mixing matrix output to network input.
		float Feedback;
	};

	// 4 channel feedback delay network (FDN) for artificial reverberation.
	class FFeedbackDelayNetwork
	{
	public:
		// InMaxNumInternalBufferSamples controls the internal buffer size used for vector operations. 
		// InSettings controls the delay line lengths.
		SIGNALPROCESSING_API FFeedbackDelayNetwork(int32 InMaxNumInternalBufferSamples, const FFDNDelaySettings& InSettings);

		SIGNALPROCESSING_API ~FFeedbackDelayNetwork();

		// Sets the coefficient values of the all pass filters, low pass filters, input scalers and feedback scalers.
		SIGNALPROCESSING_API void SetCoefficients(const FFDNCoefficients& InCoefficients);

		// Generates artificial reverberation for InSamples and places results in OutSamples.
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples);

		// Sets internal audio samples to silence. 
		SIGNALPROCESSING_API void FlushAudio();

		// Returns the actual number of samples in an internal buffer.
		SIGNALPROCESSING_API int32 GetNumInternalBufferSamples() const;


	private:

		// Process single internal buffer of audio.
		void ProcessAudioBuffer(const float* InSampleData, const int32 InNum, float* OutSampleData);

		FFDNDelaySettings Settings;

		// Internal buffer size.
		int32 NumInternalBufferSamples;

		// Coefficients structure for APF, LPF, input scaling and feedback.
		FFDNCoefficients Coefficients;

		// Previous output of low pass filters.
		float LPFZ[4];

		// Previous output of mixing matrix.
		float FMO[4];

		TUniquePtr<FAlignedBlockBuffer> DelayLine0;
		TUniquePtr<FAlignedBlockBuffer> DelayLine1;
		TUniquePtr<FAlignedBlockBuffer> DelayLine2;
		TUniquePtr<FAlignedBlockBuffer> DelayLine3;

		FAlignedFloatBuffer WorkBuffer0;
		FAlignedFloatBuffer WorkBuffer1;
		FAlignedFloatBuffer WorkBuffer2;
		FAlignedFloatBuffer WorkBuffer3;
	};
}
