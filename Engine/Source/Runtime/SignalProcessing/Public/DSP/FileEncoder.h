// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Encoders/IAudioEncoder.h"
#include "Interfaces/IAudioFormat.h"

namespace Audio
{
	class FAudioFileWriter
	{
	public:
		// Constructor. Takes an absolute file path and metadata about the audio in question.
		// Immediately allocates all data necessary.
		SIGNALPROCESSING_API FAudioFileWriter(const FString& InPath, const FSoundQualityInfo& InInfo);

		// Calling the destructor on this class finalizes and closes out the file.
		SIGNALPROCESSING_API ~FAudioFileWriter();

		// Returns file information.
		SIGNALPROCESSING_API void GetFileInfo(FSoundQualityInfo& OutInfo);

		// Use this to push audio to the file writer.
		// If you'd like to move encoding and file writing to a separate thread from PushAudio,
		// Call this with bEncodeIfPossible = false.
		SIGNALPROCESSING_API bool PushAudio(const float* InAudio, int32 NumSamples, bool bEncodeIfPossible = true);

		// If PushAudio is called with bEncodeIfPossible set to false, this will need to be called.
		SIGNALPROCESSING_API bool EncodeIfPossible();

	private:
		FAudioFileWriter();

		FSoundQualityInfo QualityInfo;
		TArray<uint8> DataBuffer;

		// Compressor we are using.
		TUniquePtr<IAudioEncoder> Encoder;

		// Handle to file we are writing to.
		TUniquePtr<IFileHandle> FileHandle;

		IAudioEncoder* GetNewEncoderForFile(const FString& InPath);
		FString GetExtensionForFile(const FString& InPath);
		void FlushEncoderToFile(int32 DataBufferSize = 4096);
	};
}
