// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/ModulationMatrix.h"
#include "DSP/Dsp.h"


namespace Audio
{
	// Enumeration of filter types
	namespace EBiquadFilter
	{
		enum Type
		{
			Lowpass,
			Highpass,
			Bandpass,
			Notch,
			ParametricEQ,
			LowShelf,
			HighShelf,
			AllPass,
			ButterworthLowPass,
			ButterworthHighPass
		};
	}

	// Biquad filter class which wraps a biquad filter struct
	// Handles multi-channel audio to avoid calculating filter coefficients for multiple channels of audio.
	class FBiquadFilter
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FBiquadFilter();
		// Destructor
		SIGNALPROCESSING_API virtual ~FBiquadFilter();

		// Initialize the filter
		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumChannels, const EBiquadFilter::Type InType, const float InCutoffFrequency = 20000.0f, const float InBandwidth = 2.0f, const float InGain = 0.0f);

		// Returns number of channels initialized with
		SIGNALPROCESSING_API int32 GetNumChannels() const;

		// Resets the filter state
		SIGNALPROCESSING_API void Reset();

		// Processes a single frame of audio
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame);

		// Process a mono buffer or an interleaved buffer of multichannel audio
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer);

		// Process a non-interleaved buffer of multichannel audio
		SIGNALPROCESSING_API void ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers);

		// Sets all filter parameters with one function
		SIGNALPROCESSING_API void SetParams(const EBiquadFilter::Type InFilterType, const float InCutoffFrequency, const float InBandwidth, const float InGainDB);

		// Sets the type of the filter to use
		SIGNALPROCESSING_API void SetType(const EBiquadFilter::Type InType);

		// Sets the filter frequency
		SIGNALPROCESSING_API void SetFrequency(const float InCutoffFrequency);

		// Sets the bandwidth (octaves) of the filter
		SIGNALPROCESSING_API void SetBandwidth(const float InBandwidth);

		// Sets the gain of the filter in decibels
		SIGNALPROCESSING_API void SetGainDB(const float InGainDB);

		// Sets whether or no this filter is enabled (if disabled audio is passed through)
		SIGNALPROCESSING_API void SetEnabled(const bool bInEnabled);

		// Apply the filter transfer function to each z-domain value in the given array (complex numbers given as interleaved floats). Passing in z-domain values on the complex unit circle will give the frequency response.
		SIGNALPROCESSING_API void ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const;

	protected:
		struct FBiquadCoeff;

		// Function computes biquad coefficients based on current filter settings
		SIGNALPROCESSING_API void CalculateBiquadCoefficients();

		// Function used to clamp the cutoff frequency.
		SIGNALPROCESSING_API float ClampCutoffFrequency(float InCutoffFrequency);

		// What kind of filter to use when computing coefficients
		EBiquadFilter::Type FilterType;

		// Biquad filter objects for each channel
		FBiquadCoeff* Biquad;

		// The sample rate of the filter
		float SampleRate;

		// Number of channels in the filter
		int32 NumChannels;

		// Current frequency of the filter
		float Frequency;

		// Current bandwidth/resonance of the filter
		float Bandwidth;

		// The gain of the filter in DB (for filters that use gain)
		float GainDB;

		// Whether or not this filter is enabled (will bypass otherwise)
		bool bEnabled;
	};

	namespace EFilter
	{
		enum Type
		{
			LowPass,
			HighPass,
			BandPass,
			BandStop,

			NumFilterTypes
		};
	}

	static const int32 MaxFilterChannels = 8;

	// Base class for filters usable in synthesis
	class IFilter
	{
	public:
		SIGNALPROCESSING_API IFilter();
		SIGNALPROCESSING_API IFilter(const IFilter&);
		SIGNALPROCESSING_API virtual ~IFilter();

		// Initialize the filter
		SIGNALPROCESSING_API virtual void Init(const float InSampleRate, const int32 InNumChannels, const int32 InVoiceId, FModulationMatrix* InModMatrix = nullptr);

		// Sets the cutoff frequency of the filter.
		SIGNALPROCESSING_API virtual void SetFrequency(const float InCutoffFrequency);

		// Sets an external modulated frequency
		SIGNALPROCESSING_API virtual void SetFrequencyMod(const float InModFrequency);

		// Sets the quality/resonance of the filter
		SIGNALPROCESSING_API virtual void SetQ(const float InQ);

		// Sets an external modulated quality/resonance of the filter
		SIGNALPROCESSING_API virtual void SetQMod(const float InModQ);

		// Sets the filter saturation (not used on all filters)
		virtual void SetSaturation(const float InSaturation) {}

		// Sets the band stop control param (not used on all filters)
		virtual void SetBandStopControl(const float InBandStop) {}

		// Sets the band pass gain compensation (not used on all filters)
		virtual void SetPassBandGainCompensation(const float InPassBandGainCompensation) {}

		// Sets the filter type
		SIGNALPROCESSING_API virtual void SetFilterType(const EFilter::Type InFilterType);

		// Reset the filter
		SIGNALPROCESSING_API virtual void Reset();

		// Updates the filter
		SIGNALPROCESSING_API virtual void Update();

		// Processes a single frame of audio. Number of channels MUST be what was used during filter initialization.
		virtual void ProcessAudioFrame(const float* InFrame, float* OutFrame)
		{
			ProcessAudio(InFrame, NumChannels, OutFrame);
		}

		// Process a mono buffer or an interleaved buffer of multichannel audio
		virtual void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer) = 0;

		// Process a non-interleaved buffer of multichannel audio
		virtual void ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers) = 0;

		// Filter patch destinations
		FPatchDestination GetModDestCutoffFrequency() const { return ModCutoffFrequencyDest; }
		FPatchDestination GetModDestQ() const { return ModQDest; }

	protected:

		inline float GetGCoefficient() const
		{
			const float OmegaDigital = 2.0f * PI * Frequency;
			const float OmegaAnalog = 2.0f * SampleRate * Audio::FastTan(0.5f * OmegaDigital / SampleRate);
			const float G = 0.5f * OmegaAnalog / SampleRate;
			return G;
		}

		// The voice id of the owner of the filter (for use with mod matrix)
		int32 VoiceId;

		// Sample rate of the filter
		float SampleRate;

		// Number of channels of the sound
		int32 NumChannels;

		// The cutoff frequency currently used by the filter (can be modulated)
		float Frequency;

		// The base filter cutoff frequency
		float BaseFrequency;

		// A modulated frequency
		float ModFrequency;

		// A modulatied frequency driven externally
		float ExternalModFrequency;

		// The current Q
		float Q;

		// The modulated Q
		float ModQ;

		// The base 
		float BaseQ;

		// An external modulation of Q
		float ExternalModQ;

		// The filter type of the filter
		EFilter::Type FilterType;

		FModulationMatrix* ModMatrix;

		// Mod matrix patchable destinations
		FPatchDestination ModCutoffFrequencyDest;
		FPatchDestination ModQDest;

		bool bChanged;
	};

	// A virtual analog one-pole filter.
	// Defaults to a low-pass mode, but can be switched to high-pass
	class FOnePoleFilter : public IFilter
	{
	public:
		SIGNALPROCESSING_API FOnePoleFilter();
		SIGNALPROCESSING_API virtual ~FOnePoleFilter();

		SIGNALPROCESSING_API virtual void Init(const float InSampleRate, const int32 InNumChannels, const int32 InVoiceId = 0, FModulationMatrix* InModMatrix = nullptr) override;
		SIGNALPROCESSING_API virtual void Reset() override;
		SIGNALPROCESSING_API virtual void Update() override;
		SIGNALPROCESSING_API virtual void ProcessAudioFrame(const float* InFrame, float* OutFrame) override;
		// Process a mono buffer or an interleaved buffer of multichannel audio
		SIGNALPROCESSING_API virtual void ProcessAudio(const float* InSamples, const int32 InNumSamples, float* OutSamples) override;
		// Process a non-interleaved buffer of multichannel audio
		SIGNALPROCESSING_API virtual void ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers) override;

		void SetCoefficient(const float InCoefficient) { A0 = InCoefficient; }
		float GetCoefficient() const { return A0; }

		float GetState(const int32 InChannel) const { return Z1[InChannel]; }

		// Apply the filter transfer function to each z-domain value in the given array (complex numbers given as interleaved floats). Passing in z-domain values on the complex unit circle will give the frequency response.
		SIGNALPROCESSING_API void ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const;

	protected:
		float A0;
		float* Z1;
	};

	class FStateVariableFilter : public IFilter
	{
	public:
		SIGNALPROCESSING_API FStateVariableFilter();
		SIGNALPROCESSING_API virtual ~FStateVariableFilter();

		SIGNALPROCESSING_API virtual void Init(const float InSampleRate, const int32 InNumChannels, const int32 InVoiceId = 0, FModulationMatrix* InModMatrix = nullptr) override;
		SIGNALPROCESSING_API virtual void SetBandStopControl(const float InBandStop) override;
		SIGNALPROCESSING_API virtual void Reset() override;
		SIGNALPROCESSING_API virtual void Update() override;
		// Process a mono buffer or an interleaved buffer of multichannel audio
		SIGNALPROCESSING_API virtual void ProcessAudio(const float* InSamples, const int32 InNumSamples, float* OutSamples) override;
		// Process a non-interleaved buffer of multichannel audio
		SIGNALPROCESSING_API virtual void ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers) override;

		// Process a mono buffer or an interleaved buffer of multichannel audio
		SIGNALPROCESSING_API void ProcessAudio(const float* InSamples, const int32 InNumSamples,
			float* LpfOutput, float* HpfOutput, float* BpfOutput, float* BsfOutput);
		// Process a non-interleaved buffer of multichannel audio
		SIGNALPROCESSING_API void ProcessAudio(const float* const* InBuffers, const int32 InNumSamples,
			float* const* LpfOutBuffers, float* const* HpfOutBuffers, float* const* BpfOutBuffers, float* const* BsfOutBuffers);

		// Apply the filter transfer function to each z-domain value in the given array (complex numbers given as interleaved floats). Passing in z-domain values on the complex unit circle will give the frequency response.
		// The transfer function utilized here represents the behavior of the filter if the non-linearity were removed.
		SIGNALPROCESSING_API void ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const;

	protected:
		float InputScale;
		float A0;
		float Feedback;
		float BandStopParam;

		struct FFilterState
		{
			float Z1_1;
			float Z1_2;

			FFilterState()
				: Z1_1(0.0f)
				, Z1_2(0.0f)
			{}
		};

		TArray<FFilterState> FilterState;
	};

	class FLadderFilter : public IFilter
	{
	public:
		SIGNALPROCESSING_API FLadderFilter();
		SIGNALPROCESSING_API virtual ~FLadderFilter();

		SIGNALPROCESSING_API virtual void Init(const float InSampleRate, const int32 InNumChannels, const int32 InVoiceId = 0, FModulationMatrix* InModMatrix = nullptr) override;
		SIGNALPROCESSING_API virtual void Reset() override;
		SIGNALPROCESSING_API virtual void Update() override;
		SIGNALPROCESSING_API virtual void SetQ(const float InQ) override;
		SIGNALPROCESSING_API virtual void SetPassBandGainCompensation(const float InPassBandGainCompensation) override;

		// Process a mono buffer or an interleaved buffer of multichannel audio
		SIGNALPROCESSING_API virtual void ProcessAudio(const float* InSamples, const int32 InNumSamples, float* OutSamples) override;

		// Process a non-interleaved buffer of multichannel audio
		SIGNALPROCESSING_API virtual void ProcessAudio(const float* const* InBuffers, const int32 InNumSamples, float* const* OutBuffers) override;

		// Apply the filter transfer function to each z-domain value in the given array (complex numbers given as interleaved floats). Passing in z-domain values on the complex unit circle will give the frequency response.
		// The transfer function utilized here represents the behavior of the filter if the non-linearity were removed.
		SIGNALPROCESSING_API void ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const;

	protected:

		// Ladder filter is made of 4 LPF filters
		FOnePoleFilter OnePoleFilters[4];
		float Beta[4];
		float K;
		float Gamma;
		float Alpha;
		float Factors[5];
		float PassBandGainCompensation;
	};

}
