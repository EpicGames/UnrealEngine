// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Delay.h"

namespace Audio
{
	class FFlanger
	{
	public: 
		SIGNALPROCESSING_API FFlanger();
		SIGNALPROCESSING_API ~FFlanger();

		SIGNALPROCESSING_API void Init(const float InSampleRate);

		SIGNALPROCESSING_API void SetModulationRate(const float InModulationRate);
		SIGNALPROCESSING_API void SetModulationDepth(const float InModulationDepth);
		SIGNALPROCESSING_API void SetCenterDelay(const float InCenterDelay);
		SIGNALPROCESSING_API void SetMixLevel(const float InMixLevel);

		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InBuffer, const int32 InNumSamples, FAlignedFloatBuffer& OutBuffer);
	protected: 
		static SIGNALPROCESSING_API const float MaxDelaySec;
		static SIGNALPROCESSING_API const float MaxModulationRate;
		static SIGNALPROCESSING_API const float MaxCenterDelay;

		// LFO parameters
		// LFO frequency 
		float ModulationRate = 0.5f;
		// Modulation depth is clamped to CenterDelayMsec to avoid clipping
		float ModulationDepth = 0.5f;
		float CenterDelayMsec = 0.5f;

		// Balance between original and delayed signal 
		// (Should be between 0 and 1.0; 
		// 0.5 is equal amounts of each and 
		// > 0.5 is more delayed signal than non-delayed signal)
		float MixLevel = 0.5f;
		float DelayedSignalLevel = 0.5f;
		float NonDelayedSignalLevel = 0.5f;

		// Internal delay buffer 
		FDelay DelayBuffer;
		// Scratch buffer used for accumulating delay samples per block
		FAlignedFloatBuffer ScratchBuffer;

		// Delay sample generated from LFO per block
		float DelaySample = 0.0f;

		// Internal LFO for delay amount
		FSinOsc2DRotation LFO;

		// The audio sample rate, Init() must be called to initialize this
		float SampleRate = 48000.0f;
	};
}
