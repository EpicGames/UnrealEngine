// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"

namespace Audio
{
	/** Sum all values in an array. */
	SIGNALPROCESSING_API void ArraySum(TArrayView<const float> InValues, float& OutSum);

	/** Sums two buffers together and places the result in the resulting buffer. */
	SIGNALPROCESSING_API void ArraySum(TArrayView<const float> InFloatBuffer1, TArrayView<const float> InFloatBuffer2, TArrayView<float> OutputBuffer);

	/** Cumulative sum of array.
	 *
	 *  InView contains data to be cumulatively summed.
	 *  OutData contains sum and is same size as InView.
	 */
	SIGNALPROCESSING_API void ArrayCumulativeSum(TArrayView<const float> InView, TArray<float>& OutData);

	/** Mean of array. Equivalent to Sum(InView) / InView.Num()
	 *
	 *  InView contains data to be analyzed.
	 *  OutMean contains the result.
	 */
	SIGNALPROCESSING_API void ArrayMean(TArrayView<const float> InView, float& OutMean);

	/** Mean Squared of array. Equivalent to Sum(InView * InView) / InView.Num()
	 *
	 *  InArray contains data to be analyzed.
	 *  OutMean contains the result.
	 */
	SIGNALPROCESSING_API void ArrayMeanSquared(TArrayView<const float> InView, float& OutMean);

	/** Takes an audio buffer and returns the magnitude across that buffer. */
	SIGNALPROCESSING_API float ArrayGetMagnitude(TArrayView<const float> Buffer);

	/** Takes an audio buffer and gets the average amplitude across that buffer. */
	SIGNALPROCESSING_API float ArrayGetAverageValue(TArrayView<const float> Buffer);

	/** Takes an audio buffer and gets the average absolute amplitude across that buffer. */
	SIGNALPROCESSING_API float ArrayGetAverageAbsValue(TArrayView<const float> Buffer);

	/** Mean filter of array.
	 *
	 *  Note: Uses standard biased mean estimator of Sum(x) / Count(x).
	 *  Note: At array boundaries, this algorithm truncates windows where no valid array data exists. Values calculated with truncated windows have corresponding increased variances. 
	 *
	 *  InView contains data to be filtered.
	 *  WindowSize determines the number of samples from InView analyzed to produce a value in OutData.
	 *  WindowOrigin describes the offset from the windows first sample to the index of OutData. For example, if WindowOrigin = WindowSize/4, then OutData[i] = Mean(InView[i - Window/4 : i + 3 * Window / 4]).
	 *  OutData contains the produceds data.
	 */
	SIGNALPROCESSING_API void ArrayMeanFilter(TArrayView<const float> InView, int32 WindowSize, int32 WindowOrigin, TArray<float>& OutData);

	/** Max filter of array.
	 *
	 *  Note: At array boundaries, this algorithm truncates windows where no valid array data exists. 
	 *
	 *  InView contains data to be filtered.
	 *  WindowSize determines the number of samples from InView analyzed to produce a value in OutData.
	 *  WindowOrigin describes the offset from the windows first sample to the index of OutData. For example, if WindowOrigin = WindowSize/4, then OutData[i] = Max(InView[i - Window/4 : i + 3 * Window / 4]).
	 *  OutData contains the produceds data.
	 */
	SIGNALPROCESSING_API void ArrayMaxFilter(TArrayView<const float> InView, int32 WindowSize, int32 WindowOrigin, TArray<float>& OutData);

	/** Computes the EuclideanNorm of the InView. Same as calculating the energy in window. */
	SIGNALPROCESSING_API void ArrayGetEuclideanNorm(TArrayView<const float> InView, float& OutEuclideanNorm);

	/** Absolute value of array elements */
	SIGNALPROCESSING_API void ArrayAbs(TArrayView<const float> InBuffer, TArrayView<float> OutBuffer);

	/** Absolute value of array elements in place.
	 *
	 *  InBuffer contains the data to be manipulated.
	 */
	SIGNALPROCESSING_API void ArrayAbsInPlace(TArrayView<float> InBuffer);

	/** Clamp minimum value of array in place.
	 *
	 *  InView contains data to be clamped.
	 *  InMin contains the minimum value allowable in InView.
	 */
	SIGNALPROCESSING_API void ArrayClampMinInPlace(TArrayView<float> InView, float InMin);

	/** Clamp maximum value of array in place.
	 *
	 *  InView contains data to be clamped.
	 *  InMax contains the maximum value allowable in InView.
	 */
	SIGNALPROCESSING_API void ArrayClampMaxInPlace(TArrayView<float> InView, float InMax);

	/** Clamp values in an array.
	 *
	 *  InView is a view of a float array to be clamped.
	 *  InMin is the minimum value allowable in the array.
	 *  InMax is the maximum value allowable in the array.
	 */
	SIGNALPROCESSING_API void ArrayClampInPlace(TArrayView<float> InView, float InMin, float InMax);

	/** Scale an array so the minimum is 0 and the maximum is 1
	 *
	 *  InView is the view of a float array with the input data.
	 *  OutArray is an array which will hold the normalized data.
	 */ 
	SIGNALPROCESSING_API void ArrayMinMaxNormalize(TArrayView<const float> InView, TArray<float>& OutArray);

	/** Element-wise Max
	 */
	SIGNALPROCESSING_API void ArrayMax(const TArrayView<const float>& InView1, const TArrayView<const float>& InView2, const TArrayView<float>& OutView);

	/** Returns the largest value of an array irrespective of sign (ex. {-3, 2, 1} would return 3).
	 *  InView is a view of a float array to get the largest absolute value from.
	 */
	SIGNALPROCESSING_API float ArrayMaxAbsValue(const TArrayView<const float> InView);

	/** Multiply the second buffer by the first buffer. */
	SIGNALPROCESSING_API void ArrayMultiply(TArrayView<const float> InFloatBufferA, TArrayView<const float> InFloatBufferB, TArrayView<float> OutBuffer);

	/** Multiply the second buffer in place by the first buffer. */
	SIGNALPROCESSING_API void ArrayMultiplyInPlace(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToMultiply);

	/** Multiplies two complex valued arrays element-wise. 
	 * This assumes elements are in interleaved format [real_0, imag_0, ..., real_N, imag_N]
	 * Stores result in InValues2
	 */
	SIGNALPROCESSING_API void ArrayComplexMultiplyInPlace(TArrayView<const float> InValues1, TArrayView<float> InValues2);

	/** Multiplies two complex valued arrays element-wise.
	 * This assumes elements are in interleaved format [real_0, imag_0, ..., real_N, imag_N]
	 * Adds result to OutArray
	 */
	SIGNALPROCESSING_API void ArrayComplexMultiplyAdd(TArrayView<const float> InValues1, TArrayView<const float> InValues2, TArrayView<float> OutArray);

	/** Multiplies the input float buffer with the given value. */
	SIGNALPROCESSING_API void ArrayMultiplyByConstant(TArrayView<const float> InFloatBuffer, float InValue, TArrayView<float> OutFloatBuffer);

	/** Similar to ArrayMultiplyByConstant, but performs the multiply in place. */
	SIGNALPROCESSING_API void ArrayMultiplyByConstantInPlace(TArrayView<float> InOutBuffer, float InGain);

	/** Add arrays element-wise in place. InAccumulateValues[i] += InValues[i]
	 *
	 *  InValues is the array to add.
	 *  InAccumulateValues is the array which holds the sum.
	 */
	SIGNALPROCESSING_API void ArrayAddInPlace(TArrayView<const float> InValues, TArrayView<float> InAccumulateValues);

	/** Adds a constant to a buffer (useful for DC offset removal) */
	SIGNALPROCESSING_API void ArrayAddConstantInplace(TArrayView<float> InOutBuffer, float InConstant);

	/** Multiply Add arrays element-wise in place. InAccumulateValues[i] += InMultiplier * InValues[i]
	 *
	 *  @param InValues - The array to add.
	 *  @param InMultiplier - The value to multiply against InValues
	 *  @param InAccumulateValues - The array which holds the sum.
	 */
	SIGNALPROCESSING_API void ArrayMultiplyAddInPlace(TArrayView<const float> InValues, float InMultiplier, TArrayView<float> InAccumulateValues);

	/** Linearly Interpolate Add arrays element-wise in place. InAccumulateValues[i] += ((1 - alpha) * InStartMultiplier + alpha * InEndMultipler) * InValues[i]
	 * Interpolation is performed over the length of the array.
	 *
	 *  @param InValues - The array to add.
	 *  @param InStartMultiplier - The beginning value to multiply against InValues
	 *  @param InEndMultiplier - The ending value to multiply against InValues
	 *  @param InAccumulateValues - The array which holds the sum.
	 */
	SIGNALPROCESSING_API void ArrayLerpAddInPlace(TArrayView<const float> InValues, float InStartMultiplier, float InEndMultiplier, TArrayView<float> InAccumulateValues);

	/** Subract arrays element-wise. OutArray = InMinuend - InSubtrahend
	 *
	 *  InMinuend is the array of data to be subtracted from.
	 *  InSubtrahend is the array of data to subtract.
	 *  OutBuffer is the array which holds the result.
	 */
	SIGNALPROCESSING_API void ArraySubtract(TArrayView<const float> InMinuend, TArrayView<const float> InSubtrahend, TArrayView<float> OutBuffer);

	/* Performs element-wise in-place subtraction placing the result in the subtrahend. InOutSubtrahend = InMinuend - InOutSubtrahend */
	SIGNALPROCESSING_API void ArraySubtractInPlace1(TArrayView<const float> InMinuend, TArrayView<float> InOutSubtrahend);

	/* Performs element-wise in-place subtraction placing the result in the minuend. InOutMinuend = InOutMinuend - InSubtrahend */
	SIGNALPROCESSING_API void ArraySubtractInPlace2(TArrayView<float> InOutMinuend, TArrayView<const float> InSubtrahend);

	/** Subtract value from each element in InValues */
	SIGNALPROCESSING_API void ArraySubtractByConstantInPlace(TArrayView<float> InValues, float InSubtrahend);

	/* Square values */
	SIGNALPROCESSING_API void ArraySquare(TArrayView<const float> InValues, TArrayView<float> OutValues);

	/** Square values in place. */
	SIGNALPROCESSING_API void ArraySquareInPlace(TArrayView<float> InValues);

	/** Take Square Root of values in place. */
	SIGNALPROCESSING_API void ArraySqrtInPlace(TArrayView<float> InValues);

	/** Perform complex conjugate of array.  Assumes complex numbers are interlaves [real_0, imag_0, real_1, image_1, ..., real_N, imag_N]. */
	SIGNALPROCESSING_API void ArrayComplexConjugate(TArrayView<const float> InValues, TArrayView<float> OutValues);

	SIGNALPROCESSING_API void ArrayComplexConjugateInPlace(TArrayView<float> InValues);

	/** Convert magnitude values to decibel values in place. db = 20 * log10(val) */
	SIGNALPROCESSING_API void ArrayMagnitudeToDecibelInPlace(TArrayView<float> InValues, float InMinimumDb);

	/** Convert power values to decibel values in place. db = 10 * log10(val) */
	SIGNALPROCESSING_API void ArrayPowerToDecibelInPlace(TArrayView<float> InValues, float InMinimumDb);

	/** Compute power of complex data. Out[i] = Complex[2 * i] * Complex[2 * i] + Complex[2 * i + 1] * Complex[2 * i + 1] */
	SIGNALPROCESSING_API void ArrayComplexToPower(TArrayView<const float> InComplexSamples, TArrayView<float> OutPowerSamples);

	/** Compute power of complex data. Out[i] = Real[i] * Real[i] + Imaginary[i] * Imaginary[i] */
	SIGNALPROCESSING_API void ArrayComplexToPower(TArrayView<const float> InRealSamples, TArrayView<const float> InImaginarySamples, TArrayView<float> OutPowerSamples);

	/* Sets a values to zero if value is denormal. Denormal numbers significantly slow down floating point operations. */
	UE_DEPRECATED(5.5, "Audio code relies on denormals flush to zero floating point mode from now on. Please use FScopedFTZFloatMode instead of this API")
	SIGNALPROCESSING_API void ArrayUnderflowClamp(TArrayView<float> InOutBuffer);

	/* Clamps the values in a buffer between a min and max value.*/
	SIGNALPROCESSING_API void ArrayRangeClamp(TArrayView<float> InOutBuffer, float InMinValue, float InMaxValue);

	/** Sets a constant to a buffer (useful for DC offset application) */
	SIGNALPROCESSING_API void ArraySetToConstantInplace(TArrayView<float> InOutBuffer, float InConstant);

	/* Performs an element-wise weighted sum OutputBuffer = (InBuffer1 x InGain1) + (InBuffer2 x InGain2) */
	SIGNALPROCESSING_API void ArrayWeightedSum(TArrayView<const float> InBuffer1, float InGain1, TArrayView<const float> InBuffer2, float InGain2, TArrayView<float> OutBuffer);

	/* Performs an element-wise weighted sum OutputBuffer = (InBuffer1 x InGain1) + InBuffer2 */
	SIGNALPROCESSING_API void ArrayWeightedSum(TArrayView<const float> InBuffer1, float InGain1, TArrayView<const float> InBuffer2, TArrayView<float> OutBuffer);

	/* Takes a float buffer and quickly interpolates it's gain from StartValue to EndValue. */
	/* This operation completely ignores channel counts, so avoid using this function on buffers that are not mono, stereo or quad */
	/* if the buffer needs to fade all channels uniformly. */
	SIGNALPROCESSING_API void ArrayFade(TArrayView<float> InOutBuffer, const float StartValue, const float EndValue);
	SIGNALPROCESSING_API void ArrayFade(TArrayView<const float> InBuffer, const float InStartValue, const float InEndValue, TArrayView<float> OutBuffer);

	/** Takes buffer InFloatBuffer, optionally multiplies it by Gain, and adds it to BufferToSumTo. */
	SIGNALPROCESSING_API void ArrayMixIn(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToSumTo, const float Gain);
	SIGNALPROCESSING_API void ArrayMixIn(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToSumTo);

	/** Takes buffer InPcm16Buffer, converts it to float and optionally multiplies it by Gain, and adds it to BufferToSumTo. */
	SIGNALPROCESSING_API void ArrayMixIn(TArrayView<const int16> InPcm16Buffer, TArrayView<float> BufferToSumTo, const float Gain = 1.0f);

	/** This version of ArrayMixIn will fade from StartGain to EndGain. */
	SIGNALPROCESSING_API void ArrayMixIn(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToSumTo, const float StartGain, const float EndGain);

	class FInterpolatedLPF;
	class FInterpolatedHPF;
	SIGNALPROCESSING_API void ArrayFilterAndMixIn(TArrayView<const float> InFloatBuffer, TArrayView<float> BufferToSumTo, const float Gain, const int32 InNumChannels, bool bEnableLPF, float InLpfCutoff, bool bEnableHPF, float InHpfCutoff, FInterpolatedLPF& InLpf, FInterpolatedHPF& InHpf, FAlignedFloatBuffer& ScratchBuffer);

	SIGNALPROCESSING_API void ArrayFloatToPcm16(TArrayView<const float> InView, TArrayView<int16> OutView);
	SIGNALPROCESSING_API void ArrayPcm16ToFloat(TArrayView<const int16> InView, TArrayView<float> OutView);

	/** Converts float PCM data to 24 bit integer PCM data */
	SIGNALPROCESSING_API void ArrayFloatToPcm24(TArrayView<const float> InView, TArrayView<int8> OutView);
	/** Copies 24 bits of a 32 bit integer (assumes data has been clamped to 24-bits) */
	SIGNALPROCESSING_API void AssignPcm24Value(int8* OutPtr, const int32 InValue, const bool bIsOutputLittleEndian);
	/** Converts float PCM data to 32 bit integer PCM data */
	SIGNALPROCESSING_API void ArrayFloatToPcm32(TArrayView<const float> InView, TArrayView<int32> OutView);
	/** Converts float PCM data to double PCM data */
	SIGNALPROCESSING_API void ArrayFloatToPcmDouble(TArrayView<const float> InView, TArrayView<double> OutView);

	/** Interleaves samples from an array of input buffers */
	SIGNALPROCESSING_API void ArrayInterleave(const TArray<FAlignedFloatBuffer>& InBuffers, FAlignedFloatBuffer& OutBuffer);

	/** Interleaves samples from an array of input buffers */
	SIGNALPROCESSING_API void ArrayInterleave(const float* const* RESTRICT InBuffers, float* RESTRICT OutBuffer, const int32 InFrames, const int32 InChannels);

	/** Interleaves samples from an array of input buffers */
	SIGNALPROCESSING_API void ArrayDeinterleave(const FAlignedFloatBuffer& InBuffer, TArray<FAlignedFloatBuffer>& OutBuffers, const int32 InChannels);

	/** Interleaves samples from an array of input buffers */
	SIGNALPROCESSING_API void ArrayDeinterleave(const TArrayView<const float> InView, TArray<FAlignedFloatBuffer>& OutBuffers, const int32 InChannels);

	/** Interleaves samples from an array of input buffers */
	SIGNALPROCESSING_API void ArrayDeinterleave(const float* RESTRICT InBuffer, float* const* RESTRICT OutBuffers, const int32 InFrames, const int32 InChannels);

	/** Interpolates a Mono audio buffer. */
	SIGNALPROCESSING_API void ArrayInterpolate(const float* RESTRICT InBuffer, float* RESTRICT OutBuffer, const int32 NumInSamples, const int32 NumOutSamples);

	/** Vectorized 16-bit integer byte swapping. */
	SIGNALPROCESSING_API void ArrayInt16SwapBytes(TArrayView<int16> InView);

	/** Vectorized 24-bit integer byte swapping. */
	SIGNALPROCESSING_API void ArrayInt24SwapBytes(TArrayView<int8> InView);

	/** Vectorized 32-bit integer byte swapping. */
	SIGNALPROCESSING_API void ArrayInt32SwapBytes(TArrayView<int32> InView);

	/** Vectorized 32-bit float byte swapping. */
	SIGNALPROCESSING_API void ArrayFloatSwapBytes(TArrayView<float> InView);

	/** Vectorized 64-bit float byte swapping. */
	SIGNALPROCESSING_API void ArrayDoubleSwapBytes(TArrayView<double> InView);

	/** Returns true if host has little endian byte ordering */
	UE_DEPRECATED(5.5, "Big Endian platforms are no longer supported.")
	SIGNALPROCESSING_API constexpr bool IsHostLittleEndian()
	{
#if PLATFORM_LITTLE_ENDIAN
		return true;
#else
		return false;
#endif // PLATFORM_LITTLE_ENDIAN
	}

	/** All Pass Filter with a long delay. */
	SIGNALPROCESSING_API void ArrayAPFLongDelayProcess(const float* InSamples, const float* InDelaySamples, const int32 InNum, float* OutSamples, float* OutDelaySamples, const float Gain);

	/** Fractional delay using linear interpolation. */
	SIGNALPROCESSING_API void ArrayLerpFractionalDelay(const float* InSamples, const float* InDelays, const float* DelayData, const int* IntegerDelays, int* UpperDelayPos, int* LowerDelayPos, const int32 InNum, float* OutSamples, const float MaxDelay);

	/** Perform complex conjugate as well as scale. */
	SIGNALPROCESSING_API void ArrayScaledComplexConjugate(const float* RESTRICT InValues, const int32 Num, float* RESTRICT OutValues, const float Scale);

	/** FContiguousSparse2DKernelTransform
	 *
	 *  FContiguousSparse2DKernelTransform applies a matrix transformation to an input array. 
	 *  [OutArray] = [[Kernal]][InView]  
	 *
	 *  It provides some optimization by exploit the contiguous and sparse qualities of the kernel rows,
	 *  which allows it to skip multiplications with the number zero. 
	 *
	 *  It works with non-sparse and non-contiguous kernels as well, but will be more computationally 
	 *  expensive than a naive implementation. Also, only takes advantage of sparse contiguous rows, not columns.
	 */
	class FContiguousSparse2DKernelTransform
	{
		struct FRow
		{
			int32 StartIndex = 0;
			TArray<float> OffsetValues;
		};

	public:
		FContiguousSparse2DKernelTransform(const FContiguousSparse2DKernelTransform& ) = delete;
		FContiguousSparse2DKernelTransform(const FContiguousSparse2DKernelTransform&& ) = delete;
		FContiguousSparse2DKernelTransform& operator=(const FContiguousSparse2DKernelTransform& ) = delete;

		/**
		 * NumInElements sets the expected number of input array elements as well as the number of elements in a row.
		 * NumOutElements sets the number of output array elements as well as the number or rows.
		 */
		SIGNALPROCESSING_API FContiguousSparse2DKernelTransform(const int32 NumInElements, const int32 NumOutElements);
		SIGNALPROCESSING_API virtual ~FContiguousSparse2DKernelTransform();

		/** Returns the required size of the input array */
		SIGNALPROCESSING_API int32 GetNumInElements() const;

		/** Returns the size of the output array */
		SIGNALPROCESSING_API int32 GetNumOutElements() const;
	
		/** Set the kernel values for an individual row.
		 *
		 *  RowIndex determines which row is being set.
		 *  StartIndex denotes the offset into the row where the OffsetValues will be inserted.
		 *  OffsetValues contains the contiguous chunk of values which represent all the nonzero elements in the row.
		 */
		SIGNALPROCESSING_API void SetRow(const int32 RowIndex, const int32 StartIndex, TArrayView<const float> OffsetValues);

		/** Transforms the input array given the kernel.
		 *
		 *  InView is the array to be transformed. It must have `NumInElements` number of elements.
		 *  OutArray is the transformed array. It will have `NumOutElements` number of elements.
		 */
		SIGNALPROCESSING_API void TransformArray(TArrayView<const float> InView, TArray<float>& OutArray) const;

		/** Transforms the input array given the kernel.
		 *
		 *  InView is the array to be transformed. It must have `NumInElements` number of elements.
		 *  OutArray is the transformed array. It will have `NumOutElements` number of elements.
		 */
		SIGNALPROCESSING_API void TransformArray(TArrayView<const float> InView, FAlignedFloatBuffer& OutArray) const;

		/** Transforms the input array given the kernel.
		 *
		 *  InArray is the array to be transformed. It must have `NumInElements` number of elements.
		 *  OutArray is the transformed array. It must be allocated to hold at least NumOutElements. 
		 */
		SIGNALPROCESSING_API void TransformArray(const float* InArray, float* OutArray) const;

	private:

		int32 NumIn;
		int32 NumOut;
		TArray<FRow> Kernel;
	};

}
