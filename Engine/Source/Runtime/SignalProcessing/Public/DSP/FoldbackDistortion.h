// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Osc.h"

namespace Audio
{
	// Foldback distortion effect
	// https://en.wikipedia.org/wiki/Foldback_(power_supply_design)
	class FFoldbackDistortion
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FFoldbackDistortion();

		// Destructor
		SIGNALPROCESSING_API ~FFoldbackDistortion();

		// Initialize the equalizer
		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumChannels);

		// Sets the foldback distortion threshold
		SIGNALPROCESSING_API void SetThresholdDb(const float InThresholdDb);

		// Sets the input gain
		SIGNALPROCESSING_API void SetInputGainDb(const float InInputGainDb);

		// Sets the output gain
		SIGNALPROCESSING_API void SetOutputGainDb(const float InOutputGainDb);

		// Processes a single audio sample
		SIGNALPROCESSING_API float ProcessAudioSample(const float InSample);

		// Processes a mono stream
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame);

		// Processes a stereo stream
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer);

	private:
		// Threshold to check before folding audio back on itself
		float Threshold;

		// Threshold times 2
		float Threshold2;

		// Threshold time 4
		float Threshold4;

		// Input gain used to force hitting the threshold
		float InputGain;

		// A final gain scaler to apply to the output
		float OutputGain;

		// How many channels we expect the audio intput to be.
		int32 NumChannels;
	};

}
