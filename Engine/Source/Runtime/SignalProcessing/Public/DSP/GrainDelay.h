// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/GrainEnvelope.h"
#include "DSP/DynamicsProcessor.h"
#include "DSP/TapDelayPitchShifter.h"

namespace Audio
{
	namespace GrainDelay
	{
		class FGrainDelay
		{
		public:
			SIGNALPROCESSING_API FGrainDelay(const float InSampleRate, const float InMaxDelaySeconds = 2.0f);
			SIGNALPROCESSING_API ~FGrainDelay();

			SIGNALPROCESSING_API void Reset();

			// Helpers to clamp audio
			SIGNALPROCESSING_API float GetGrainDelayClamped(const float InDelay) const;
			SIGNALPROCESSING_API float GetGrainDurationClamped(const float InDuration) const;
			SIGNALPROCESSING_API float GetGrainDelayRatioClamped(const float InGrainDelayRatio) const;
			SIGNALPROCESSING_API float GetGrainPitchShiftClamped(const float InPitchShift) const;
			SIGNALPROCESSING_API float GetGrainPitchShiftFrameRatio(const float InPitchShift) const;

			// Dynamically sets the max grains 
			SIGNALPROCESSING_API void SetMaxGrains(const int32 InMaxGrains); 
			
			// Sets the grain envelope 
			SIGNALPROCESSING_API void SetGrainEnvelope(const Audio::Grain::EEnvelope InGrainEnvelope);

			// Sets the feedback amount (how much output grain audio feeds into the delay line)
			SIGNALPROCESSING_API void SetFeedbackAmount(float InFeedbackAmount);

			// Sets the base pitch shift ratio of all grains 
			SIGNALPROCESSING_API void SetGrainBasePitchShiftRatio(const float InPitchRatioBase);

			// Spawns a new grain with the given parameters
			SIGNALPROCESSING_API void SpawnGrain(const float InDelay, const float InDuration, const float InPitchShiftRatioOffset);

			// Synthesize audio in the given frame range from input audio. Writes to OutAudioBuffer
			SIGNALPROCESSING_API void SynthesizeAudio(const int32 StartFrame, const int32 EndFrame, const float* InAudioBuffer, float* OutAudioBuffer);
			
		private:
			// Synthesize a single frame of audio
			float SynthesizeFrame(const Audio::FDelay& InDelayLine);

			void InitDynamicsProcessor();
			
			// Simple grain data structure for tracking grain state
			struct FGrain
			{
				// Where the grain is currently tapped from the delay line
				float DelayTapPositionMilliseconds = 0.0f;

				// The current frame the grain is in it's duration
				float NumFramesRendered = 0.0f;

				// The total number of frames this grain will render for
				float DurationFrames = 0.0f;

				// The unique randomized pitch scale, so we can modulate the base pitch scale
				float PitchShiftRatioOffset = 1.0f;
				
				// The tap delay pitch shifter which manages taps from the shared delay line for each grain
				FTapDelayPitchShifter PitchShifter;
				
				// Helper function for computing the phaser frequency and phaser increment
				void SetGrainPitchShiftRatio(const float InPitchShiftRatioBase, const float InSampleRate)
				{
					const float DurationMilliseconds = 1000.0f * DurationFrames / InSampleRate;
					PitchShifter.SetDelayLength(DurationMilliseconds);
					PitchShifter.SetPitchShiftRatio(InPitchShiftRatioBase * PitchShiftRatioOffset);
				}
			};

			// Free grain indices in the grain pool
			TArray<int32> FreeGrains;

			// Active grain indices in the grain pool
			TArray<int32> ActiveGrains;

			// The actual grain pool. Reuses tracking memory for grain lifetimes.
			TArray<FGrain> GrainPool;

			// The shared envelope value. Recomputed when a new envelope is set. 
			Grain::FEnvelope GrainEnvelope;
			Grain::EEnvelope GrainEnvelopeType = Grain::EEnvelope::Gaussian;
			
			// A pitch shifting delay line
			TArray<FTapDelayPitchShifter> DelayPitchShifters;

			// The shared delay lines per channel
			FDelay GrainDelayLine;

			// The current feedback amount
           	float FeedbackAmount = 0.0f;

			// Cached value of last rendered frame. Used for feedback
			float LastFrame = 0.0f;
			
			// Dynamics processor for the grain synthesis
			FDynamicsProcessor DynamicsProcessor;

			// The maximum grains to have rendering
			int32 MaxGrains = 16;

			// Sample rate track to easily deal with pitch scale calculations
			float SampleRate = 0.0f;
			
			// The base pitch shift ratio (pitch scale) of all grains in the grain delay
			float PitchShiftRatioBase = 1.0f;

			float MaxDelaySeconds = 2.0f;
		};
	}
}
