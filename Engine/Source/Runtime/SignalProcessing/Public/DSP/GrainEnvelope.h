// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	// Utility for managing a grain data (enveloped buffer) 
	namespace Grain
	{
		typedef TArray<float> FEnvelope;
		
		// Grain envelope types
		enum class EEnvelope
		{
			Gaussian,
			Triangle,
			DownwardTriangle,
			UpwardTriangle,
			ExponentialDecay,
			ExponentialAttack,
			Hann
		};
		
		// Utility function generates an envelope with the input array for the number of desired frames
		static void GenerateEnvelopeData(FEnvelope& InData, const int32 InNumFrames, const EEnvelope InEnvelopeType)
		{
			InData.Reset();
			InData.AddUninitialized(InNumFrames);

			const float N = static_cast<float>(InNumFrames);
			const float N_1 = N - 1.0f;
			float n = 0.0f;

			switch (InEnvelopeType)
			{
			case EEnvelope::Gaussian:
				{
					const float Denominator = 0.3f * N_1 / 2.0f;
					for (int32 i = 0; i < InNumFrames; ++i, n += 1.0f)
					{
						InData[i] = FMath::Exp(-0.5f * FMath::Pow((n - 0.5f * N_1) / Denominator, 2.0f));
					}
				}
				break;

			case EEnvelope::Triangle:
				{
					const float A = 0.5f * N_1;
					for (int32 i = 0; i < InNumFrames; ++i, n += 1.0f)
					{
						InData[i] = 1.0f - FMath::Abs((n - A) / A);
					}
				}
				break;

			case EEnvelope::DownwardTriangle:
				{
					for (int32 i = 0; i < InNumFrames; ++i, n += 1.0f)
					{
						InData[i] = 1.0f - n / N_1;
					}
				}
				break;

			case EEnvelope::UpwardTriangle:
				{
					for (int32 i = 0; i < InNumFrames; ++i, n += 1.0f)
					{
						InData[i] = n / N_1;
					}
				}
				break;

			case EEnvelope::ExponentialDecay:
				{
					for (int32 i = 0; i < InNumFrames; ++i, n += 1.0f)
					{
						InData[i] = FMath::Pow((n - N + 1.0f) / N_1, 4.0f);
					}
				}
				break;

			case EEnvelope::ExponentialAttack:
				{
					for (int32 i = 0; i < InNumFrames; ++i, n += 1.0f)
					{
						InData[i] = FMath::Pow(n / N_1, 4.0f);
					}
				}
				break;

			case EEnvelope::Hann:
				{
					for (int32 i = 0; i < InNumFrames; ++i, n += 1.0f)
					{
						InData[i] = 0.5f - 0.5f * FMath::Cos(2.0f * PI * n / N_1);
					}
				}
				break;
			}
		}

		// Returns the interpolated envelope value given the fraction through the grain
		static float GetValue(const FEnvelope& InGrainData, const float InFraction)
		{
			const int32 NumFrames = InGrainData.Num();
			const float Frame = static_cast<float>(InFraction) * (NumFrames - 1);
			const int32 PrevIndex = static_cast<int32>(Frame);
			const int32 NextIndex = FMath::Min(NumFrames, PrevIndex + 1);
			const float AlphaIndex = Frame - static_cast<float>(PrevIndex);
			const float* GrainEnvelopePtr = InGrainData.GetData();
			return FMath::Lerp(GrainEnvelopePtr[PrevIndex], GrainEnvelopePtr[NextIndex], AlphaIndex);
		}
		
	}
}
