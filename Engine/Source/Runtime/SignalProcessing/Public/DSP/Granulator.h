// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "DSP/Osc.h"
#include "DSP/SampleBufferReader.h"
#include "DSP/Envelope.h"
#include "DSP/Amp.h"
#include "DSP/DynamicsProcessor.h"
#include "SampleBuffer.h"

namespace Audio
{
	enum class EGranularSynthMode : uint8
	{
		Synthesis,
		Granulation,

		Count,
	};

	enum class EGrainEnvelopeType
	{
		Rectangular,
		Triangle,
		DownwardTriangle,
		UpwardTriangle,
		ExponentialDecay,
		ExponentialIncrease,
		Gaussian,
		Hanning,
		Lanczos,
		Cosine,
		CosineSquared,
		Welch,
		Blackman,
		BlackmanHarris,

		Count
	};

	// Simple class that generates an envelope and lets you retrieve interpolated values at any given fraction
	class FGrainEnvelope
	{
	public:
		SIGNALPROCESSING_API FGrainEnvelope();
		SIGNALPROCESSING_API ~FGrainEnvelope();

		SIGNALPROCESSING_API void GenerateEnvelope(const EGrainEnvelopeType EnvelopeType, const int32 NumFrames);
		SIGNALPROCESSING_API float GetValue(const float Fraction) const;

	private:
		EGrainEnvelopeType CurrentType;
		TArray<float> GrainEnvelope;
	};

	struct FGrainData
	{
		// What oscillator type to use
		EOsc::Type OscType;

		// Where in the buffer to seek the grain playback to
		float BufferSeekTime;

		// Duration of the grain
		float DurationSeconds;

		// Grain pitch scale (if in buffer reading mode)
		float PitchScale;

		// The grain frequency (if in synthesis mode)
		float Frequency;

		// Grain volume
		float Volume;

		// Grain pan
		float Pan;
	};

	class FGranularSynth;

	// Class representing a grain of audio
	class FGrain
	{
	public:
		SIGNALPROCESSING_API FGrain(const int32 InGrainId, FGranularSynth* InParent);
		SIGNALPROCESSING_API ~FGrain();

		// Plays the grain with the supplied grain data
		SIGNALPROCESSING_API void Play(const FGrainData& InGrainData);

		// Changes the oscillator type for the grain (if the grain is in synth mode)
		SIGNALPROCESSING_API void SetOscType(const EOsc::Type InType);

		// Sets the oscillator frequency
		SIGNALPROCESSING_API void SetOscFrequency(const float InFrequency);

		// Sets the oscillator frequency modulation
		SIGNALPROCESSING_API void SetOscFrequencyModuation(const float InFrequencyModulation);

		// Sets the grain pitch modulation 
		SIGNALPROCESSING_API void SetPitchModulation(const float InPitchModulation);

		// Sets the grain modulation
		SIGNALPROCESSING_API void SetVolumeModulation(const float InVolumeModulation);

		// Sets the pan modulation angle
		SIGNALPROCESSING_API void SetPanModulation(const float InPanModulation);

		// Changes how quickly the grain reads through envelope
		SIGNALPROCESSING_API void SetDurationScale(const float InDurationScale);

		// Queries if this grain is finished playing and needs to be reaped
		SIGNALPROCESSING_API bool IsDone() const;

		// Generates the next frame from the grain
		SIGNALPROCESSING_API bool GenerateFrame(float* OutStereoFrame);

	protected:
		SIGNALPROCESSING_API float GetEnvelopeValue();

		// Grain id
		int32 GrainId;

		// Parent synth
		FGranularSynth* Parent;

		// Grain data struct
		FGrainData GrainData;

		// The sample buffer reader to use for the grain of the mode is granulation
		FSampleBufferReader SampleBufferReader;

		// Oscillator to use for synthesis mode
		FOsc Osc;

		// The current pitch
		float CurrentPitch;

		// Current frequency
		float CurrentFrequency;

		// The current volume scale
		float CurrentVolumeScale;

		// The current pan
		float CurrentPan;

		// How quickly we read through the envelope 
		float DurationScale;

		// The current frame count the grain is on
		float CurrentFrameCount;

		// The end frame count the grain will finish on
		float EndFrameCount;

		// Speaker map based on the current grain azimuth
		TArray<float> SpeakerMap;

		// Scratch buffer for sample reading
		TArray<float> FrameScratch;
	};

	// A stereo granulator 
	class FGranularSynth
	{
	public:
		SIGNALPROCESSING_API FGranularSynth();
		SIGNALPROCESSING_API ~FGranularSynth();

		SIGNALPROCESSING_API void Init(const int32 InSampleRate, const int32 InNumInitialGrains);
		
		// Loads a sound wave to use for granular synth mode
		SIGNALPROCESSING_API void LoadSampleBuffer(const TSampleBuffer<int16>& InSampleBuffer);

		// Plays a granular synthesis "Note"
		SIGNALPROCESSING_API void NoteOn(const uint32 InMidiNote, const float InVelocity, const float InDurationSec = INDEX_NONE);

		// Note off, triggers release envelope
		SIGNALPROCESSING_API void NoteOff(const uint32 InMidiNote, const bool bKill);

		// Sets the granular synth attack time
		SIGNALPROCESSING_API void SetAttackTime(const float InAttackTimeMSec);

		// Sets the granular synth decay time
		SIGNALPROCESSING_API void SetDecayTime(const float InDecayTimeMSec);

		// Sets the granular synth sustain gain
		SIGNALPROCESSING_API void SetSustainGain(const float InSustainGain);

		// Sets the granular synth releas etime
		SIGNALPROCESSING_API void SetReleaseTime(const float InReleaseTimeMSec);

		// Seeks the loaded buffer used for granulation. Grains will spawn from this location
		SIGNALPROCESSING_API void SeekTime(const float InTimeSec, const float LerpTimeSec = 0.0f, const ESeekType::Type InSeekType = ESeekType::FromBeginning);

		// Sets whether or not the buffer playback advances on its own or if it just sits in one place
		SIGNALPROCESSING_API void SetScrubMode(const bool bIsScrubMode);

		// Sets how fast the granular play head for granulation is is played (and in what direction)
		SIGNALPROCESSING_API void SetPlaybackSpeed(const float InPlaybackSpeed);

		// The rate at which new grains are attempted to be spawned
		SIGNALPROCESSING_API void SetGrainsPerSecond(const float InNumberOfGrainsPerSecond);

		// The probability at which a grain will occur when a grain tries to spawn. Allows for sporatic grain generation.
		SIGNALPROCESSING_API void SetGrainProbability(const float InGrainProbability);

		// Sets the envelope type to use for new grains. Will instantly switch all grains to this envelope type so may cause discontinuities if switched while playing.
		SIGNALPROCESSING_API void SetGrainEnvelopeType(const EGrainEnvelopeType InGrainEnvelopeType);

		// Sets the grain oscillator type (for use with granular synthesis mode)
		SIGNALPROCESSING_API void SetGrainOscType(const EOsc::Type InGrainOscType);

		// Sets the base grain volume and randomization range
		SIGNALPROCESSING_API void SetGrainVolume(const float InBaseVolume, const FVector2D InVolumeRange = FVector2D::ZeroVector);

		// Sets the grain modulation -- allows modulating actively playing grain volumes
		SIGNALPROCESSING_API void SetGrainVolumeModulation(const float InVolumeModulation);

		// Sets the base grain pitch and randomization range
		SIGNALPROCESSING_API void SetGrainPitch(const float InBasePitch, const FVector2D InPitchRange = FVector2D::ZeroVector);

		// Sets the grain frequency
		SIGNALPROCESSING_API void SetGrainFrequency(const float InFrequency, const FVector2D InPitchRange = FVector2D::ZeroVector);

		// Sets the grain frequency modulation
		SIGNALPROCESSING_API void SetGrainFrequencyModulation(const float InFrequencyModulation);

		// Sets the grain pitch modulation -- allows modulating actively playing grain pitches
		SIGNALPROCESSING_API void SetGrainPitchModulation(const float InPitchModulation);

		// Sets the grain azimuth (pan) and randomization range
		SIGNALPROCESSING_API void SetGrainPan(const float InBasePan, const FVector2D InPanRange = FVector2D::ZeroVector);

		// Sets the grain azimuth modulation - allows modulating actively playing grain azimuths
		SIGNALPROCESSING_API void SetGrainPanModulation(const float InPanModulation);

		// Sets the grain duration. 
		SIGNALPROCESSING_API void SetGrainDuration(const float InBaseDuration, const FVector2D InDurationRange = FVector2D::ZeroVector);

		// Sets the grain duration modulation.
		SIGNALPROCESSING_API void SetGrainDurationScale(const float InDurationScale);

		// Return the number of currently active grains
		SIGNALPROCESSING_API int32 GetNumActiveGrains() const;

		// Get current playback time (in granular mode)
		SIGNALPROCESSING_API float GetCurrentPlayheadTime() const;

		// Returns the duration of the internal loaded sample buffer
		SIGNALPROCESSING_API float GetSampleDuration() const;

		// Generate the next audio buffer
		SIGNALPROCESSING_API void Generate(float* OutAudiobuffer, const int32 NumFrames);

	protected:
		// Spawns grains
		SIGNALPROCESSING_API void SpawnGrain();

		// Return wrapped playhead position
		SIGNALPROCESSING_API float GetWrappedPlayheadPosition(float PlayheadFrame);


		// Current grain azimuth modulation
		struct FGrainParam
		{
			float Modulation;
			float Base;
			FVector2D Range;

			float GetValue() const
			{
				return Base + FMath::FRandRange(Range.X, Range.Y);
			}

			float GetModulation() const
			{
				return Modulation;
			}

			FGrainParam()
				: Modulation(0.0f)
				, Base(0.0f)
			{}
		};

		int32 SampleRate;
		int32 NumChannels;

		// The single envelope function used by all grains
		FGrainEnvelope GrainEnvelope;

		// What mode the granular synthesizer is in
		EGranularSynthMode Mode;

		// The oscillator type to use if in synthesis mode
		EOsc::Type GrainOscType;

		// Current grain envelope type
		EGrainEnvelopeType GrainEnvelopeType;

		// A pool of free grains. Will dynamically grow to needed grains based on grain density.
		TArray<FGrain> GrainPool;
		TArray<int32> FreeGrains;
		TArray<int32> ActiveGrains;
		TArray<int32> DeadGrains;

		// The rate at which grains are spawned
		float GrainsPerSecond;

		// The probability of a grain occurring when it tries to spawn (based off the GrainsPerSecond)
		float GrainProbability;

		// The current number of frames since last attempt to spawn
		int32 CurrentSpawnFrameCount;

		// The next frame when a grain needs to spawn
		int32 NextSpawnFrame;

		// Counts for overall note duration of the granulator
		int32 NoteDurationFrameCount;
		int32 NoteDurationFrameEnd;

		FGrainParam Pan;
		FGrainParam Volume;
		FGrainParam Pitch;
		FGrainParam Frequency;
		FGrainParam Duration;

		// Overall envelope of the granulator
		FEnvelope GainEnv;
		FAmp Amp;
		FDynamicsProcessor DynamicsProcessor;

		// The buffer which holds the sample to be granulated
		TSampleBuffer<int16> SampleBuffer;

		// The current playhead frame
		float CurrentPlayHeadFrame;
		float PlaybackSpeed;
		int32 NumActiveGrains;
		bool bScrubMode;
		FLinearEase SeekingPlayheadTimeFrame;

		friend class FGrain;
	};
}

