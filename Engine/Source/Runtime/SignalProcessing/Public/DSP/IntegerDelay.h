// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "DSP/AlignedBlockBuffer.h"
#include "DSP/BufferVectorOperations.h"

namespace Audio
{

	// An adjustable delay line. Delays values are limited to integer values. 
	class FIntegerDelay
	{
	public:

		// InMaxDelaySamples is the maximum supported delay.
		// InDelaySamples is the initial delay in samples.
		SIGNALPROCESSING_API FIntegerDelay(int32 InMaxNumDelaySamples, int32 InNumDelaySamples, int32 InNumInternalBufferSamples = 256);

		// Destructor
		SIGNALPROCESSING_API ~FIntegerDelay();

		// Sets the current delay in samples. InDelay must be less than or equal to the InMaxDelaySamples set in the constructor.
		SIGNALPROCESSING_API void SetDelayLengthSamples(int32 InNumDelaySamples);

		// Resets the delay line state, flushes buffer and resets read/write pointers.
		SIGNALPROCESSING_API void Reset();

		// Returns the current delay length in samples.
		SIGNALPROCESSING_API int32 GetNumDelaySamples() const;

		// Process InSamples, placing delayed versions in OutSamples.
		inline float ProcessAudioSample(float InSample)
		{
			// Update delay line.	
			DelayLine->AddSamples(&InSample, 1);

			// Copy delayed version to output
			const float DelayedSample = *(DelayLine->InspectSamples(1 + NumBufferOffsetSamples, NumDelayLineOffsetSamples));

			// Remove unneeded delay line.
			DelayLine->RemoveSamples(1);

			return DelayedSample;
		}

		SIGNALPROCESSING_API void ProcessAudio(const Audio::FAlignedFloatBuffer& InSamples, Audio::FAlignedFloatBuffer& OutSamples);
		// Process InSamples, placing delayed versions in OutSamples.
		SIGNALPROCESSING_API void ProcessAudio(TArrayView<const float> InSamples, TArrayView<float> OutSamples);

		// Retrieve a copy of the internal delay line.
		// InNum must be less than or equal to InMaxNumDelaySamples.
		SIGNALPROCESSING_API void PeekDelayLine(int32 InNum, Audio::FAlignedFloatBuffer& OutSamples);

	private:
		// Process a block of audio. InNum is always less than or equal to NumInternalBufferSamples.
		void ProcessAudioBlock(const float* InSamples, const int32 InNum, float* OutSamples);

		// Maximum supported number of delay samples
		int32 MaxNumDelaySamples;

		// Current number of delay samples
		int32 NumDelaySamples;

		// Offset of output delayed samples relative to DelayLine block buffer
		int32 NumDelayLineOffsetSamples;

		// Offset to handle delay values which are not multiple of required buffer alignment.
		int32 NumBufferOffsetSamples;

		// Number of samples in an internal buffer
		int32 NumInternalBufferSamples;

		// Buffer for holding delayed data.
		TUniquePtr<FAlignedBlockBuffer> DelayLine;
	};
}
