// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "DSP/ParamInterpolator.h"

namespace Audio
{
	class FLinearPitchShifter
	{
	public:
		// ctor
		SIGNALPROCESSING_API FLinearPitchShifter(int32 InNumChannels = 0);

		SIGNALPROCESSING_API void Reset(int32 InNumChannels, float InInitialPitchShiftSemitones = 0.0f, int32 InInterpLengthFrames = 100);

		// Sample rate converts the input audio buffer and pushes converted audio to the output circular buffer.
		// (Returns the number of output frames generated.)
		SIGNALPROCESSING_API int32 ProcessAudio(const TArrayView<float> InputBuffer, Audio::TCircularAudioBuffer<float>& OutputBuffer);

		SIGNALPROCESSING_API void UpdatePitchShift(float InNewPitchSemitones);

	private:
		// returns next fractional index increment
		float GetNextIndexDelta();

		TArray<float> PreviousFrame;
		FParam PitchShiftRatio;
		float CurrentIndex{ 0.0f };
		int32 NumChannels{ 0 };
		int32 InterpLengthFrames{ 100 };
		int32 InterpFramesRemaining{ 0 };

	}; // class FLinearPitchShifter
} // namespace Audio
