// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AlignedBuffer.h"
#include "VectorLinearResampler.h"

namespace Audio
{
	/**
	 *  Delay line supporting multiple interpolated tap reads before advancing
	 *  InBufferSizeSamples must be >= the size of the output buffer passed into Read
	 *  If the delay length can be less than the output buffer size, Advance should be called before Read
	 */
	class FInterpolatedMultiTapDelay
	{
	public:
		FInterpolatedMultiTapDelay() = default;
		
		SIGNALPROCESSING_API void Init(const int32 InDelayBufferSamples);

		SIGNALPROCESSING_API void Advance(TArrayView<const float> InBuffer);

		// Read and interpolate a variable number of samples back in the delay line
		// return a fixed-point representation of fraction of the last sample, which should be passed as StartSampleFraction on successive reads for that tap.
		SIGNALPROCESSING_API uint32 Read(const uint32 StartNumDelaySamples, const uint32 StartSampleFraction, const uint32 EndNumDelaySamples, TArrayView<float> OutBuffer);
		SIGNALPROCESSING_API void Reset();
		SIGNALPROCESSING_API bool IsInitialized() const;

	private:
		int32 WriteIndex = 0;

		FAlignedFloatBuffer DelayLine;
		FAlignedFloatBuffer WrapBuffer;
		FVectorLinearResampler Resampler;
	};	
}

