// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "DSP/Dsp.h"


namespace Audio
{
	class FInterpolatedLPF
	{
	public:
		// ctor
		SIGNALPROCESSING_API FInterpolatedLPF();

		inline float GetCutoffFrequency() const
		{
			return CutoffFrequency;
		}

		SIGNALPROCESSING_API void Init(float InSampleRate, int32 InNumChannels);

		/*
			StartFrequencyInterpolation():
				Intended use: call once at the beginning of callback, pass the callback length
				then call ProcessAudioFrame() exactly "InterpLength" number of times
				At the end of the callback, call StopFrequencyInterpolation()

				Failing to call StopFrequencyInterpolation() after more than InterpLength calls to ProcessAudioFrame
				will cause the filter to become unstable and blow up to +/- inf!
		*/
		SIGNALPROCESSING_API void StartFrequencyInterpolation(const float InTargetFrequency, const int32 InterpLength = 1);

		// interpolates coefficient and processes a sample
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* RESTRICT InputFrame, float* RESTRICT OutputFrame);

		// interpolates coefficient and processes a buffer
		SIGNALPROCESSING_API void ProcessAudioBuffer(const float* RESTRICT InputBuffer, float* RESTRICT OutputBuffer, const int32 NumSamples);

		// interpolates coefficient and processes a buffer
		SIGNALPROCESSING_API void ProcessBufferInPlace(float* InOutBuffer, int32 NumSamples);

		/*
			StopFrequencyInterpolation() needs to be called manually when the interpolation should be done.
			Snaps the coefficient to the target value.
			Setting the lerp delta to 0.0f avoids introducing a branch in ProcessAudioFrame()
		*/
		inline void StopFrequencyInterpolation()
		{
			B1Delta = 0.0f;
			B1Curr = B1Target;
		}

		// Resets the sample delay to 0
		SIGNALPROCESSING_API void Reset();

		// Clears memory without reevaluating coefficients.
		// This function is useful when there is a break between ProcessAudio calls.
		SIGNALPROCESSING_API void ClearMemory();

		// Apply the filter transfer function to each z-domain value in the given array (complex numbers given as interleaved floats). Passing in z-domain values on the complex unit circle will give the frequency response.
		SIGNALPROCESSING_API void ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const;

	private:
		float CutoffFrequency{ -1.0f };
		float B1Curr{ 0.0f }; // coefficient
		float B1Delta{ 0.0f }; // coefficient step size 
		float B1Target{ 0.0f };
		TArray<float> Z1; // multi-channel delay terms
		int32 CurrInterpLength{ 0 };
		int32 NumInterpSteps;
		float SampleRate{ 0 };
		int32 NumChannels{ 1 };
		bool bIsFirstFrequencyChange{ true };

	}; // class Interpolated Low pass filter



	class FInterpolatedHPF
	{
	public:
		// ctor
		SIGNALPROCESSING_API FInterpolatedHPF();

		inline float GetCutoffFrequency() const
		{
			return CutoffFrequency;
		}

		SIGNALPROCESSING_API void Init(float InSampleRate, int32 InNumChannels);

		/*
			StartFrequencyInterpolation():
				Intended use: call once at the beginning of callback, pass the callback length
				then call ProcessAudioFrame() exactly "InterpLength" number of times
				At the end of the callback, call StopFrequencyInterpolation()

				Failing to call StopFrequencyInterpolation() after more than InterpLength calls to ProcessAudioFrame
				will cause the filter to become unstable and blow up to +/- inf!
		*/
		SIGNALPROCESSING_API void StartFrequencyInterpolation(const float InTargetFrequency, const int32 InterpLength = 1);

		// interpolates coefficient and processes a sample
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* RESTRICT InputFrame, float* RESTRICT OutputFrame);

		// interpolates coefficient and processes a buffer
		SIGNALPROCESSING_API void ProcessAudioBuffer(const float* RESTRICT InputBuffer, float* RESTRICT OutputBuffer, const int32 NumSamples);

		// interpolates coefficients and processes a buffer in place
		SIGNALPROCESSING_API void ProcessBufferInPlace(float* InOutBuffer, int32 NumSamples);

		/*
			StopFrequencyInterpolation() needs to be called manually when the interpolation should be done.
			Snaps the coefficient to the target value.
			Setting the lerp delta to 0.0f avoids introducing a branch in ProcessAudioFrame()
		*/
		inline void StopFrequencyInterpolation()
		{
			A0Delta = 0.0f;
			A0Curr = A0Target;
		}

		// Resets the sample delay to 0
		SIGNALPROCESSING_API void Reset();

		// Clears memory without reevaluating coefficients.
		// This function is useful when there is a break between ProcessAudio calls.
		SIGNALPROCESSING_API void ClearMemory();

		// Apply the filter transfer function to each z-domain value in the given array (complex numbers given as interleaved floats). Passing in z-domain values on the complex unit circle will give the frequency response.
		SIGNALPROCESSING_API void ArrayCalculateResponseInPlace(TArrayView<float> InOutComplexValues) const;
		
	private:
		float CutoffFrequency{ -1.0f };
		float A0Curr{ 0.0f }; // coefficient
		float A0Delta{ 0.0f }; // coefficient step size
		float A0Target{ 0.0f };
		TArray<float> Z1; // multi-channel delay terms
		int32 CurrInterpLength{ 0 };
		int32 NumInterpSteps;
		float SampleRate{ 0 };
		float NyquistLimit{ 0.0f };
		int32 NumChannels{ 1 };
		bool bIsFirstFrequencyChange{ true };
	
	}; // class Interpolated High pass filter
} // namespace Audio
