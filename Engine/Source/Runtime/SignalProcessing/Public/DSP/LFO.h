// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DSP/Osc.h"
#include "DSP/ModulationMatrix.h"


namespace Audio
{
	namespace ELFO
	{
		enum Type
		{
			Sine = 0,
			UpSaw,
			DownSaw,
			Square,
			Triangle,
			Exponential,
			RandomSampleHold,

			NumLFOTypes
		};
	}

	namespace ELFOMode
	{
		enum Type
		{
			// Constantly oscillates
			Sync = 0,

			// Performs the LFO only once, then stops
			OneShot,

			// Doesn't restart the phase of the LFO on subsequent calls to Start
			Free,

			NumLFOModes
		};
	}

	// Low frequency oscillator
	class FLFO : public IOscBase
	{
	public:
		SIGNALPROCESSING_API FLFO();
		SIGNALPROCESSING_API virtual ~FLFO();

		//~ Begin FOscBase
		SIGNALPROCESSING_API virtual void Init(const float InSampleRate, const int32 InVoiceId = 0, FModulationMatrix* InMatrix = nullptr, const int32 ModMatrixStage = 1) override;
		SIGNALPROCESSING_API virtual void Start() override;
		SIGNALPROCESSING_API virtual void Stop() override;
		SIGNALPROCESSING_API virtual void Reset() override;
		SIGNALPROCESSING_API virtual float Generate(float* QuadPhaseOutput = nullptr) override;
		//~ End FOscBase

		// Set whether or not LFO outputs in bipolar domain or unipolar (false) domain. Defaults to bipolar.
		SIGNALPROCESSING_API void SetBipolar(const bool bInBipolar);

		// Set the waveform type of LFO generator
		SIGNALPROCESSING_API void SetType(const ELFO::Type InLFOType);

		// Returns current waveform type of LFO generator
		SIGNALPROCESSING_API ELFO::Type GetType() const;

		// Sets mode of LFO (Looping or one-shot.  Free not supported)
		SIGNALPROCESSING_API void SetMode(const ELFOMode::Type InLFOMode);

		// Returns current mode of LFO
		SIGNALPROCESSING_API ELFOMode::Type GetMode() const;

		// Sets a phase offset for the LFO.  Clamped to positive values only
		SIGNALPROCESSING_API void SetPhaseOffset(const float InOffset);

		// Sets the exponential factor of LFO if type is "exponential". Clamped to positive, non-zero values.
		SIGNALPROCESSING_API void SetExponentialFactor(const float InExpFactor);

		// Returns mod source's normal phase patch source
		SIGNALPROCESSING_API FPatchSource GetModSourceNormalPhase() const;

		// Returns mod source's quad phase patch source
		SIGNALPROCESSING_API FPatchSource GetModSourceQuadPhase() const;

	protected:
		SIGNALPROCESSING_API float ComputeLFO(float InputPhase, float* OutQuad = nullptr);

		// Returns initial phase, which differs between generators
		// ensuring certain LFO shapes start on rising edge from zero-
		// crossing by default (assuming user's provided phase offset is 0).
		SIGNALPROCESSING_API float GetInitPhase() const;

		// Resets generator to initial phase
		SIGNALPROCESSING_API virtual void ResetPhase() override;

		ELFO::Type LFOType;
		ELFOMode::Type LFOMode;
		float ExponentialFactor;
		uint32 RSHCounter;
		float RSHValue;
		float ModScale;
		float ModAdd;
		float LastOutput;
		float LoopCount;
		float QuadLastOutput;
		float PhaseOffset;

		bool bBipolar;

		FPatchSource ModNormalPhase;
		FPatchSource ModQuadPhase;
	};
} // namespace Audio
