// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/CircularBuffer.h"
#include "DSP/LoudnessAnalyzer.h"
#include "DSP/SlidingWindow.h"
#include "HAL/Platform.h"

#define UE_API SIGNALPROCESSING_API

namespace Audio
{
	template <typename InSampleType> class TSlidingBuffer;

	/** Settings for FLKFSAnalyzer */
	struct FLKFSAnalyzerSettings
	{
		// These values are taken from the suggested standards for LKFS/LUFS defined in ITU-R BS.1770
		static constexpr float StandardShortTermLoudnessDuration = 3.f;			// 3 seconds
		static constexpr float StandardIntegratedLoudnessAnalysisPeriod = 1.f;	// 1 second
		
		// Streaming audio does not have a natural end point. Integrated loudness in this 
		// implementation is taken over a long duration (as opposed to accumulating indefinitely). 
		static constexpr float DefaultIntegratedLoudnessDuration = 60.f; // 1 minute

		FLoudnessAnalyzerSettings LoudnessAnalyzerSettings {
			.FFTSize = 4096,
			.WindowType = EWindowType::None,
			.LoudnessCurveType = ELoudnessCurveType::K,
			.MinAnalysisFrequency = 20.0f,
			.MaxAnalysisFrequency = 24000.0f,
			.ScalingMethod = ELoudnessAnalyzerScalingMethod::Corrected
		};

		/** Number of seconds between loudness measurements */
		float AnalysisPeriod = 0.1f;

		/** Number of seconds of audio analyzed for each loudness measurements */
		float AnalysisWindowDuration = 0.4f;

		/** Duration of audio analyzed for short term loudness. */
		float ShortTermLoudnessDuration = StandardShortTermLoudnessDuration;

		/** Number of seconds between long term loudness updates. */
		float IntegratedLoudnessAnalysisPeriod = StandardIntegratedLoudnessAnalysisPeriod;

		/** Duration of audio analyzed for long term loudness. */
		float IntegratedLoudnessDuration = DefaultIntegratedLoudnessDuration;

		/** Should the Loudness Range (LRA) statistic be calculated for each channel? */
		bool bCalculatePerChannelLoudnessRange = false;

		/** Should the overall Loudness Range (LRA) statistic be calculated? */
		bool bCalculateOverallLoudnessRange = false;
	};

	/** LKFS Result for a single channel (or overall channels result if Channel is set to -1) */
	struct FLKFSResult
	{
		/** The audio channel index which produced this result. If -1, then the data was produced by combining all other audio channels */
		int32 Channel = 0;

		/** Time in seconds of the source audio which corresponds to the loudness measurements. */
		float Timestamp = 0.0f;

		/** The instantaneous, perceptually weighted energy relative to full scale. */
		float Energy = 0.f;

		/** The instantaneous, perceptually weighted loudness in dB. */
		float Loudness = 0.0f;

		/** The average loudness over a short term window of time in dB. */
		float ShortTermLoudness = 0.0f;

		/** The average loudness over a long term window of time in dB. */
		float IntegratedLoudness = 0.0f;

		/** The average gated loudness over a long term window of time in dB. */
		float GatedLoudness = 0.0f;

		/** Loudness Range (LRA), as defined in EBU Tech 3342. */
		TOptional<FFloatInterval> LoudnessRange;
	};

	/** Analyzer results contains all channels LKFS results */
	class FLKFSAnalyzerResults
	{
	public:
		/** Construct analyzer results */
		FLKFSAnalyzerResults() = default;

		/** The number of individual channels available in the result. */
		UE_API int32 GetNumChannels() const;

		/** Add to the result. */
		UE_API void Add(FLKFSResult InResults);

		/** Returns const reference to a FLKFSResult array for individual channel. */
		UE_API const TArray<FLKFSResult>& GetChannelLoudnessResults(int32 ChannelIdx) const;

		/** Returns const reference to a FLKFSResult array associated with overall loudness. */
		UE_API const TArray<FLKFSResult>& GetLoudnessResults() const;

		/** Denotes the overall loudness channel index as opposed individual channel indices. */
		inline static const int32 ChannelIndexOverall = -1;

	private:
		TSortedMap<int32, TArray<FLKFSResult>> ChannelResults;
	};


	/** FLKFSAnalyzer
	 *
	 * FLKFSAnalyzer computes Loudness K-Weighted FS (LKFS) metrics for interleaved audio channels.
	 * Uses a sliding window approach and an internal perceptual energy analyzer.
	 * Produces instantaneous energy and loudness as well as short term, integrated, gated LKFS
	 * and optionally the loudness range (based on EBU-3342).
	 */
	class FLKFSAnalyzer
	{
	public:
		/** Construct analyzer */
		UE_API FLKFSAnalyzer(int32 InNumChannels, float InSampleRate, const FLKFSAnalyzerSettings& InSettings);

		/** Return const reference to settings used inside this analyzer. */
		UE_API const FLKFSAnalyzerSettings& GetSettings() const;

		/** Analyze audio to obtain LKFS values (see FLKFSAnalyzerResults and FLKFSResult for more details) */
		UE_API void Analyze(TArrayView<const float> InAudio, FLKFSAnalyzerResults& OutLKFSResults);

	private:
		/**
		 * Holds aggregated loudness statistics calculated over a history of energy measurements.
		 */
		struct FAggregateLoudnessStats
		{
			static constexpr float InvalidLoudness = TNumericLimits<float>::Lowest();

			float ShortTermLoudness  = InvalidLoudness;
			float IntegratedLoudness = InvalidLoudness;
			float GatedLoudness      = InvalidLoudness;

			TOptional<FFloatInterval> LoudnessRange;
		};

		/**
		 * Holds state for Loudness Range calculation (for a single channel).
		 */
		class FLoudnessRange
		{
		public:
			explicit FLoudnessRange(uint32 InMaxShortTermHistorySize)
				: ShortTermEnergyHistory(InMaxShortTermHistorySize)
				, MaxShortTermHistorySize(InMaxShortTermHistorySize)
			{
			}

			void AddShortTermEnergyMeasurement(float InShortTermEnergy);

			TOptional<FFloatInterval> CalculateLoudnessRange() const;

		private:
			TCircularBuffer<float> ShortTermEnergyHistory;

			uint32 MaxShortTermHistorySize = 0;
			uint32 ShortTermHistorySize = 0;
			uint32 ShortTermWriteIndex = 0;
		};

		void AddCurrentLoudnessDataToResult(FLKFSAnalyzerResults& OutResult);
		void AnalyzeSlidingWindow(TAutoSlidingWindow<float>& InSlidingWindow, FLKFSAnalyzerResults& OutResult);
		void AnalyzeWindow(TArrayView<const float> InWindow);

		void AddCurrentLoudnessDataToHistory();

		void UpdateAggregateLoudnessStats();
		void UpdateAggregateLoudnessStats(const TCircularBuffer<float>& InHistory, int32 InOffset, int32 InStride, bool bInUpdateIntegratedLoudness, FAggregateLoudnessStats& OutStats, FLoudnessRange* LoudnessRange = nullptr) const;

		FLKFSAnalyzerSettings Settings;

		int32 NumChannels = 0;
		float SampleRate = 0.0f;

		int32 NumAnalyzedBuffers = 0;
		int32 NumEnergyHistory = 0;
		int32 NumSlidingWindowHopFrames = 0;

		// Number of windows between each loudness analysis
		int32 NumAnalysisHopWindows = 1;

		// Number of windows inspected for each loudness analysis
		int32 NumAnalysisWindows = 1;

		// Number of values in the "EnergyHistory" to analyze for short term loudness
		int32 NumShortTermHistoryResults = 1;

		// Number of values in the "EnergyHistory" to analyze for integrated and gated loudness
		int32 NumIntegratedHistoryResults = 1;

		// Number of values in the "EnergyHistory" between integrated and gated loudness anlysis
		int32 NumIntegratedHopHistoryResults = 1;

		TArray<float> ChannelPerceptualEnergy;
		TArray<float> OverallPerceptualEnergy;
		TArray<float> InternalWindow;

		// Per channel stats for short-term, integrated and gated loudness, and optional loudness range. 
		TSortedMap<int32, FAggregateLoudnessStats> AggregateLoudnessStats;
		
		// Per channel state for calculating Loudness Range.
		TSortedMap<int32, FLoudnessRange> LoudnessRangeStatePerChannel;

		// Channel interleaved history buffer of energy measurements.
		// These are taken once per an loudness analysis, not necessarily once per a audio window. 
		TCircularBuffer<float> ChannelEnergyHistory;
		TCircularBuffer<float> OverallEnergyHistory;

		TSlidingBuffer<float> InternalBuffer;

		FMultichannelLoudnessAnalyzer LoudnessAnalyzer;
	};
}

#undef UE_API
