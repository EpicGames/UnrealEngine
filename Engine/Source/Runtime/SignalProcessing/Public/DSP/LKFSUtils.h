// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"

#define UE_API SIGNALPROCESSING_API

namespace Audio::LKFSUtils
{
	// There may be multiple calls to the LoudnessAnalyzer to produce a single result. 
	// The sliding window hop is tuned here so that it best matches the desired AnalysisPeriod 
	// while maintaining between a 25% to 75% window overlap.
	UE_INTERNAL UE_API void TuneSlidingWindowHopSize(int32 InNumAnalysisHopFrames, int32 InNumSlidingWindowFrames, int32& InNumSlidingWindowHopFrames, int32& InNumAnalysisHopWindows);
} // namespace Audio::LKFSUtils

#undef UE_API
