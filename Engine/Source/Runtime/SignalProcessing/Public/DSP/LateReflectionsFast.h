// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/IntegerDelay.h"
#include "DSP/LongDelayAPF.h"
#include "DSP/BufferOnePoleLPF.h"
#include "DSP/DynamicDelayAPF.h"

namespace Audio
{
	// Structure to hold the various delay line tap outputs produced from a plate.
	struct FLateReflectionsPlateOutputs
	{
		// Number of tap locations in plate reverb
		static const int32 NumTaps = 7;

		// Tap sample buffers.
		FAlignedFloatBuffer Taps[NumTaps];

		// Output sample buffer.
		FAlignedFloatBuffer Output;

		// Resizes and zero all buffers in structure.
		SIGNALPROCESSING_API void ResizeAndZero(int32 InNumSamples);
	};

	// Delay line settings for reverb plate delay
	// Tap locations are determined by analyzing these delay values. 
	struct FLateReflectionsPlateDelays
	{
		/*
			Setting the delay values affects the tap locations. The audio flowchart below shows the
			relationship between delay values and tap locations. 

			+---------------+
			|    Input      |
			+------+--------+
				   |
			+------v--------+
			| Modulated APF |
			+------+--------+
				   |
			+------+-------+
			|    DelayA    |
			+------+-------+  +---------+
				   |--------->+  Tap 0  |
			+------v-------+  +---------+
			|    DelayB    |
			+------+-------+  +---------+
				   |---------->  Tap 1  |
			+------v-------+  +---------+
			|    DelayC    |
			+------+-------+  +---------+
				   |---------->  Tap 2  |
			+------v-------+  +---------+
			|    DelayD    |
			+------+-------+
				   |
			+------v--------+
			|     LPF       |
			+------+--------+
				   |
			+------v-------+  +---------+  +---------+  +---------+  +--------+
			|     APF      +--> DelayE  +--> |Tap 3  +--> DelayF  +-->  Tap 4 |
			+------+-------+  +---------+  +---------+  +---------+  +--------+
				   |
			+------v-------+
			|    DelayG    |
			+------+-------+  +---------+
				   |---------->  Tap 5  |
			+------v-------+  +---------+
			|    DelayH    |
			+------+-------+  +---------+
				   |--------->+  Tap 6  |
			+------v-------+  +---------+
			|    DelayI    |
			+------+-------+
				   |
			+------v-------+
			|    Output    |
			+--------------+
		 */

		// Base delay samples for modulated delay
		int32 NumSamplesModulatedBase;
		// Max delta samples for modulated delay
		int32 NumSamplesModulatedDelta;
		// Tap 0 exists at the output of DelayA
		int32 NumSamplesDelayA;
		// Tap 1 exists at the output of DelayB
		int32 NumSamplesDelayB;
		// Tap 2 exists at the output of DelayC
		int32 NumSamplesDelayC;
		// Delay line length
		int32 NumSamplesDelayD;
		// All pass filter length
		int32 NumSamplesAPF;
		// Tap 3 exists at the output of DelayE
		int32 NumSamplesDelayE;
		// Tap 4 exists at the output of DelayF
		int32 NumSamplesDelayF;
		// Tap 5 exists at the output of DelayG
		int32 NumSamplesDelayG;
		// Tap 6 exists at the output of DelayH
		int32 NumSamplesDelayH;
		// Output tap exists at the output of DelayI
		int32 NumSamplesDelayI;

		// Default delay settings for left channel.
		static SIGNALPROCESSING_API FLateReflectionsPlateDelays DefaultLeftDelays(float InSampleRate);

		// Default delay settings for right channel.
		static SIGNALPROCESSING_API FLateReflectionsPlateDelays DefaultRightDelays(float InSampleRate);
	};


	// Single plate channel for plate reverberation.
	class FLateReflectionsPlate {
	public:
		// InMaxNumInternalBufferSamples sets the mmaximum number of samples in an internal buffer.
		SIGNALPROCESSING_API FLateReflectionsPlate(
			float InSampleRate,
			int32 InMaxNumInternalBufferSamples,
			const FLateReflectionsPlateDelays& InDelays);

		SIGNALPROCESSING_API ~FLateReflectionsPlate();

		// Processing input samples. All input buffers must be of equal length.
		// InSamples contains previously unseen audio samples.
		// InFeedbackSamples contains the delay line output from the other plate.
		// InDelayModulations contains the fractional delay values per a sample to modulate the first all pass filters delay line.
		// OutPlateSamples is filled with audio from the various tap points and delay line output.
		SIGNALPROCESSING_API void ProcessAudioFrames(
			const FAlignedFloatBuffer& InSamples,
			const FAlignedFloatBuffer& InFeedbackSamples,
			const FAlignedFloatBuffer& InDelayModulations,
			FLateReflectionsPlateOutputs& OutPlateSamples);

		// Flush internal audio to silence.
		SIGNALPROCESSING_API void FlushAudio();

		SIGNALPROCESSING_API void SetDensity(float InDensity);
		SIGNALPROCESSING_API void SetDampening(float InDampening);
		SIGNALPROCESSING_API void SetDecay(float InDecay);

		// The number of samples in an internal buffer. This describes the limit of how many values
		// are accessible via PeekDelayLineOutput(...)
		SIGNALPROCESSING_API int32 GetNumInternalBufferSamples() const;

		// Retrieve the internal delay line data. The number of samples retrieved is limited to the
		// number of samples in the internal buffer. That amount can be determined by calling
		// "GetNumInternalBufferSamples()".
		SIGNALPROCESSING_API void PeekDelayLine(int32 InNum, FAlignedFloatBuffer& OutSamples);

	protected:
		// Current parameter settings of reverb
		float SampleRate;
		int32 NumInternalBufferSamples;
		float Dampening;
		float Decay;
		float Density;

		// Sample delay values
		FLateReflectionsPlateDelays PlateDelays;

		// Sample rate used for hard-coded delay line values
		static const int32 PresetSampleRate = 29761;

		TUniquePtr<FDynamicDelayAPF> ModulatedAPF;
		TUniquePtr<FIntegerDelay> DelayA;
		TUniquePtr<FIntegerDelay> DelayB;
		TUniquePtr<FIntegerDelay> DelayC;
		TUniquePtr<FIntegerDelay> DelayD;
		TUniquePtr<FBufferOnePoleLPF> LPF;
		TUniquePtr<FLongDelayAPF> APF;
		TUniquePtr<FIntegerDelay> DelayE;
		TUniquePtr<FIntegerDelay> DelayF;
		TUniquePtr<FIntegerDelay> DelayG;
		TUniquePtr<FIntegerDelay> DelayH;
		TUniquePtr<FIntegerDelay> DelayI;

		Audio::FAlignedFloatBuffer WorkBufferA;
		Audio::FAlignedFloatBuffer WorkBufferB;
		Audio::FAlignedFloatBuffer WorkBufferC;
	};

	// Settings for controlling the FLateReflections
	struct FLateReflectionsFastSettings
	{
		// Milliseconds for the predelay
		float LateDelayMsec;

		// Initial attenuation of audio after it leaves the predelay
		float LateGainDB;

		// Frequency bandwidth of audio going into input diffusers. 0.999 is full bandwidth
		float Bandwidth;

		// Amount of input diffusion (larger value results in more diffusion)
		float Diffusion;

		// The amount of high-frequency dampening in plate feedback paths
		float Dampening;

		// The amount of decay in the feedback path. Lower value is larger reverb time.
		float Decay;

		// The amount of diffusion in decay path. Larger values is a more dense reverb.
		float Density;

		SIGNALPROCESSING_API FLateReflectionsFastSettings();

		SIGNALPROCESSING_API bool operator==(const FLateReflectionsFastSettings& Other) const;

		SIGNALPROCESSING_API bool operator!=(const FLateReflectionsFastSettings& Other) const;
	};


	// FLateReflections generates the long tail reverb of an input audio signal using a relatively
	// fast algorithm using all pass filter delay lines. It supports 1 or 2 channel input and always 
	// produces 2 channel output. 
	class FLateReflectionsFast {
	public:

		// Limits on settings.
		static SIGNALPROCESSING_API const float MaxLateDelayMsec;
		static SIGNALPROCESSING_API const float MinLateDelayMsec;
		static SIGNALPROCESSING_API const float MaxLateGainDB;
		static SIGNALPROCESSING_API const float MaxBandwidth;
		static SIGNALPROCESSING_API const float MinBandwidth;
		static SIGNALPROCESSING_API const float MaxDampening;
		static SIGNALPROCESSING_API const float MinDampening;
		static SIGNALPROCESSING_API const float MaxDiffusion;
		static SIGNALPROCESSING_API const float MinDiffusion;
		static SIGNALPROCESSING_API const float MaxDecay;
		static SIGNALPROCESSING_API const float MinDecay;
		static SIGNALPROCESSING_API const float MaxDensity;
		static SIGNALPROCESSING_API const float MinDensity;

		static SIGNALPROCESSING_API const FLateReflectionsFastSettings DefaultSettings;

		// InMaxNumInternalBufferSamples sets the maximum possible number of samples in an internal
		// buffer. This only affectsinternal chunking and does not affect the number of samples that
		// can be sent to ProcessAudio(...), 
		SIGNALPROCESSING_API FLateReflectionsFast(float InSampleRate, int InMaxNumInternalBufferSamples, const FLateReflectionsFastSettings& InSettings=DefaultSettings);
		
		SIGNALPROCESSING_API ~FLateReflectionsFast();

		// Copies the reverb settings internally, clamps the internal copy and applies the clamped settings.
		SIGNALPROCESSING_API void SetSettings(const FLateReflectionsFastSettings& InSettings);

		// Alters settings to ensure they are within acceptable ranges. 
		static SIGNALPROCESSING_API void ClampSettings(FLateReflectionsFastSettings& InOutSettings);

		// Create reverberation in OutSamples based upon InSamples. 
		// OutSamples is a 2 channel audio buffer no matter the value of InNumChannels.
		// OutSamples is mixed with InSamples based upon gain.
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, const int32 InNumChannels, FAlignedFloatBuffer& OutLeftSamples, FAlignedFloatBuffer& OutRightSamples);

		// Flush internal audio to silence.
		SIGNALPROCESSING_API void FlushAudio();

	private:
		void ProcessAudioBuffer(const float* InSampleData, const int32 InNumFrames, const int32 InNumChannels, float* OutLeftSampleData, float* OutRightSampleData);

		void ApplySettings();

		void GeneraterPlateModulations(const int32 InNum, FAlignedFloatBuffer& OutLeftDelays, FAlignedFloatBuffer& OutRightDelays);

		float SampleRate;
		float Gain;
		float ModulationPhase;
		float ModulationQuadPhase;
		float ModulationPhaseIncrement;
		int32 NumInternalBufferSamples;

		// Current parameter settings of reverb
		FLateReflectionsFastSettings Settings;

		// Sample rate used for hard-coded delay line values
		static const int32 PresetSampleRate = 29761;

		// A simple pre-delay line to emulate large delays for late-reflections
		TUniquePtr<FIntegerDelay> PreDelay;

		// Input diffusion
		TUniquePtr<FBufferOnePoleLPF> InputLPF;
		TUniquePtr<FLongDelayAPF> APF1;
		TUniquePtr<FLongDelayAPF> APF2;
		TUniquePtr<FLongDelayAPF> APF3;
		TUniquePtr<FLongDelayAPF> APF4;

		// The plate delay data
		FLateReflectionsPlateDelays LeftPlateDelays;
		FLateReflectionsPlateDelays RightPlateDelays;

		// The plates.
		TUniquePtr<FLateReflectionsPlate> LeftPlate;
		TUniquePtr<FLateReflectionsPlate> RightPlate;

		// The plate outputs
		FLateReflectionsPlateOutputs LeftPlateOutputs;
		FLateReflectionsPlateOutputs RightPlateOutputs;

		// Buffer for delay modulation of left and right plates.
		FAlignedFloatBuffer LeftDelayModSamples;
		FAlignedFloatBuffer RightDelayModSamples;

		FAlignedFloatBuffer WorkBufferA;
		FAlignedFloatBuffer WorkBufferB;
		FAlignedFloatBuffer WorkBufferC;

	};

}
