// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/AlignedBlockBuffer.h"

namespace Audio
{
	// Fractional delay using linear interpolation.
	class FLinearInterpFractionalDelay
	{
	public:
		
		// InMaxDelay sets the maximum allowable delay that this object can support. The minimum delay is 0.0
		// InNumInternalBufferSamples sets the maximum block processing size.
		SIGNALPROCESSING_API FLinearInterpFractionalDelay(int32 InMaxDelay, int32 InMaxNumInternalBufferSamples);
		SIGNALPROCESSING_API ~FLinearInterpFractionalDelay();

		// Apply delay to InSamples. Fill OutSamples with data from the delay line at a delay of InDelay.
		// InDelay must be equal length to InSamples. 
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, const FAlignedFloatBuffer& InDelays, FAlignedFloatBuffer& OutSamples);

		// Set all values in internal delay line to zero. 
		SIGNALPROCESSING_API void Reset();

	private:
		void ProcessAudioBlock(const float* InSamples, const float* InDelays, const int32 InNum, float* OutSamples);

		int32 MaxDelay;
		int32 NumInternalBufferSamples;

		TUniquePtr<FAlignedBlockBuffer> DelayLine;
		AlignedInt32Buffer IntegerDelayOffsets;
		int* UpperDelayPos;
		int* LowerDelayPos;

	};
}
