// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Filter.h"
#include "DSP/VariablePoleFilter.h"
#include "DSP/BufferVectorOperations.h"

#define  MAX_BUFFER_SIZE 8192 // 8 channels * 1024 samples

namespace Audio
{
	struct FLinkwitzRileyBandFilter
	{
		TArray<FVariablePoleFilter> Filters;

		FVariablePoleFilter& operator[](int32 InIndex) { return Filters[InIndex]; }
	};

	struct FMultibandBuffer
	{
		Audio::FAlignedFloatBuffer Buffer;

		int32 NumBands = 0;
		int32 NumSamples = 0;

		FMultibandBuffer() = default;

		FMultibandBuffer(int32 InBands, int32 InSamples)
			: NumBands(InBands)
			, NumSamples(InSamples)
		{
			Buffer.SetNumZeroed(InBands * InSamples);
		}

		void Init(int32 InBands, int32 InSamples)
		{
			NumBands = InBands;
			NumSamples = InSamples;

			Buffer.SetNumZeroed(InBands * InSamples);
		}

		void SetBands(int32 InBands)
		{
			NumBands = InBands;
			Reset();
		}

		void SetSamples(int32 InSamples)
		{
			NumSamples = InSamples;
			Reset();
		}

		// Zero buffers
		void Reset()
		{
			Buffer.Reset(NumSamples * NumBands);
			Buffer.AddZeroed(NumSamples * NumBands);
		}

		float* operator[](int32 BandIndex) { return &Buffer[BandIndex * NumSamples]; }
	};

	/*
	 * Helper for Multi-Band processing to generate Linwitz-Riley filtered outputs from input
	 * https://en.wikipedia.org/wiki/Linkwitz%E2%80%93Riley_filter
	 */
	class FLinkwitzRileyBandSplitter
	{
	public:
		struct FCrossoverBandwidthPair
		{
			float Frequency = 0.f;
			float Bandwidth = 0.f;
		};

		FLinkwitzRileyBandSplitter() {};

		// initalize filters
		SIGNALPROCESSING_API void Init(const int32 InChannels,
				  const float InSampleRate,
				  const EFilterOrder FilterOrder, 
				  const bool bInPhaseCompensate,
				  const TArray<float>& InCrossovers); // Always InBands - 1 Crossovers

		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InBuffer, FMultibandBuffer& OutBuffer);
		SIGNALPROCESSING_API void ProcessAudioBuffer(const float* InBuffer, FMultibandBuffer& OutBuffer, const int32 NumFrames);

		SIGNALPROCESSING_API void SetCrossovers(const TArray<float>& InCrossoverFrequencies);

	private:
		EFilterOrder FilterOrder = EFilterOrder::FourPole;

		int32 NumBands = 1;
		int32 NumChannels = 2;
		float SampleRate = 48000.f;

		TArray<float> SharedBuffer;
		TArray<float> BandWorkBuffer;
		FAlignedFloatBuffer SharedAlignedBuffer;
		FAlignedFloatBuffer BandAlignedBuffer;

		TArray<FLinkwitzRileyBandFilter> BandFilters;
		TArray<FCrossoverBandwidthPair> Crossovers;

		SIGNALPROCESSING_API void CopyToBuffer(float* Destination, const float* Origin, const int32 NumSamples);
		SIGNALPROCESSING_API void InvertBuffer(float* Buffer, const int32 NumSamples);
		SIGNALPROCESSING_API float GetQ(EFilterOrder InFilterOrder);
	};
}
