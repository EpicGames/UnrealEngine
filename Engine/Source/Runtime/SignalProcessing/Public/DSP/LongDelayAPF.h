// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/AlignedBlockBuffer.h"

namespace Audio
{
	// All Pass Filter with a long delay. This filter is specifically designed for 
	// reverb applications where filter delay lines are long.
	class FLongDelayAPF
	{
	public:
		// InG is the filter coefficient used in the long delay all pass filter.
		// InNumDelaySamples is the delay line length in samples.
		// InMaxNumInternalBufferSamples is the maximum internal block size used internally. 
		SIGNALPROCESSING_API FLongDelayAPF(float InG, int32 InNumDelaySamples, int32 InMaxNumInternalBufferSamples);

		// Destructor
		SIGNALPROCESSING_API ~FLongDelayAPF();

		// Set the APF feedback/feedforward gain coefficient
		void SetG(float InG) { G = InG; }

		// Process InSamples and place filtered data in OutSamples
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples);

		// Process Samples in place
		SIGNALPROCESSING_API void ProcessAudio(FAlignedFloatBuffer& Samples);

		// Process InSamples and place filtered data in OutSamples and delay line samples in OutDelaySamples.
		// The OutDelaySamples correspond to "w[n] = InSamples[n] + InG * w[n - InDelay]"
		SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, FAlignedFloatBuffer& OutSamples, FAlignedFloatBuffer& OutDelaySamples);

		// Sets delay line values to zero.
		SIGNALPROCESSING_API void Reset();

	private:
		// Performs all pass filtering on an audio block of BlockSize samples are less. 
		// InNum must be less than or equal to BlockSize.
		SIGNALPROCESSING_API void ProcessAudioBlock(const float* InSamples, const float* InDelaySamples, const int32 InNum, float* OutSamples, float* OutDelaySamples);

		SIGNALPROCESSING_API int32 GetNumInternalBufferSamples() const;

		// Feedback/Feedforward gain coefficient
		float G;

		// Delay in samples
		int32 NumDelaySamples;

		// Buffer size for internal block processing.
		int32 NumInternalBufferSamples;

		// Buffer for block operations.
		FAlignedFloatBuffer WorkBuffer;

		// Filter delay line memory.
		TUniquePtr<FAlignedBlockBuffer> DelayLine;
	};

}
