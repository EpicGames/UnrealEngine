// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioFFT.h"

#define UE_API SIGNALPROCESSING_API

namespace Audio { class IFFTAlgorithm; }

namespace Audio
{
    /** Equal loudness curves from: ANSI Standards S1.4-1983 and S1.42-2001 */
	enum class ELoudnessCurveType : uint8
	{
		A,
		B,
		C,
		D,
		K,
		None
	};

	/** Generate equal loudness weight for magnitude spectrum using curve A */
    UE_API float GetEqualLoudnessAWeigthForFrequency(const float Freq);

	/** Generate equal loudness weight for magnitude spectrum using curve B */
    UE_API float GetEqualLoudnessBWeigthForFrequency(const float Freq);

	/** Generate equal loudness weight for magnitude spectrum using curve C */
    UE_API float GetEqualLoudnessCWeigthForFrequency(const float Freq);

	/** Generate equal loudness weight for magnitude spectrum using curve D */
    UE_API float GetEqualLoudnessDWeigthForFrequency(const float Freq);

	/** Generate equal loudness weight for magnitude spectrum using curve None */
    UE_API float GetEqualLoudnessNoneWeigthForFrequency(const float Freq);

	enum class ELoudnessAnalyzerScalingMethod : uint8
	{
		// Normalize by FFTSize and having perceptual curves be unity gain at 1khz. 
		// 
		// Note: This method does not correctly account for different window functions
		// and also reduces the energy by half. It remains here for backwards
		// compatibility.
		BackCompatible,

		// Normalize by Window Function. Use this when performing loudness analysis
		// which must match the LKFS standardized measurements. 
		Corrected
	};

	/** Settings for FLoudnessAnalyzer */
    struct FLoudnessAnalyzerSettings
    {
		/** Size of FFT describe in number of samples */
        int32 FFTSize = 8192;

		/** Type of window to be applied to input audio */
        EWindowType WindowType = EWindowType::Blackman;

		/** Type of equal loudness curve to weight power spectrum */
        ELoudnessCurveType LoudnessCurveType = ELoudnessCurveType::D;

		/** Minimum frequency for analyzing audio */
        float MinAnalysisFrequency = 20.f;

		/** Maximum frequency for analyzing audio */
        float MaxAnalysisFrequency = 10000.f;

		/** Size of the audio analysis window in samples.
		 *
		 * If this is less than 1, it will become equal to the FFTSize.
		 * Otherwise, this must be less than or equal to the FFT Size.
		 */
		int32 WindowSize = -1;

		/** Method to use for scaling and normalizing data. Corrected is preferred
		 * in terms of correctness. BackCompatible is available for backwards compatibility.
		 */
		ELoudnessAnalyzerScalingMethod ScalingMethod = ELoudnessAnalyzerScalingMethod::BackCompatible;
    };

	/** FLoudnessAnalyzer
	 *
	 * FLoudnessAnalyzer will calculate perceptual energy or loudness one window at a time. 
	 * This analyzer only handles mono audio. See FMultichannelLoudnessAnalyzer for multichannel
	 * support.
	 */
    class FLoudnessAnalyzer
    {
        public:
			/** Construct analyzer */
            UE_API FLoudnessAnalyzer(float InSampleRate, const FLoudnessAnalyzerSettings& InSettings);

			/**
			 * Calculate the perceptual energy for the input samples.  The number of 
			 * input samples must be equal to the FFTSize supplied in FLoudnessAnalyzerSettings.
			 */
            UE_API float CalculatePerceptualEnergy(TArrayView<const float> InSampleView);

			/**
			 * Calculate the loudness for the input samples.  The number of  input samples must 
			 * be equal to the FFTSize supplied in FLoudnessAnalyzerSettings.
			 */
            UE_API float CalculateLoudness(TArrayView<const float> InSampleView);

			/**
			 * Convert from perceptual energy to loudness [Loudness = 10 * log_10(Energy)]
			 */
			static UE_API float ConvertPerceptualEnergyToLoudness(float InPerceptualEnergy);
			
			/**
			 * Convert from perceptual energy to loudness [LKFS = -0.691 + 10 * log_10(Energy)]
			 */
			static UE_API float ConvertPerceptualEnergyToLKFS(float InPerceptualEnergy);

			/**
			 * Return const reference to settings used inside this analyzer.
			 */
            UE_API const FLoudnessAnalyzerSettings& GetSettings() const;
			
        private:
			float CalculateEnergyScaling() const;
			float GetEqualLoudnessWeightForFrequency(float InHz) const;
			float CalculateEnergyScalingBackCompatible() const;
			float CalculateEnergyScalingCorrected() const;
			float CalculateWindowEnergyScaling() const;

            FLoudnessAnalyzerSettings Settings;

			/** Scale factor used to normalize between differing window sizes and fft sizes */
            float EnergyScale = 1.f;

            int32 MinFreqIndex = 0;
            int32 MaxFreqIndex = 0;

			/** Number of FFTFrequency bins worth looking at. */
			int32 FFTFreqSize = 0;
			int32 ActualFFTSize = 0;

			TUniquePtr<IFFTAlgorithm> FFT;

			FAlignedFloatBuffer Window;
            FAlignedFloatBuffer WindowedSamples;
            FAlignedFloatBuffer ComplexSpectrum;
            FAlignedFloatBuffer RealSpectrum;

            FAlignedFloatBuffer CurveWeights;
    };


	/** FMultichannelLoudnessAnalyzer
	 *
	 * FMultichannelLoudnessAnalyzer will calculate perceptual energy or loudness one window 
	 * at a time. This module can handle one or more channels at a time. It expects 
	 * interleaved audio samples.
	 */
    class FMultichannelLoudnessAnalyzer 
    {
        public:
			/** Construct analyzer. */
            UE_API FMultichannelLoudnessAnalyzer(float InSampleRate, const FLoudnessAnalyzerSettings& InSettings);

			/** 
			 * Calculate perceptual energy. 
			 *
			 * InSampleView		  InSampleView is expected to contain interleaved audio with 
			 *                    (InSettings.FFTSize * InNumChannels) number of samples.
			 *
			 * InNumChannels      Number of input channels.
			 *
			 * OutChannelEnergies Will be populated with energies on a per channel basis.
			 *
			 * The overall perceptual energy is returned.
			 */
			UE_DEPRECATED(5.7, "Use method which accepts a TArrayView<float> for OutChannelEnergies")
            UE_API float CalculatePerceptualEnergy(TArrayView<const float> InSampleView, const int32 InNumChannels, TArray<float>& OutChannelEnergies); 

			/** 
			 * Calculate perceptual energy. 
			 *
			 * InSampleView		  InSampleView is expected to contain interleaved audio with 
			 *                    (InSettings.FFTSize * InNumChannels) number of samples.
			 *
			 * InNumChannels      Number of input channels.
			 *
			 * OutChannelEnergies Will be populated with energies on a per channel basis.
			 *
			 * The overall perceptual energy is returned.
			 */
            UE_API float CalculatePerceptualEnergy(TArrayView<const float> InSampleView, const int32 InNumChannels, TArrayView<float> OutChannelEnergies); 

			/** 
			 * Calculate loudness
			 *
			 * InSampleView		  InSampleView is expected to contain interleaved audio with 
			 *                    (InSettings.FFTSize * InNumChannels) number of samples.
			 *
			 * InNumChannels      Number of input channels.
			 *
			 * OutChannelLoudness Will be populated with energies on a per channel basis.
			 *
			 * The overall loudness is returned.
			 */
			UE_DEPRECATED(5.7, "Use method which accepts a TArrayView<float> for OutChannelLoudness")
            UE_API float CalculateLoudness(TArrayView<const float> InSampleView, const int32 InNumChannels, TArray<float>& OutChannelLoudness);

			/** 
			 * Calculate loudness
			 *
			 * InSampleView		  InSampleView is expected to contain interleaved audio with 
			 *                    (InSettings.FFTSize * InNumChannels) number of samples.
			 *
			 * InNumChannels      Number of input channels.
			 *
			 * OutChannelLoudness Will be populated with energies on a per channel basis.
			 *
			 * The overall loudness is returned.
			 */
            UE_API float CalculateLoudness(TArrayView<const float> InSampleView, const int32 InNumChannels, TArrayView<float> OutChannelLoudness);

			/**
			 * Return const reference to settings used inside this analyzer.
			 */
            UE_API const FLoudnessAnalyzerSettings& GetSettings() const;

        private:
			void InitializeChannelWeights(int32 InNumChannels);

            FAlignedFloatBuffer MonoBuffer;
            TArray<float> ChannelWeights;
            FLoudnessAnalyzer Analyzer;
    };
}

#undef UE_API
