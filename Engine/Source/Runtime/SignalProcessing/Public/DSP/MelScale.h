// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/FloatArrayMath.h"
#include "HAL/Platform.h"
#include "Templates/UniquePtr.h"

namespace Audio
{
	/** Convert a mel frequency to a frequency in hz. */
	SIGNALPROCESSING_API float MelToHz(float InMel);

	/** Convert  a frequency in hz to a mel frequency. */
	SIGNALPROCESSING_API float HzToMel(float InHz);

	/** Normalization methods for a mel transform. */
	enum class EMelNormalization : uint8
	{
		EqualAmplitude, 	//< Useful when working on magnitude spectrum
		EqualEuclideanNorm, //< Scale energy by euclidean norm. Good when using magnitude spectrum.
		EqualEnergy			//< Useful when working on power spectrum
	};

	/** Settings for a mel kernel which transforms an linearly space spectrum (e.g. FFT Magnitude)
	 * to a mel spectrum */
	struct FMelSpectrumKernelSettings
	{
		int32 NumBands;						//< Number of bands in Mel spectrum
		float KernelMinCenterFreq;			//< Minimum frequency of lowest mel band.
		float KernelMaxCenterFreq;			//< Maximum frequency of highest mel band.
		EMelNormalization Normalization;	//< Mel band scaling. EqualAmplitude is better for the magnitude spectrum, while EqualEnergy is better for the power spectrum.
		float BandWidthStretch;				//< Controls the wideness of each band.

		FMelSpectrumKernelSettings()
		:	NumBands(26)
		,	KernelMinCenterFreq(100.f)
		,	KernelMaxCenterFreq(10000.f)
		,	Normalization(EMelNormalization::EqualEnergy)
		,	BandWidthStretch(1.f)
		{}
	};

	SIGNALPROCESSING_API TUniquePtr<FContiguousSparse2DKernelTransform> NewMelSpectrumKernelTransform(const FMelSpectrumKernelSettings& InSettings, const int32 InFFTSize, const float InSampleRate);
}
