// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	namespace EMusicalScale
	{
		// Any additions to this Enum should also be added to MetasoundMidiNoteQuantizerNode.cpp
		// in the "#pragma region Metasound Enums" area to show up as options in that Node
		enum Scale
		{
			// Modes
			Major,
			Minor_Dorian,
			Phrygian,
			Lydian,
			Dominant7th_Mixolydian,
			NaturalMinor_Aeolian,
			HalfDiminished_Locrian,

			// Non-Diatonic
			Chromatic,
			WholeTone,
			DiminishedWholeTone,

			// pentatonic
			MajorPentatonic,
			MinorPentatonic,
			Blues,

			// bebop
			Bebop_Major,
			Bebop_Minor,
			Bebop_MinorNumber2,
			Bebop_Dominant,

			// common major/minors
			HarmonicMajor,
			HarmonicMinor,
			MelodicMinor,
			SixthModeOfHarmonicMinor,

			// lydian/augmented
			LydianAugmented,
			LydianDominant,
			Augmented,

			// diminished
			Diminished,
			Diminished_BeginWithHalfStep,
			Diminished_BeginWithWholeStep,
			HalfDiminished_LocrianNumber2,

			// other
			Spanish_or_Jewish,
			Hindu,

			Count
		};
	}

	struct ScaleDegreeSet
	{
	public:
		// ctor
		SIGNALPROCESSING_API ScaleDegreeSet(const TArray<float>& InScaleDegrees, const TArray<float>& InChordTones = {});

		SIGNALPROCESSING_API TArrayView<float> GetScaleDegreeSet(bool bChordTonesOnlyIfApplicable = false);

	private:
		TArray<float> ScaleDegrees;
		TArray<float> ChordTones;
	};

	class FMidiNoteQuantizer
	{
	public:
		static SIGNALPROCESSING_API float QuantizeMidiNote(const float InNote, const float InRoot, EMusicalScale::Scale InScale, bool bChordTonesOnlyIfApplicable = false);
		static SIGNALPROCESSING_API float QuantizeMidiNote(const float InNote, const float InRoot, const TArrayView<float> InScaleDegrees, const float InSemitoneScaleRange = 12.0f);

	private:
		static float QuantizeValueToScaleDegree(const float InValue, const TArrayView<float> InScaleDegrees, const float InSemitoneScaleRange = 12.0f);

	public:
		// Statically-defined scale/chord tone definitions.
		static SIGNALPROCESSING_API TMap<EMusicalScale::Scale, ScaleDegreeSet> ScaleDegreeSetMap;

	}; // class FMidNoteQuantizer
} // namespace Audio
