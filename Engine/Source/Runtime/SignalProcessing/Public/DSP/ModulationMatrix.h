// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define MOD_MATRIX_DEBUG_NAMES 1

namespace Audio
{
	struct FPatchSource
	{
		FPatchSource()
			: Id(INDEX_NONE)
		{}

		FPatchSource(const uint32 InId)
			: Id(InId)
		{}

		void SetName(const FString& InName)
		{
#if MOD_MATRIX_DEBUG_NAMES
			Name = InName;
#endif
		}

		uint32 Id;

#if MOD_MATRIX_DEBUG_NAMES
		FString Name;
#endif

	};

	struct FPatchDestination
	{
		FPatchDestination()
			: Id(INDEX_NONE)
			, Stage(INDEX_NONE)
			, Depth(0.0f)
		{}

		FPatchDestination(const uint32 InId)
			: Id(InId)
			, Stage(INDEX_NONE)
			, Depth(0.0f)
		{}

		uint32 Id;

		int32 Stage;
		float Depth;

		void SetName(const FString& InName)
		{
#if MOD_MATRIX_DEBUG_NAMES
			Name = InName;
#endif
		}

#if MOD_MATRIX_DEBUG_NAMES
		FString Name;
#endif
	};

	struct FPatch
	{
		FPatch()
			: Source(INDEX_NONE)
			, bEnabled(true)
		{
		}

		FPatch(const FPatchSource& InSourceId, const FPatchDestination& InDestinationId)
			: Source(InSourceId)
			, bEnabled(true)
		{
			Destinations.Add(InDestinationId);
		}

		void SetName(const FString& InName)
		{
#if MOD_MATRIX_DEBUG_NAMES
			Name = InName;
#endif
		}

		// The modulation source of the patch
		FPatchSource Source;

		// The modulation destinations of the patch to support multiple destinations
		TArray<FPatchDestination> Destinations;

#if MOD_MATRIX_DEBUG_NAMES
		FString Name;
#endif
		bool bEnabled;
	};

	class FModulationMatrix
	{
	public:
		SIGNALPROCESSING_API FModulationMatrix();
		SIGNALPROCESSING_API virtual ~FModulationMatrix();

		// Initialize the modulation matrix with the desired number of voices
		SIGNALPROCESSING_API void Init(const int32 NumVoices);

		// Returns the number of patch connections
		SIGNALPROCESSING_API int32 GetNumPatches(const int32 VoiceId) const;

		// Creates a new patch source object and returns the patch source id
		SIGNALPROCESSING_API FPatchSource CreatePatchSource(const int32 VoiceId);

		// Crates a new patch destination object and returns the patch destination id
		SIGNALPROCESSING_API FPatchDestination CreatePatchDestination(const int32 VoiceId, const int32 Stage, const float DefaultDepth);

		// Adds a new patch connection between one source and one or more destinations
		SIGNALPROCESSING_API bool AddPatch(const int32 VoiceId, FPatch* Patch);

		// Removes the given patch connection between a source and one or more destinations
		SIGNALPROCESSING_API bool RemovePatch(const int32 VoiceId, FPatch* Patch);

		// Reset all patch state
		SIGNALPROCESSING_API void ResetPatchSourceState();

		// Clear all patch connections
		SIGNALPROCESSING_API void ClearPatches(const int32 VoiceId);

		// Set the modulation source value of the given id
		SIGNALPROCESSING_API bool SetSourceValue(const int32 VoiceId, const FPatchSource& Source, const float Value);

		// Get the modulation destination value of the given destination id
		SIGNALPROCESSING_API bool GetDestinationValue(const int32 VoiceId, const FPatchDestination& Destination, float& OutValue) const;

		// Perform the matrix update. Optionally do only a given stage. 
		SIGNALPROCESSING_API void Update(const int32 VoiceId, const int32 Stage = INDEX_NONE);

	protected:

		SIGNALPROCESSING_API void ResetDestinations(const int32 VoiceId);
		SIGNALPROCESSING_API bool ValidatePatch(const int32 VoiceId, FPatch* Patch);

		int32 NumVoices;
		TArray<TArray<FPatch*>> Patches;
		TArray<TArray<float>> Sources;

		struct FDestData
		{
			// What the value of the destination is
			float Value;
			
			// Whether or not anybody changed it
			bool bDirty;

			FDestData()
				: Value(0.0f)
				, bDirty(false)
			{}
		};

		TArray<TArray<FDestData>> Destinations;
	};

}
