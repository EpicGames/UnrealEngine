// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"

// Collection of multi-mono DSP algorithms. Although technically most of these are just setup to call the mono versions, there is enough
// boilerplate to warrant a single lib.

// Example Multi-mono-layout: [L][L][L][L][L][R][R][R][R][R], i.e. an entire channels worth of frames consecutively in memory, next
// to the next channel and so on.

namespace Audio
{
	constexpr int32 MaxStackMultiMonoChannels = 128; // Some relatively high number. These are just pointers on stack.

	// Stack array of pointers alias.
	template<typename T> using TStackArrayOfPointers = TArray<T*,TInlineAllocator<MaxStackMultiMonoChannels>>;

	// Helper to create array of pointers from multi-mono.
	// Warning Array of pointers will be larger than you need, but filled to InNumChannels
	template<typename T> TStackArrayOfPointers<T> MakeMultiMonoPointersFromView(TArrayView<T> InMultiMono, const int32 InNumFrames, const int32 InNumChannels)
	{
		checkSlow(InNumChannels <= MaxStackMultiMonoChannels);
		checkSlow(InNumChannels > 0);
		checkSlow(InMultiMono.GetData());
		checkSlow(InMultiMono.Num() % InNumFrames == 0 && InMultiMono.Num() >= InNumFrames);
		TStackArrayOfPointers<T> ArrayOfPointers;
		ArrayOfPointers.SetNum(InNumChannels);
		for (int32 i = 0; i < InNumChannels; ++i)
		{
			ArrayOfPointers[i] = InMultiMono.GetData() + (InNumFrames * i);
		}
		return ArrayOfPointers; // RTO so this should just inline
	}
	
	/**
	 * Given a Matrix of Gains (produced by calling ChannelMap.cpp (Audio::Create2DChannelMap), Mix Up/Down Source into Destination.
	 * @param InSrc ArrayView to Source (Multi-mono)
	 * @param InDst ArrayView to Destination (Mult-mono).
	 * @param NumFrames Number of Frames in Each Channel.
	 * @param MixGains Matrix of Gains (in Row Major Format, see Create2DChannelMap). [NumDstChannels*NumSrcChannels]
	 * @param NumDstChannels Num Of Destination Channels.
	 * @param NumSrcChannels Num Of Source Channels.
	*/
	SIGNALPROCESSING_API void MultiMonoMixUpOrDown(
		TArrayView<const float> InSrc, const TArrayView<float> InDst, const int32 NumFrames, TArrayView<const float> MixGains,
			const int32 NumSrcChannels, const int32 NumDstChannels);

	SIGNALPROCESSING_API void MultiMonoMixUpOrDown(
		TArrayView<const float*> InSrc, TArrayView<float*> InDst, const int32 NumFrames,TArrayView<const float> MixGains);
	
}


