// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"

namespace Audio
{
	/** A deinterleaved multichannel buffer */
	using FMultichannelBuffer = TArray<Audio::FAlignedFloatBuffer>;

	/** A deinterleaved multichannel buffer view. The semantics are similar to 
	 * TArray<> and TArrayView<>. FMultichannelBufferView allows sample values to 
	 * be read and written, but it cannot be resized. 
	 */
	using FMultichannelBufferView = TArray<TArrayView<float>>;

	/** A deinterleaved multichannel circular buffer */
	using FMultichannelCircularBuffer = TArray<Audio::TCircularAudioBuffer<float>>;

	/** Set the number of channels and frames for a multichannel buffer.
	 *
	 * @param InNumChannels - Number of channels to hold in the buffer.
	 * @param InNumFrames - Number of frames to hold in the buffer.
	 * @param OutBuffer - Buffer to resize.
	 */
	SIGNALPROCESSING_API void SetMultichannelBufferSize(int32 InNumChannels, int32 InNumFrames, FMultichannelBuffer& OutBuffer);

	/** Set the number of channels and capacity for a multichannel circular buffer.
	 *
	 * @param InNumChannels - Number of channels to hold in the buffer.
	 * @param InNumFrames - Maximum number of frames the buffer can hold.
	 * @param OutBuffer - Buffer to resize.
	 */
	SIGNALPROCESSING_API void SetMultichannelCircularBufferCapacity(int32 InNumChannels, int32 InNumFrames, FMultichannelCircularBuffer& OutBuffer);

	/** Set the number of frames for a multichannel buffer.
	 *
	 * @param InNumFrames - Number of frames to hold in the buffer.
	 * @param OutBuffer - Buffer to resize.
	 */
	SIGNALPROCESSING_API void SetMultichannelBufferSize(int32 InNumFrames, FMultichannelBuffer& OutBuffer);

	/** Return the number of frames in the buffer. It is expected that each channel
	 * contains the same number of frames. */
	SIGNALPROCESSING_API int32 GetMultichannelBufferNumFrames(const FMultichannelBuffer& InBuffer);

	/** Return the number of frames in the buffer. It is expected that each channel
	 * contains the same number of frames. */
	SIGNALPROCESSING_API int32 GetMultichannelBufferNumFrames(const FMultichannelCircularBuffer& InBuffer);

	/** Return the number of frames in the buffer. It is expected that each channel
	 * contains the same number of frames. */
	SIGNALPROCESSING_API int32 GetMultichannelBufferNumFrames(const FMultichannelBufferView& InBuffer);

	/** Creates a FMultichannelBufferView from a FMultichannelBuffer. */
	SIGNALPROCESSING_API FMultichannelBufferView MakeMultichannelBufferView(FMultichannelBuffer& InBuffer);

	/** Creates a FMultichannelBufferView from a FMultichannelBuffer. */
	SIGNALPROCESSING_API FMultichannelBufferView MakeMultichannelBufferView(FMultichannelBuffer& InBuffer, int32 InStartFrameIndex, int32 InNumFrames);

	/** Slice a multichannel buffer view to a given frame range. */
	SIGNALPROCESSING_API FMultichannelBufferView SliceMultichannelBufferView(const FMultichannelBufferView& View, int32 InStartFrameIndex, int32 InNumFrames);

	/** Shift the start frame of a multichannel buffer view to a given number of frames. */
	SIGNALPROCESSING_API void ShiftMultichannelBufferView(int32 InNumFrames, FMultichannelBufferView& View);
}
