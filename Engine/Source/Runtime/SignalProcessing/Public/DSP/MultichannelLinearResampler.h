// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/MultichannelBuffer.h"

namespace Audio
{
	/** Linear resampler working on multichannel buffers */
	class FMultichannelLinearResampler
	{
		static constexpr float MinFrameRatioFrameDelta = 0.001f;
	public:

		SIGNALPROCESSING_API static const float MaxFrameRatio;
		SIGNALPROCESSING_API static const float MinFrameRatio;

		/** Construct a linear resampler.
		 *
		 * @param InNumChannel - Number of audio channels in input and output buffers.
		 */
		SIGNALPROCESSING_API FMultichannelLinearResampler(int32 InNumChannels);

		/** Sets the number of input frames to read per an output frame. 0.5 is 
		 * half speed, 1.f is normal speed, 2.0 is double speed.
		 *
		 * @param InRatio - Ratio of input frames consumed per an output frame produced. 
		 * @param InNumFramesToInterpolate - Number of output frames over which 
		 *                                   to interpolate the frame ratio.
		 */
		SIGNALPROCESSING_API void SetFrameRatio(float InRatio, int32 InNumFramesToInterpolate=0);

		/** Translates an input frame index to an output frame index given the
		 * state of the resampler. 
		 *
		 * Note: Indices are relative to the buffers passed to `ProcessAudio(...)`.
		 * The resampler does not maintain a sample counter between calls to `ProcessAudio(...)`.
		 *
		 * @param InInputFrameIndex - Index of input frame.
		 * @return Index of output frame. 
		 */
		SIGNALPROCESSING_API float MapInputFrameToOutputFrame(float InInputFrameIndex) const;

		/** Translates an output frame index to an input frame index given the
		 * state of the resampler. 
		 *
		 * Note: Indices are relative to the buffers passed to `ProcessAudio(...)`.
		 * The resampler does not maintain a sample counter between calls to `ProcessAudio(...)`.
		 *
		 * @param InOutputFrameIndex - Index of output frame.
		 * @return Index of input frame. 
		 */
		SIGNALPROCESSING_API float MapOutputFrameToInputFrame(float InOutputFrameIndex) const;

		/** Returns the minimum number of input frames needed to produce the desired
		 * number of output frames given the current state of the resampler. 
		 *
		 * @param InNumOutputFrames - The desired number of output frames.
		 * @return The minimum number of input frames. 
		 */
		SIGNALPROCESSING_API int32 GetNumInputFramesNeededToProduceOutputFrames(int32 InNumOutputFrames) const;

		/** Consumes audio from the input buffer and produces audio in the output buffer.
		 * The desired number of frames to produce is determined by the output audio buffer
		 * size. For the desired number of samples to be produced, the input audio must have the minimum 
		 * number of frames needed to produce the output frames (see `GetNumInputFramesNeededToProduceOutputFrames(...)`).
		 * Input samples which are no longer needed are removed from the input buffer.
		 *
		 * @param InAudio - Multichannel circular buffer of input audio.
		 * @param OutAudio - Multichannel buffer of output audio.
		 *
		 * @return Actual number of frames produced. 
		 */
		SIGNALPROCESSING_API int32 ProcessAndConsumeAudio(FMultichannelCircularBuffer& InAudio, FMultichannelBuffer& OutAudio);
		SIGNALPROCESSING_API int32 ProcessAndConsumeAudio(FMultichannelCircularBuffer& InAudio, FMultichannelBufferView& OutAudio);

	private:

		template<typename OutputMultichannelBufferType>
		int32 ProcessAndConsumeAudioInternal(FMultichannelCircularBuffer& InAudio, OutputMultichannelBufferType& OutAudio);


		float ProcessChannelAudioInternal(TArrayView<const float> InAudio, TArrayView<float> OutAudio);
		int32 GetNumBufferFramesToProduceOutputFrames(int32 InNumOutputFrames) const;

		Audio::FAlignedFloatBuffer WorkBuffer;
		float CurrentInputFrameIndex = 0.f;
		float CurrentFrameRatio = 1.f;
		float TargetFrameRatio = 1.f;
		float FrameRatioFrameDelta = 0.f;
		int32 NumFramesToInterpolate = 0;
		int32 NumChannels = 0;
	};
}
