// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"

class FEvent;

namespace Audio
{

	
	/**
	 * This class can be thought of as an output for a single constructed instance of FPatchInput.
	 * Each FPatchOutput can only be connected to one FPatchInput. To route multiple outputs, see FPatchSplitter.
	 * To route multiple inputs, see FPatchMixer.
	 *
	 * Example usage:
	 * 
	 * FPatchOutputStrongPtr NewOutput(new FPatchOutput(4096));
	 * FPatchInput NewInput(NewOutput);
	 *
	 * // On one thread, push audio to the output:
	 * NewInput.PushAudio(AudioBufferPtr, AudioBufferNumSamples);
	 *
	 * // and on a seperate thread, retrieve the audio:
	 * NewOutput->PopAudio(OutAudioBufferPtr, AudioBufferNumSamples);
	 * 
	 */
	struct FPatchOutput
	{
	public:
		SIGNALPROCESSING_API FPatchOutput(int32 InMaxCapacity, float InGain = 1.0f);

		/** The default constructor will result in an uninitialized, disconnected patch point. */
		SIGNALPROCESSING_API FPatchOutput();

		SIGNALPROCESSING_API virtual ~FPatchOutput();
		
		/** Copies the minimum of NumSamples or however many samples are available into OutBuffer. Returns the number of samples copied, or -1 if this output's corresponding input has been destroyed. */
		SIGNALPROCESSING_API int32 PopAudio(float* OutBuffer, int32 NumSamples, bool bUseLatestAudio);

		/** Sums the minimum of NumSamples or however many samples are available into OutBuffer. Returns the number of samples summed into OutBuffer. */
		SIGNALPROCESSING_API int32 MixInAudio(float* OutBuffer, int32 NumSamples, bool bUseLatestAudio);

		/** Returns the current number of samples buffered on this output. */
		SIGNALPROCESSING_API int32 GetNumSamplesAvailable() const;

		/** Returns the amount of space available in this output. */
		SIGNALPROCESSING_API int32 GetNumSamplesOverhead() const;

		/** Pauses the current thread until there are the given number of samples available to pop. Will return true if it succeeded, false if it timed out. */
		SIGNALPROCESSING_API bool WaitUntilNumSamplesAvailable(int32 NumSamples, uint32 TimeOutMilliseconds = MAX_uint32);

		/** Returns true if the input for this patch has been destroyed. */
		SIGNALPROCESSING_API bool IsInputStale() const;

		friend class FPatchInput;
		friend class FPatchMixer;
		friend class FPatchSplitter;
	private:
		
		SIGNALPROCESSING_API int32 PushAudioToInternalBuffer(const float* InBuffer, int32 NumSamples);
		
		SIGNALPROCESSING_API void EmptyInternalBuffer();

		// Internal buffer.
		TCircularAudioBuffer<float> InternalBuffer;

		// For MixInAudio, audio is popped off of InternalBuffer onto here, and then mixed into OutBuffer in MixInAudio.
		FAlignedFloatBuffer MixingBuffer;

		// This is applied in PopAudio or MixInAudio.
		TAtomic<float> TargetGain;
		float PreviousGain;

		// This is used to breadcrumb the FPatchOutput when we want to delete it.
		int32 PatchID;

		// Counter that is incremented/decremented to allow FPatchInput to be copied around safely.
		std::atomic<int32> NumAliveInputs;

		// Event to pause the current thread until a given number of samples has been filled
		std::atomic<FEvent*> SamplesPushedEvent;

		static SIGNALPROCESSING_API TAtomic<int32> PatchIDCounter;
	};

	/** Patch outputs are owned by the FPatchMixer, and are pinned by the FPatchInput. */
	typedef TSharedPtr<FPatchOutput, ESPMode::ThreadSafe> FPatchOutputStrongPtr;
	typedef TWeakPtr<FPatchOutput, ESPMode::ThreadSafe> FPatchOutputWeakPtr;

	/**
	 * Handle to a patch. Should only be used by a single thread.
	 */
	class FPatchInput
	{
	public:
		/** PatchInputs can only be created from explicit outputs. */
		SIGNALPROCESSING_API FPatchInput(const FPatchOutputStrongPtr& InOutput);
		SIGNALPROCESSING_API FPatchInput(const FPatchInput& Other);
		SIGNALPROCESSING_API FPatchInput& operator=(const FPatchInput& Other);
		SIGNALPROCESSING_API FPatchInput& operator=(FPatchInput&& Other);
		SIGNALPROCESSING_API FPatchInput(FPatchInput&& Other);

		/** Default constructed FPatchInput instances will always return -1 for PushAudio and true for IsOutputStillActive. */
		FPatchInput() = default;

		SIGNALPROCESSING_API ~FPatchInput();

		/** Pushes audio from InBuffer to the corresponding FPatchOutput.
		 *  Pushes zeros if InBuffer is nullptr.
		 *  Returns how many samples were able to be pushed, or -1 if the output was disconnected.
		 */
		SIGNALPROCESSING_API int32 PushAudio(const float* InBuffer, int32 NumSamples);

		/** Clears any audio queued up in the patch. */
		SIGNALPROCESSING_API void ClearAudio();

		/** Returns the current number of samples buffered in this input. */
		SIGNALPROCESSING_API int32 GetNumSamplesAvailable() const;

		/** Returns the amount of space available in this input. */
		SIGNALPROCESSING_API int32 GetNumSamplesOverhead() const;

		SIGNALPROCESSING_API void SetGain(float InGain);

		/** Returns false if this output was removed, either because someone called FPatchMixer::RemoveTap with this FPatchInput, or the FPatchMixer was destroyed. */
		SIGNALPROCESSING_API bool IsOutputStillActive() const;

		/** Returns false if this output was not initialized properly. */
		SIGNALPROCESSING_API bool IsValid() const;

		SIGNALPROCESSING_API void Reset();
		
		friend class FPatchMixer;
		friend class FPatchSplitter;

	private:
		/** Strong pointer to our destination buffer. */
		FPatchOutputStrongPtr OutputHandle;

		/** Counter of the number of push calls. */
		int32 PushCallsCounter = 0;
	};

	/**
	 * This class is used for retrieving and mixing down audio from multiple threads.
	 * Important to note that this is MPSC: while multiple threads can enqueue audio on an instance of FPatchMixer using instances of FPatchInput,
	 * only one thread can call PopAudio safely.
	 */
	class FPatchMixer
	{
	public:
		/** Adds a new input to the tap collector. Calling this is thread safe, but individual instances of FPatchInput are only safe to be used from one thread. */
		SIGNALPROCESSING_API FPatchInput AddNewInput(int32 MaxLatencyInSamples, float InGain);

		/** Adds an existing patch input to the patch mixer. */
		SIGNALPROCESSING_API void AddNewInput(const FPatchInput& InPatchInput);

		/** Removes a tap from the tap collector. Calling this is thread safe, though FPatchOutput will likely not be deleted until the next call of PopAudio. */
		SIGNALPROCESSING_API void RemovePatch(const FPatchInput& InPatchInput);

		/** Mixes all inputs into a single buffer. This should only be called from a single thread. Returns the number of non-silent samples popped to OutBuffer. */
		SIGNALPROCESSING_API int32 PopAudio(float* OutBuffer, int32 OutNumSamples, bool bUseLatestAudio);

		/** This returns the number of inputs currently connected to this patch mixer. Thread safe, but blocks for PopAudio. */
		SIGNALPROCESSING_API int32 Num();

		/** This function call gets the maximum number of samples that's safe to pop, based on the thread with the least amount of samples buffered. Thread safe, but blocks for PopAudio. */
		SIGNALPROCESSING_API int32 MaxNumberOfSamplesThatCanBePopped();

		/** Pauses the current thread until there are the given number of samples available to pop. Will return true if it succeeded, false if it timed out. */
		SIGNALPROCESSING_API bool WaitUntilNumSamplesAvailable(int32 NumSamples, uint32 TimeOutMilliseconds = MAX_uint32);

		/** Disconnect everything currently connected to this mixer. */
		SIGNALPROCESSING_API void DisconnectAllInputs();

	private:
		/** Called within PopAudio. Flushes the PendingNewPatches array into CurrentPatches. During this function, AddNewPatch is blocked. */
		void ConnectNewPatches();

		/** Called within PopAudio and MaxNumberOfSamplesThatCanBePopped. Removes PendingTapsToDelete from CurrentPatches and ConnectNewPatches. 
		 * During this function, RemoveTap and AddNewPatch are blocked. Callers of this function must have CurrentPatchesCriticalSection locked. */
		void CleanUpDisconnectedPatches();

		/** New taps are added here in AddNewPatch, and then are moved to CurrentPatches in ConnectNewPatches. */
		TArray<FPatchOutputStrongPtr> PendingNewInputs;
		/** Contended by AddNewPatch, ConnectNewPatches and CleanUpDisconnectedTaps. */
		FCriticalSection PendingNewInputsCriticalSection;

		/** Patch IDs of individual audio taps that will be removed on the next call of CleanUpDisconnectedPatches. */
		TArray<int32> DisconnectedInputs;
		/** Contended by RemoveTap, AddNewPatch, and ConnectNewPatches. */
		FCriticalSection InputDeletionCriticalSection;

		/** Only accessed within PopAudio. Indirect array of taps that are mixed in during PopAudio. */
		TArray<FPatchOutputStrongPtr> CurrentInputs;
		FCriticalSection CurrentPatchesCriticalSection;
	};

	/**
	 * This class is used to post audio from one source to multiple threads.
	 * This class is SPMC: multiple threads can call FPatchOutputStrongPtr->PopAudio safely,
	 * but only one thread can call PushAudio.
	 */
	class FPatchSplitter
	{
	public:
		/**
		 * Adds a new output. Calling this is thread safe, but individual instances of FPatchOutput are only safe to be used from one thread.
		 * the returned FPatchOutputPtr can be safely destroyed at any point.
		 */
		SIGNALPROCESSING_API FPatchOutputStrongPtr AddNewPatch(int32 MaxLatencyInSamples, float InGain);

		/** Adds a new a patch from an existing patch output. */
		SIGNALPROCESSING_API void AddNewPatch(FPatchOutputStrongPtr&& InPatchOutputStrongPtr);
		SIGNALPROCESSING_API void AddNewPatch(const FPatchOutputStrongPtr& InPatchOutputStrongPtr);

		/** This call pushes audio to all outputs connected to this splitter. Only should be called from one thread. */
		SIGNALPROCESSING_API int32 PushAudio(const float* InBuffer, int32 InNumSamples);

		/** This returns the number of outputs currently connected to this patch splitter. Thread safe, but blocks for PushAudio. */
		SIGNALPROCESSING_API int32 Num();

		/** This function call gets the maximum number of samples that's safe to push. Thread safe, but blocks for PushAudio. */
		SIGNALPROCESSING_API int32 MaxNumberOfSamplesThatCanBePushed();

	private:
		void AddPendingPatches();

		TArray<FPatchInput> PendingOutputs;
		FCriticalSection PendingOutputsCriticalSection;

		TArray<FPatchInput> ConnectedOutputs;
		FCriticalSection ConnectedOutputsCriticalSection;
	};

	/**
	 * This class is used to mix multiple inputs from disparate threads to a single mixdown and deliver that mixdown to multiple outputs.
	 * This class is MPMC, but only one thread can and should call ProcessAudio().
	 */
	class FPatchMixerSplitter
	{
	public:
		/**
		 * Adds a new output. Calling this is thread safe, but individual instances of FPatchOutput are only safe to be used from one thread.
		 * the returned FPatchOutputPtr can be safely destroyed at any point.
		 */
		SIGNALPROCESSING_API FPatchOutputStrongPtr AddNewOutput(int32 MaxLatencyInSamples, float InGain);

		/** Adds a new a patch from an existing patch output. */
		SIGNALPROCESSING_API void AddNewOutput(const FPatchOutputStrongPtr& InPatchOutputStrongPtr);

		/** Adds a new input to the tap collector. Calling this is thread safe, but individual instances of FPatchInput are only safe to be used from one thread. */
		SIGNALPROCESSING_API FPatchInput AddNewInput(int32 MaxLatencyInSamples, float InGain);

		/** Adds a new a patch input from an existing patch input object. */
		SIGNALPROCESSING_API void AddNewInput(FPatchInput& InInput);

		/** Removes a tap from the tap collector. Calling this is thread safe, though FPatchOutput will likely not be deleted until the next call of PopAudio. */
		SIGNALPROCESSING_API void RemovePatch(const FPatchInput& InInput);

		/** Mixes audio from all inputs and pushes it to all outputs. Should be called regularly. */
		SIGNALPROCESSING_API void ProcessAudio();

	private:
		FPatchMixer Mixer;
		FPatchSplitter Splitter;

		/** This buffer is used to pop audio from our Mixer and push to our splitter. */
		FAlignedFloatBuffer IntermediateBuffer;
	};
}
