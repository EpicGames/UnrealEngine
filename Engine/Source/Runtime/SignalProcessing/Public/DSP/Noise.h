// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/RandomStream.h"

namespace Audio
{
	/** 
	* White noise generator 
	* Flat spectrum
	*/
	class FWhiteNoise
	{
	public:
		SIGNALPROCESSING_API FWhiteNoise();
		SIGNALPROCESSING_API FWhiteNoise(int32 InRandomSeed);

		/** Generate next sample of white noise */
		inline float Generate()
		{
			return (RandomStream.FRand() * 2.f) - 1.0f;
		}
		
		/** Generate next sample of white noise (with optional Scale and Add params) */
		inline float Generate(float InScale, float InAdd)
		{
			return Generate() * InScale + InAdd;
		}
	private:	
		FRandomStream RandomStream;
	};

	/** 
	* Pink noise generator
	* 1/Frequency noise spectrum
	*/
	class FPinkNoise
	{
	public:
		/** Constructor. Without seed argument, uses Cpu cycles to chose one at "random" */
		SIGNALPROCESSING_API FPinkNoise();

		/** Constructor with seed input */
		SIGNALPROCESSING_API FPinkNoise(int32 InRandomSeed);

		/** Generate next sample of pink noise. */
		SIGNALPROCESSING_API float Generate();

		/** Set Pink Noise Filter Gain (default -3db) */
		void SetFilterGain(float InFilterGain) 
		{
			A0 = InFilterGain;
		}

	private:
		FWhiteNoise Noise;
		float X_Z[4];
		float Y_Z[4];
		float A0 = 1.f;
	};
}
