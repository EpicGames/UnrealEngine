// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"

namespace Audio
{
	// Simple 1-pole lowpass filter
	class FOnePoleLPF
	{
	public:

		// Constructor 
		FOnePoleLPF()
			: CutoffFrequency(0.0f)
			, B1(0.0f)
			, A0(1.0f)
			, Z1(0.0f)
		{}

		// Set the LPF gain coefficient
		inline void SetG(float InG)
		{ 
			B1 = InG;
			A0 = 1.0f - B1;
		}

		// Resets the sample delay to 0
		void Reset()
		{
			B1 = 0.0f;
			A0 = 1.0f;
			Z1 = 0.0f;
		}

		/** Sets the filter frequency using normalized frequency (between 0.0 and 1.0f or 0.0 hz and Nyquist Frequency in Hz) */
		inline void SetFrequency(const float InFrequency)
		{
			if (!FMath::IsNearlyEqual(InFrequency, CutoffFrequency))
			{
				CutoffFrequency = InFrequency;
				B1 = FMath::Exp(-PI * CutoffFrequency);
				A0 = 1.0f - B1;
			}
		}

		inline float ProcessAudioSample(const float InputSample)
		{
			const float Yn = InputSample*A0 + B1*Z1;
			Z1 = Yn;
			return Yn;
		}

	protected:
		float CutoffFrequency;

		// Filter coefficients
		float B1;
		float A0;

		// 1-sample delay
		float Z1;
	};

	// one pole LPF filter for multiple channels
	class FOnePoleLPFBank
	{
	public:
		FOnePoleLPFBank()
			: DelayPtr(nullptr)
			, NumChannels(1)
			, CutoffFrequency(-1.0f)
			, B1(0.0f)
			, A0(1.0f)
		{
			Z1.Init(0.0f, NumChannels);
			DelayPtr = Z1.GetData();
		}

		void Init(int32 InSampleRate, int32 InNumChannels)
		{
			SampleRate = InSampleRate;
			NumChannels = InNumChannels;
			CutoffFrequency = -1.0f;
			Z1.Init(0.0f, NumChannels);
			DelayPtr = Z1.GetData();
			Reset();
		}

		// Set the LPF gain coefficient
		inline void SetG(float InG)
		{
			B1 = InG;
			A0 = 1.0f - B1;
		}

		// Clears memory without reevaluating coefficients. This function is useful when there is a break between ProcessAudio calls.
		void ClearMemory()
		{
			Z1.SetNumZeroed(NumChannels);
		}

		// Resets the sample delay to 0
		void Reset()
		{
			B1 = 0.0f;
			A0 = 1.0f;
			ClearMemory();
		}

		/** Sets the filter frequency using normalized frequency (between 0.0 and 1.0f or 0.0 hz and Nyquist Frequency in Hz) */
		void SetFrequency(const float InFrequency)
		{
			if (!FMath::IsNearlyEqual(InFrequency, CutoffFrequency))
			{
				CutoffFrequency = InFrequency;
				float NormalizedFreq = FMath::Clamp(2.0f * InFrequency / SampleRate, 0.0f, 1.0f);
				B1 = FMath::Exp(-PI * NormalizedFreq);
				A0 = 1.0f - B1;
			}
		}

		inline void ProcessAudio(float* InputFrame, float* OutputFrame)
		{			
			for (int32 i = 0; i < NumChannels; ++i)
			{
				const float Yn = InputFrame[i] * A0 + B1*DelayPtr[i];
				DelayPtr[i] = Yn;
				OutputFrame[i] = Yn;
			}
		}

	protected:

		TArray<float> Z1;
		float* DelayPtr;
		int32 NumChannels;
		float CutoffFrequency;
		int32 SampleRate;
		float B1;
		float A0;
	};

}
