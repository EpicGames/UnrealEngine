// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "DSP/ModulationMatrix.h"
#include "DSP/Noise.h"

namespace Audio
{
	// Struct which wraps all factors which contribute to pitch of the oscillator
	struct FOscFrequencyMod
	{	
		// A factor which directly scales the frequency output (used for FM modulation)
		float Scale;
		
		// External modulation source
		float ExternalMod;

		// The modulated frequency value
		float Mod;

		// Exponential frequency modulation
		float Detune;
		
		// Pitch bend modulation (i.e. from mod wheel)
		float PitchBend;

		// Octave frequency modulation
		float Octave;

		// Semitones frequency modulation
		float Semitones;

		// cents frequency modulation
		float Cents;

		FOscFrequencyMod()
			: Scale(1.0f)
			, ExternalMod(0.0f)
			, Mod(0.0f)
			, Detune(0.0f)
			, PitchBend(0.0f)
			, Octave(0.0f)
			, Semitones(0.0f)
			, Cents(0.0f)
		{}
	};

	// Oscillator base class
	class IOscBase
	{
	public:
		SIGNALPROCESSING_API IOscBase();
		SIGNALPROCESSING_API IOscBase(const IOscBase&);
		SIGNALPROCESSING_API virtual ~IOscBase();

		// Initializes the oscillator
		SIGNALPROCESSING_API virtual void Init(const float InSampleRate, const int32 InVoiceId = 0, FModulationMatrix* InMatrix = nullptr, const int32 ModMatrixStage = 1);

		// Starts the oscillator
		virtual void Start() = 0;

		// Stops the oscillator
		virtual void Stop() = 0;

		// Generates the next sample of audio, optionally passes out the auxillary output (supported in some oscillators)
		virtual float Generate(float* AuxOutput = nullptr) = 0;

		// Updates oscillator state
		SIGNALPROCESSING_API virtual void Update();

		// Resets the oscillator
		SIGNALPROCESSING_API virtual void Reset();

		// Resets the phase of this oscillator to 0.0f
		SIGNALPROCESSING_API virtual void ResetPhase();

		// Sets the gain of the oscillator
		void SetGain(const float InGain) { Gain = InGain; }

		// Sets the gain modulator of the oscillator
		void SetGainMod(const float InGainMod) { ExternalGainMod = InGainMod; }

		// Sets the base frequency of the oscillator
		SIGNALPROCESSING_API void SetFrequency(const float InFreqBase);

		// Sets a frequency modulation value
		SIGNALPROCESSING_API void SetFrequencyMod(const float InFreqMod);

		// Sets the base frequency of the oscillator from the midi note number
		SIGNALPROCESSING_API void SetNote(const float InNote);

		// Returns the frequency of the oscillator
		float GetFrequency() const { return BaseFreq; }

		// Returns the frequency of the oscillator
		float GetGain() const { return Gain; }

		SIGNALPROCESSING_API void SetCents(const float InCents);
		SIGNALPROCESSING_API void SetOctave(const float InOctave);
		SIGNALPROCESSING_API void SetSampleRate(const float InSampleRate);
		SIGNALPROCESSING_API void SetSemitones(const float InSemiTone);
		SIGNALPROCESSING_API void SetDetune(const float InDetune);
		SIGNALPROCESSING_API void SetPitchBend(const float InPitchBend);
		SIGNALPROCESSING_API void SetFreqScale(const float InFreqScale);

		// Sets the LFO pulse width for square-wave type oscillators
		SIGNALPROCESSING_API void SetPulseWidth(const float InPulseWidth);

		// Returns whether or not this oscillator is playing
		bool IsPlaying() const { return bIsPlaying; }

		// Returns if this oscillator should be synced to a leader oscillator
		bool IsSync() const { return bIsSync; }

		// Sets whether or not this oscillator should be synced to a leader oscillator. leader oscillator needs to have set this oscillator as its follower.
		void SetSync(const bool bInSync) { bIsSync = bInSync; }

		// Deprecated: use SetFollowerOsc
		UE_DEPRECATED(5.1, "SetSlaveOsc is deprecated, please use SetFollowerOsc instead.")
		SIGNALPROCESSING_API void SetSlaveOsc(IOscBase* InSlaveOsc);

		// Sets the input oscillator as the follower of this oscillator
		SIGNALPROCESSING_API void SetFollowerOsc(IOscBase* InFollowerOsc);

		// Return patch destinations for various modulatable parameters
		FPatchDestination GetModDestFrequency() const { return ModFrequencyDest; }
		FPatchDestination GetModDestPulseWidth() const { return ModPulseWidthDest; }
		FPatchDestination GetModDestGain() const { return ModGainDest; }
		FPatchDestination GetModDestAdd() const { return ModAddDest; }
		FPatchDestination GetModDestScale() const { return ModScaleDest; }

	protected:

		// Updates the phase based on the phase increment
		void UpdatePhase()
		{
			Phase += PhaseInc;
		}

		// Returns true if there was a phase wrap this update
		bool WrapPhase()
		{
			bool Result = false;
			if (PhaseInc > 0.0f && Phase >= 1.0f)
			{
				Phase = FMath::Fmod(Phase, 1.0f);
				Result = true;
			}
			else if (PhaseInc < 0.0f && Phase <= 0.0f)
			{
				Phase = FMath::Fmod(Phase, 1.0f) + 1.0f;
				Result = true;
			}

			if (Result && FollowerOsc && FollowerOsc->IsSync())
			{
				FollowerOsc->ResetPhase();
			}

			return Result;
		}

		// Returns the current phase of the oscillator
		inline float GetPhase() const
		{
			return Phase;
		}

		// Returns the quadrature phase, wrapped
		inline float GetQuadPhase() const
		{
			return FMath::Fmod(Phase + 0.25f, 1.0f);
		}

	protected:

		// The voice ID that this oscillator belongs to
		int32 VoiceId;

		// Sample rate of the oscillator
		float SampleRate;

		// Maximum frequency allowed
		float Nyquist;

		// The final frequency of the oscillator after computing all factors contributing to frequency
		float Freq;

		// The base frequency of the oscillator
		float BaseFreq;

		// Holds all frequency data
		FOscFrequencyMod FreqData;

		// Linear gain of the oscillator
		float Gain;

		// Linear gain modulation of the oscillator (used in amplitude modulation)
		float ExternalGainMod;

		// The current phase of oscillator (between 0.0 and 1.0)
		float Phase;

		// How much to increment the phase each update
		float PhaseInc;

		// Pulse width used in square LFOs
		float PulseWidthBase;

		// Pulse width modulator factor
		float PulseWidthMod;

		// The final pulse width
		float PulseWidth;

		// Modulation matrix to use for this oscillator
		FModulationMatrix* ModMatrix;

		FPatchDestination ModFrequencyDest;
		FPatchDestination ModPulseWidthDest;
		FPatchDestination ModGainDest;
		FPatchDestination ModScaleDest;
		FPatchDestination ModAddDest;

		// Ptr to a follower oscillator that can be triggered to 0 phase if it is synced.
		IOscBase* FollowerOsc;

		// Whether or not the oscillator is on or off
		bool bIsPlaying;
		bool bChanged;
		bool bIsSync;
	};

	namespace EOsc
	{
		enum Type
		{
			Sine,
			Saw,
			Triangle,
			Square,
			Noise,
			NumOscTypes
		};
	}

	// Pitched oscillator
	class FOsc : public IOscBase
	{
	public:
		SIGNALPROCESSING_API FOsc();
		SIGNALPROCESSING_API virtual ~FOsc();

		//~ Begin FOscBase
		SIGNALPROCESSING_API virtual void Start() override;
		SIGNALPROCESSING_API virtual void Stop() override;
		SIGNALPROCESSING_API virtual void Reset() override;
		SIGNALPROCESSING_API virtual void Update() override;
		SIGNALPROCESSING_API virtual float Generate(float* AuxOutput = nullptr) override;
		//~ End FOscBase

		// Sets the oscillator type
		void SetType(const EOsc::Type InType) { OscType = InType; }

		// Gets the oscillator type
		EOsc::Type GetType() const { return OscType; }

	protected:

		// Smooth out the saw-tooth discontinuities to improve aliasing
		SIGNALPROCESSING_API float PolySmooth(const float InPhase, const float InPhaseInc);

		// Current sign of the square mod, used for triangle wave generation
		float TriangleSign; 
		
		// Used to store state for triangular differentiator
		float DPW_z1; 

		// The pulse width base lerped
		FExponentialEase PulseWidthLerped;

		// The type of the oscillator
		EOsc::Type OscType;

		// a noise generator
		FWhiteNoise Noise;
	};

}
