// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	class FParam
	{
	public:
		FParam()
			: CurrentValue(0.0f)
			, StartingValue(0.0f)
			, TargetValue(0.0f)
			, DeltaValue(0.0f)
			, bIsInit(true)
		{}

		// Set the parameter value to the given target value over the given interpolation frames.
		inline void SetValue(const float InValue, const int32 InNumInterpFrames = 0)
		{
			TargetValue = InValue;
			if (bIsInit || InNumInterpFrames == 0)
			{
				bIsInit = false;
				StartingValue = TargetValue;
				CurrentValue = TargetValue;
				DeltaValue = 0.0f;
			}
			else
			{
				DeltaValue = (InValue - CurrentValue) / (float)InNumInterpFrames;
				StartingValue = CurrentValue;
			}
		}

		inline void Init()
		{
			bIsInit = true;
		}

		// Resets the delta value back to 0.0. To be called at beginning of callback render.
		inline void Reset()
		{
			DeltaValue = 0.0f;
			CurrentValue = TargetValue;
		}

		// Updates the parameter, assumes called in one of the frames.
		inline float Update()
		{
			CurrentValue += DeltaValue;
			return CurrentValue;
		}

		// Returns the current value, but does not update the value
		inline float GetValue() const
		{
			return CurrentValue;
		}

		// Returns the target value
		inline float GetTarget() const
		{
			return TargetValue;
		}

	private:
		float CurrentValue;
		float StartingValue;
		float TargetValue;
		float DeltaValue;
		bool bIsInit;
	};

}
