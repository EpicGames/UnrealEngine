// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad includes! Audio is not available in this module

#include "CoreMinimal.h"
#include "DSP/AudioFFT.h"
#include "DSP/FFTAlgorithm.h"
#include "Async/ParallelFor.h"
#include "Audio.h"

namespace Audio
{

	struct FPassiveFilterParams
	{
		enum class EClass
		{
			Butterworth,
			Chebyshev
		};

		enum class EType
		{
			Lowpass,
			Highpass
		};

		EClass Class;
		EType Type;

		int32 Order;
		float NormalizedCutoffFrequency;
		float UnitGain;

		bool bRemoveDC;
		bool bScaleByOffset;

		FPassiveFilterParams()
			: Class(EClass::Butterworth)
			, Type(EType::Lowpass)
			, Order(4)
			, NormalizedCutoffFrequency(0.8f)
			, UnitGain(1.0f)
			, bRemoveDC(false)
			, bScaleByOffset(true)
		{
		}
	};

	static float EvaluateChebyshevPolynomial(float FrequencyRatio, int32 Order)
	{
		if (FMath::IsNearlyEqual(FrequencyRatio, 1.0f))
		{
			return 1.0f;
		}

		switch (Order)
		{
		case 0:
		{
			return 1.0f;
		}
		case 1:
		{
			return FrequencyRatio;
		}
		case 2:
		{
			return 2.0f * FrequencyRatio * FrequencyRatio - 1.0f;
		}
		default:
		{
			// Rather than recursing,
			// we perform an iterative chebyshev polynomial evaluation,
			// since we are likely deep in the stack already:
			float Temp0 = 2.0f * FrequencyRatio * FrequencyRatio - 1.0f;
			float Temp1 = FrequencyRatio;
			
			// Start at an order of 2:
			float Result = Temp0;

			for (int32 CurrentOrder = 3; CurrentOrder <= Order; CurrentOrder++)
			{
				Result = (2.0f * FrequencyRatio * Temp0) - Temp1;
				Temp1 = Temp0;
				Temp0 = Result;
			}

			return Result;
		}
			break;
		}
	}

	static float GetGainForFrequency(float NormalizedFreq, const FPassiveFilterParams& InParams)
	{
		const float FrequencyRatio = (InParams.Type == FPassiveFilterParams::EType::Lowpass) ? (NormalizedFreq / InParams.NormalizedCutoffFrequency) : (InParams.NormalizedCutoffFrequency / NormalizedFreq);

		switch (InParams.Class)
		{
		case FPassiveFilterParams::EClass::Chebyshev:
		{
			const float ChebyshevPolynomial = EvaluateChebyshevPolynomial(FrequencyRatio, InParams.Order);
			return InParams.UnitGain / (FMath::Sqrt(1.0f + ChebyshevPolynomial * ChebyshevPolynomial));
			break;
		}
		default:
		case FPassiveFilterParams::EClass::Butterworth:
		{
			const float Denominator = FMath::Sqrt((1 + FMath::Pow(FrequencyRatio, InParams.Order * 2.0f)));
			return InParams.UnitGain / Denominator;
			break;
		}
		}
	}

	/**
	* This can be called on any TArrayView<float> whose length is a power of 2.
	*/
	static void Filter(TArrayView<float>& Signal, const FPassiveFilterParams& InParams)
	{
		if (!FMath::IsPowerOfTwo(Signal.Num()))
		{
			UE_LOGF(LogAudio, Error, "Error in Filter: if using TArrayView<float>, Signal's length should be power of 2.");
			return;
		}

		FFFTSettings FFTSettings;
		FFTSettings.Log2Size = CeilLog2(Signal.Num());
		FFTSettings.bArrays128BitAligned = true;

		TUniquePtr<IFFTAlgorithm> FFT = FFFTFactory::NewFFTAlgorithm(FFTSettings);
		if (!FFT.IsValid())
		{
			UE_LOGF(LogAudio, Error, "Failed to create an FFT Algorithm with log2size %d", FFTSettings.Log2Size);
			return;
		}

		float MaxVal = Signal[0];
		float MinVal = Signal[0];

		if (InParams.bScaleByOffset)
		{
			// First, find the max and min range:
			for (int32 Index = 0; Index < Signal.Num(); ++Index)
			{
				const float Value = Signal[Index];
				if (MaxVal < Value)
				{
					MaxVal = Value;
				}
				else if (Value < MinVal)
				{
					MinVal = Value;
				}
			}

			// Map to [-1.0f, 1.0f];
			float Scale = 1.0f / ((MaxVal - MinVal) * 2.0f);
			float Offset = MinVal - 1.0f;

			for (int32 Index = 0; Index < Signal.Num(); ++Index)
			{
				
				Signal[Index] = (Signal[Index] - Offset) * Scale;
			}
		}

		// Perform FFT on data:
		FAlignedFloatBuffer TimeData;
		TimeData.AddZeroed(FFT->NumInputFloats());
		FMemory::Memcpy(TimeData.GetData(), Signal.GetData(), Signal.Num() * sizeof(float));

		FAlignedFloatBuffer ComplexSpectrum;
		ComplexSpectrum.AddUninitialized(FFT->NumOutputFloats());

		FFT->ForwardRealToComplex(TimeData.GetData(), ComplexSpectrum.GetData());

		const float NumBins = FFT->NumOutputFloats() / 2;

		if (InParams.bRemoveDC)
		{
			ComplexSpectrum[0] = 0.f;
			ComplexSpectrum[1] = 0.f;
		}

		// Apply filter in parallel:
		ParallelFor(NumBins, [&](int32 Index)
		{
			float NormalizedFreq = ((float)Index) / (NumBins - 1);
			float Gain = GetGainForFrequency(NormalizedFreq, InParams);

			ComplexSpectrum[2 * Index] *= Gain;
			ComplexSpectrum[2 * Index + 1] *= Gain;
		});

		// Inverse FFT back into the signal:
		FFT->InverseComplexToReal(ComplexSpectrum.GetData(), TimeData.GetData());

		FMemory::Memcpy(Signal.GetData(), TimeData.GetData(), Signal.Num() * sizeof(float));

		// If we're scaling, map back to it's original range:
		if (InParams.bScaleByOffset)
		{
			// Map back to original values:
			float Scale = ((MaxVal - MinVal) * 2.0f);
			float Offset = -1.0f * (MinVal - 1.0f);

			for (int32 Index = 0; Index < Signal.Num(); ++Index)
			{

				Signal[Index] = (Signal[Index] - Offset) * Scale;
			}
		}
	}

	/**
	* Static function for applying a filter to any time series.
	*/
	static void Filter(TArray<float>& Signal, const FPassiveFilterParams& InParams)
	{
		if (FMath::IsPowerOfTwo(Signal.Num()))
		{
			TArrayView<float> SignalView(Signal);
			Filter(SignalView, InParams);
		}
		else
		{
			const int32 OriginalLength = Signal.Num();
			const int32 NumZerosRequired = FMath::RoundUpToPowerOfTwo(OriginalLength) - OriginalLength;
			Signal.AddZeroed(NumZerosRequired);
			TArrayView<float> SignalView(Signal);
			Filter(SignalView, InParams);
			Signal.SetNum(OriginalLength);
		}
	}
}
