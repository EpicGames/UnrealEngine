// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SignalProcessingModule.h"

namespace Audio
{
	/**
	* PerlinValueNoise1D. Computes 1 sample of "value" noise using the Perlin 1D function with N octaves.
	* Each "Octave" is a new layer and comes from the doubling of frequencies (through the noise), hence octave.
	* The higher frequencies have less amplitude, creating higher "detail".
	*
	* @param InX: The input "X" Value to hash into Perlin noise space. Time or an "X" coordinate (in the case of 1d) is often used.
	* @param InNumOctaves: The number of octaves (layers) to compute. Each octave has higher frequency and less amplitude. Value needs to be > 0.
	*/
	SIGNALPROCESSING_API float PerlinValueNoise1D(const float InX, const int32 InNumOctaves);

	/**
	* PerlinValueNoise1DBuffer. Whole buffer version of Value noise.
	*
	* @param InXBuffer: Buffer to compute value noise on.
	* @param InOffset: Offset each value is offset by. This is normally used to give different results when using multiple calls with the same input.
	* @param InNumOctaves: The number of octaves (layers) to compute. Each octave has higher frequency and less amplitude.
	* @param OutNoiseBuffer: Buffer to write the noise too. Should be as big as the input buffer.
	*/
	SIGNALPROCESSING_API void PerlinValueNoise1DBuffer(TArrayView<const float> InXBuffer, const float InOffset, const int32 InNumOctaves, TArrayView<float> OutNoiseBuffer);
}