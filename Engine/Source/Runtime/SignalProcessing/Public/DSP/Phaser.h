// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Delay.h"
#include "DSP/LFO.h"
#include "DSP/Filter.h"

namespace Audio
{

	class FPhaser
	{
	public:
		SIGNALPROCESSING_API FPhaser();
		SIGNALPROCESSING_API ~FPhaser();

		SIGNALPROCESSING_API void Init(const float SampleRate, const int32 InNumChannels);

		// Sets the phaser LFO rate
		SIGNALPROCESSING_API void SetFrequency(const float InFreqHz);

		// Sets the wet level of the phaser
		SIGNALPROCESSING_API void SetWetLevel(const float InWetLevel);

		// Sets the feedback of the phaser
		SIGNALPROCESSING_API void SetFeedback(const float InFeedback);

		// Sets the phaser LFO type
		SIGNALPROCESSING_API void SetLFOType(const ELFO::Type LFOType);

		// Sets whether or not to put the phaser in quadrature mode
		SIGNALPROCESSING_API void SetQuadPhase(const bool bQuadPhase);

		// Process an audio frame
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame);

		// Process an audio buffer.
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer);

	protected:
		SIGNALPROCESSING_API void ComputeNewCoefficients(const int32 ChannelIndex, const float LFOValue);

		// First-order all-pass filters in series
		static const int32 NumApfs = 6;
		static const int32 MaxNumChannels = 2;

		int32 ControlSampleCount;
		int32 ControlRate;
		float Frequency;
		float WetLevel;
		float Feedback;
		ELFO::Type LFOType;

		// 6 APFs per channel
		FBiquadFilter APFs[MaxNumChannels][NumApfs];
		FVector2D APFFrequencyRanges[NumApfs];

		// Feedback samples
		float FeedbackFrame[MaxNumChannels];

		FLFO LFO;

		// Number of channels actually used
		int32 NumChannels;

		bool bIsBiquadPhase;
	};

}
