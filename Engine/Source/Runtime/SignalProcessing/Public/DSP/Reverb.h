// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Delay.h"
#include "DSP/AllPassFilter.h"
#include "DSP/OnePole.h"

namespace Audio
{
	// forward
	class FWaveTableOsc;

	struct FEarlyReflectionsSettings
	{
		// Early reflections gain
		float Gain;

		// Delay between input signal and early reflections
		float PreDelayMsec;

		// Input sample bandwidth before entering early reflections
		float Bandwidth;

		// Early reflections decay (lower value is longer)
		float Decay;

		// Early reflection high frequency absorption factor
		float Absorption;

		FEarlyReflectionsSettings()
			: Gain(1.0f)
			, PreDelayMsec(0.0f)
			, Bandwidth(0.8f)
			, Decay(0.5f)
			, Absorption(0.7f)
		{}
	};

	// Basic implementation of a 4x4 Feedback Delay Network
	class FEarlyReflections
	{
	public:
		SIGNALPROCESSING_API FEarlyReflections();
		SIGNALPROCESSING_API virtual ~FEarlyReflections();

		SIGNALPROCESSING_API void Init(const int32 InSampleRate);

		// Sets the reverb settings, applies, and updates
		SIGNALPROCESSING_API void SetSettings(const FEarlyReflectionsSettings& InSettings);

		// Process the single audio frame
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InBuffer, const int32 InChannels, float* OutBuffer, const int32 OutChannels);

	protected:
		SIGNALPROCESSING_API void ApplySettings();
		SIGNALPROCESSING_API float ProcessDelayLine(const float InSample, FDelayAPF& InAPF, FOnePoleLPF& InLPF);

		FEarlyReflectionsSettings Settings;

		struct FFDNDelayData
		{
			// Pre-Delay for the input audio to the FDN reverberator
			FDelay PreDelay;

			// Input LPF for early reflections
			FOnePoleLPF InputLPF;

			// 4 feedback delay lines per input channel (stereo) which feed into each other using a normalized feedback matrix
			FDelayAPF APF[4];
			FOnePoleLPF LPF[4];

			float DelayLineInputs[4];
			float DelayLineOuputs[4];
		};

		float MatrixScaleFactor;

		// Stereo input/output for early reflections
		FFDNDelayData Data[2];
	};

	struct FPlateReverbSettings
	{
		// The settings for the early reflections part of the reverb
		FEarlyReflectionsSettings EarlyReflections;

		// Milliseconds for the predelay
		float LateDelayMsec;

		// Initial attenuation of audio after it leaves the predelay
		float LateGain;

		// Frequency bandwidth of audio going into input diffusers. 0.999 is full bandwidth
		float Bandwidth;

		// Amount of input diffusion (larger value results in more diffusion)
		float Diffusion;

		// The amount of high-frequency dampening in plate feedback paths
		float Dampening;

		// The amount of decay in the feedback path. Lower value is larger reverb time.
		float Decay;

		// The amount of diffusion in decay path. Larger values is a more dense reverb.
		float Density;

		// The amount of output wetness of the reverb as a whole
		float Wetness;

		FPlateReverbSettings()
			: LateDelayMsec(0.0f)
			, LateGain(0.0f)
			, Bandwidth(0.5f)
			, Diffusion(0.5f)
			, Dampening(0.5f)
			, Decay(0.5f)
			, Density(0.5f)
			, Wetness(0.5f)
		{}
	};

	class FPlateReverb
	{
	public:
		SIGNALPROCESSING_API FPlateReverb();
		SIGNALPROCESSING_API ~FPlateReverb();

		// Initialize the reverb with the given sample rate
		SIGNALPROCESSING_API void Init(const int32 InSampleRate);

		// Whether or not to enable late reflections
		SIGNALPROCESSING_API void EnableLateReflections(const bool bInEnableLateReflections);

		// Whether or not to enable late reflections
		SIGNALPROCESSING_API void EnableEarlyReflections(const bool bInEnableEarlyReflections);

		// Sets the reverb settings, applies, and updates
		SIGNALPROCESSING_API void SetSettings(const FPlateReverbSettings& InSettings);

		// Process the single audio frame
		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InBuffer, const int32 InChannels, float* OutBuffer, const int32 OutChannels);

	protected:

		SIGNALPROCESSING_API void ApplySettings();

		// Current parameter settings of reverb
		FPlateReverbSettings Settings;

		// Early reflections effect
		FEarlyReflections EarlyReflections;

		// Sample rate used for hard-coded delay line values
		static const int32 PresetSampleRate = 29761;

		// A simple pre-delay line to emulate large delays for late-reflections
		FDelay PreDelay;

		// Input diffusion
		FOnePoleLPF InputLPF;
		FDelayAPF APF1;
		FDelayAPF APF2;
		FDelayAPF APF3;
		FDelayAPF APF4;

		// Wave table oscillator, modulates input APF in plates
		TSharedPtr<FWaveTableOsc> LFO;

		// Plate data struction to organize each plate's delay lines and filters
		struct FPlate
		{
			FDelayAPF ModulatedAPF;
			FDelay Delay1;
			FOnePoleLPF LPF;
			FDelayAPF APF;
			FDelay Delay2;

			// Sample output of tank
			float PreviousSample;
			float ModulatedBaseDelayMsec;
			float ModulatedDeltaDelayMsec;

			FPlate()
				: PreviousSample(0.0f)
				, ModulatedBaseDelayMsec(0.0f)
				, ModulatedDeltaDelayMsec(0.0f)
			{}
		};

		// The plate data
		FPlate LeftPlate;
		FPlate RightPlate;

		// Tap points to read audio from various delay lines in plates
		static const int32 NumTaps = 7;

		float LeftTaps[NumTaps];
		float RightTaps[NumTaps];

		// Whether or not late reflections are enabled
		bool bEnableLateReflections;

		// Whether or not to enable early reflections
		bool bEnableEarlyReflections;
	};
}
