// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/IntegerDelay.h"
#include "DSP/LongDelayAPF.h"
#include "DSP/BufferOnePoleLPF.h"
#include "DSP/DynamicDelayAPF.h"
#include "DSP/LateReflectionsFast.h"
#include "DSP/EarlyReflectionsFast.h"

namespace Audio
{
	// Settings for plate reverb
	struct FPlateReverbFastSettings
	{
		// EQuadBehavior describes how reverb is generated when there are 5 or more output channels.
		enum class EQuadBehavior : uint8
		{
			// Only produce reverb in front left and front right.
			StereoOnly,
			// Produce reverb in front left and front right. Copy front left to rear left and front right to rear right.
			QuadMatched,
			// Produce reverb in front left and front right. Copy front left to rear right and front right to rear left.
			QuadFlipped
		};

		SIGNALPROCESSING_API FPlateReverbFastSettings();

		SIGNALPROCESSING_API bool operator==(const FPlateReverbFastSettings& Other) const;

		SIGNALPROCESSING_API bool operator!=(const FPlateReverbFastSettings& Other) const;

		// EarlyReflectionSettings controls the initial perceived echoes from a sound, modeling the first few
		// orders of reflections from a sound source to the listener's ears. 
		FEarlyReflectionsFastSettings EarlyReflections;

		// LateReflectionSettings controls the long tail diffused echo modeling the higher order reflections
		// from a sound source to the listener's ears. 
		FLateReflectionsFastSettings LateReflections;

		// Enables / Disables early reflections.
		bool bEnableEarlyReflections;

		// Enables / Disables late reflections.
		bool bEnableLateReflections;

		// Set how reverb module generates reverb when there are 5 or more output channels. 
		EQuadBehavior QuadBehavior;
	};

	// The Plate Reverb emulates the interactions between a sound, the listener and the space they share. Early reflections
	// are modeled using a feedback delay network while late reflections are modeled using a plate reverb. This 
	// class aims to support a flexible and pleasant sounding reverb balanced with computational efficiency. 
	class FPlateReverbFast {
		public:
			static SIGNALPROCESSING_API const FPlateReverbFastSettings DefaultSettings;
			
			// InMaxInternalBufferSamples sets the maximum number of samples used in internal buffers.
			SIGNALPROCESSING_API FPlateReverbFast(float InSampleRate, int32 InMaxInternalBufferSamples = 512, const FPlateReverbFastSettings& InSettings=DefaultSettings);

			SIGNALPROCESSING_API ~FPlateReverbFast();

			// Copies, clamps and applies settings.
			SIGNALPROCESSING_API void SetSettings(const FPlateReverbFastSettings& InSettings);

			SIGNALPROCESSING_API const FPlateReverbFastSettings& GetSettings() const;

			// Creates reverberated audio in OutSamples based upon InSamples
			// InNumChannels can be 1 or 2 channels.
			// OutSamples must be greater or equal to 2.
			SIGNALPROCESSING_API void ProcessAudio(const FAlignedFloatBuffer& InSamples, const int32 InNumChannels, FAlignedFloatBuffer& OutSamples, const int32 OutNumChannels);

			SIGNALPROCESSING_API void FlushAudio();

			// Clamp individual settings to values supported by this class.
			static SIGNALPROCESSING_API void ClampSettings(FPlateReverbFastSettings& InOutSettings);

		private:
			// Copy reverberated samples to interleaved output samples. Map channels according to internal settings.
			void InterleaveAndMixOutput(const FAlignedFloatBuffer& InFrontLeftSamples, const FAlignedFloatBuffer& InFrontRightSamples, FAlignedFloatBuffer& OutSamples, const int32 OutNumChannels);

			void ApplySettings();


			float SampleRate;

			FPlateReverbFastSettings Settings;
			FEarlyReflectionsFast EarlyReflections;
			FLateReflectionsFast LateReflections;

			FAlignedFloatBuffer FrontLeftLateReflectionsSamples;
			FAlignedFloatBuffer FrontRightLateReflectionsSamples;
			FAlignedFloatBuffer FrontLeftEarlyReflectionsSamples;
			FAlignedFloatBuffer FrontRightEarlyReflectionsSamples;
			FAlignedFloatBuffer FrontLeftReverbSamples;
			FAlignedFloatBuffer FrontRightReverbSamples;
			FAlignedFloatBuffer LeftAttenuatedSamples;
			FAlignedFloatBuffer RightAttenuatedSamples;
			FAlignedFloatBuffer ScaledInputBuffer;

	};
}
