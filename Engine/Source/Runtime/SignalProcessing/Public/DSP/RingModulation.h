// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Osc.h"
#include "DSP/MultithreadedPatching.h"

namespace Audio
{
	// Ring modulation effect
	// https://en.wikipedia.org/wiki/Ring_modulation
	class FRingModulation
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FRingModulation();

		// Destructor
		SIGNALPROCESSING_API ~FRingModulation();

		// Initialize the equalizer
		SIGNALPROCESSING_API void Init(const float InSampleRate, const int32 InNumChannels);

		// The type of modulation
		SIGNALPROCESSING_API void SetModulatorWaveType(const EOsc::Type InType);

		// Set the ring modulation frequency
		SIGNALPROCESSING_API void SetModulationFrequency(const float InModulationFrequency);

		// Set the ring modulation depth
		SIGNALPROCESSING_API void SetModulationDepth(const float InModulationDepth);

		// Sets that the modulation buffer is external
		SIGNALPROCESSING_API void SetExternalPatchSource(Audio::FPatchOutputStrongPtr InPatch);

		// Set the dry level of the ring modulation
		void SetDryLevel(const float InDryLevel) { DryLevel = InDryLevel; }

		// Set the wet level of the ring modulation
		SIGNALPROCESSING_API void SetWetLevel(const float InWetLevel);

		// Process audio buffer
		SIGNALPROCESSING_API void ProcessAudio(const float* InBuffer, const int32 InNumSamples, float* OutBuffer);

	private:
		SIGNALPROCESSING_API void UpdateScale();

		Audio::FPatchOutputStrongPtr Patch;
		Audio::FOsc Osc;
		float ModulationFrequency;
		float ModulationDepth;
		float DryLevel;
		float WetLevel;
		float Scale;
		int32 NumChannels;
		TArray<float> ModulationBuffer;
	};

}
