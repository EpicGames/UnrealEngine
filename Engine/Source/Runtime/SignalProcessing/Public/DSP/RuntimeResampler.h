// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/BufferVectorOperations.h"
#include "DSP/Dsp.h"
#include "DSP/MultichannelBuffer.h"

#define UE_API SIGNALPROCESSING_API

namespace Audio
{
	/** Linear interpolating resampler which supports any number of channels of audio, interleaved or deinterleaved.
	 To support different types of input and output objects, add a new "Process..." method which wraps around calls to
	 ProcessAudioInternal(). */
	class FRuntimeResampler
	{
	public:

		static UE_API const float MaxFrameRatio;
		static UE_API const float MinFrameRatio;

		/** Construct a linear resampler.
		 *
		 * @param InNumChannel - Number of audio channels in input and output buffers.
		 */
		UE_API FRuntimeResampler(int32 InNumChannels);

		/** Sets the number of input frames to read per an output frame. 0.5 is
		 * half speed, 1.f is normal speed, 2.0 is double speed.
		 *
		 * @param InRatio - Ratio of input frames consumed per an output frame produced.
		 * @param InNumFramesToInterpolate - Number of output frames over which
		 *                                   to interpolate the frame ratio.
		 */
		UE_API void SetFrameRatio(float InRatio, int32 InNumFramesToInterpolate = 0);

		/** Returns the minimum number of input frames needed to produce the desired
		 * number of output frames given the current state of the resampler.
		 *
		 * @param InNumOutputFrames - The desired number of output frames.
		 * @return The minimum number of input frames.
		 */
		UE_API int32 GetNumInputFramesNeededToProduceOutputFrames(int32 InNumOutputFrames) const;

		/** Returns the maximum number of output frames that can be produced
		 * from the provided number of input frames given the current state of the resampler.
		 *
		 * @param InNumInputFrames - The number of provided input frames.
		 * @return The maximum number of output frames.
		 */
		UE_API int32 GetNumOutputFramesProducedByInputFrames(int32 InNumInputFrames) const;

		/** Consumes audio from the input buffer and produces audio in the output buffer.
		 * The desired number of frames to produce is determined by the output audio buffer
		 * size. For the desired number of samples to be produced, the input audio must have the minimum
		 * number of frames needed to produce the output frames (see `GetNumInputFramesNeededToProduceOutputFrames(...)`).
		 * Input samples which are no longer needed are removed from the input buffer.
		 *
		 * @param InAudio - Multichannel circular buffer of input audio.
		 * @param OutAudio - Multichannel buffer of output audio.
		 * FMultichannelBuffer
		 *
		 * @return Actual number of frames produced.
		 */
		UE_API int32 ProcessCircularBuffer(FMultichannelCircularBuffer& InAudio, FMultichannelBuffer& OutAudio);

		/** Consumes audio from the input buffer and produces audio in the output buffer.
		 * The desired number of frames to produce is determined by the output audio buffer
		 * size. For the desired number of samples to be produced, the input audio must have the minimum
		 * number of frames needed to produce the output frames (see `GetNumInputFramesNeededToProduceOutputFrames(...)`).
		 * Input samples which are no longer needed are removed from the input buffer.
		 *
		 * @param InAudio - Multichannel circular buffer of input audio.
		 * @param OutAudio - Multichannel buffer of output audio.
		 * FMultichannelBuffer
		 *
		 * @return Actual number of frames produced.
		 */
		UE_API int32 ProcessCircularBuffer(FMultichannelCircularBuffer& InAudio, FMultichannelBufferView& OutAudio);

		/** Consumes audio in an interleaved channel format from the input buffer and produces audio in the output buffer.
		 * The desired number of frames to produce is determined by the output audio buffer
		 * size. For the desired number of samples to be produced, the input audio must have the minimum
		 * number of frames needed to produce the output frames (see `GetNumInputFramesNeededToProduceOutputFrames(...)`).
		 *
		 * @param InAudio - Multichannel interleaved audio to use as input.
		 * @param OutAudio - Buffer to receive multichannel interleaved output audio.
		 * @param OutNumInputFramesConsumed - Receives the actual number of frames of input consumed.
		 * @param OutNumOutputFramesProduced - Receives the actual number of frames of output produced.
		 */
		UE_API void ProcessInterleaved(TArrayView<const float> Input, TArrayView<float> Output, int32& OutNumInputFramesConsumed, int32& OutNumOutputFramesProduced);

		/** Reset the internal state of the resampler. Call this if the next audio data to be submitted
		 * is unrelated to the audio that has been submitted previously.
		*/
		UE_API void Reset(int32 InNumChannels);

	private:
		int64 MapOutputFrameToInputFrameFP(int32 InOutputFrameIndex) const;

		struct ResamplingParameters
		{
			TConstArrayView<const float*> InputAudio;
			int32 InputStride = 0;
			int32 NumInputFrames = -1;
			TConstArrayView<float*> OutputAudio;
			int32 OutputStride = 0;
			int32 NumOutputFrames = -1;

			int32 OutNumInputFramesConsumed = 0;
			int32 OutNumOutputFramesProduced = 0;
		};

		void ProcessAudioInternal(ResamplingParameters& Parameters);

		void ProcessAudioInBatches(ResamplingParameters& Parameters, int32 MaxSamplesPerBatch);

		/** Perform a quick memcpy if appropriate rather than resampling. Returns true if the memcpy was performed, false otherwise. */
		bool DoDirectCopy(ResamplingParameters& Parameters);

		void GenericResamplingCore(const ResamplingParameters& Parameters, int32& OutInputFrameIndexFP, uint32& OutInputFrameRatioFP, int32 OutputSampleIndex, int32 OutputEndIndex);
		void MonoResamplingCore(const float* Input, float* Output, int32& OutInputFrameIndexFP, uint32& OutInputFrameRatioFP, int32 OutputSampleIndex, int32 OutputEndIndex);
		void StereoInterleavedResamplingCore(const float* Input, float* Output, int32& OutInputFrameIndexFP, uint32& OutInputFrameRatioFP, int32 OutputSampleIndex, int32 OutputEndIndex);
		void StereoDeinterleavedResamplingCore(const float* InputLeft, const float* InputRight, float* OutputLeft, float* OutputRight, int32& OutInputFrameIndexFP, uint32& OutInputFrameRatioFP, int32 OutputSampleIndex, int32 OutputEndIndex);

		template<typename OutputBufferType, typename OutputChannelType>
		int32 ProcessCircularBufferGeneric(FMultichannelCircularBuffer& InAudio, OutputBufferType& OutAudio);

		static const int32 FPScale;
		static const float FPScaleFloat;

		int32 CurrentInputFrameIndexFP = 0;
		uint32 CurrentFrameRatioFP = 0;
		uint32 TargetFrameRatioFP = 0;
		int32 FrameRatioFrameDeltaFP = 0;
		int32 NumFramesToInterpolate = 0;

		TArray<float, TInlineAllocator<2>> PreviousFrame;
		TArray<const float*, TInlineAllocator<2>> TempInputPointers;
		TArray<float*, TInlineAllocator<2>> TempOutputPointers;
	};
}

#undef UE_API
