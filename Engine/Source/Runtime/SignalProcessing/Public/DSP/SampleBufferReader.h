// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "SampleBuffer.h"

namespace Audio
{
	namespace ESeekType
	{
		enum Type
		{
			FromBeginning,
			FromCurrentPosition,
			FromEnd
		};
	}

	class FSampleBufferReader
	{
	public:
		SIGNALPROCESSING_API FSampleBufferReader();
		SIGNALPROCESSING_API ~FSampleBufferReader();

		SIGNALPROCESSING_API void Init(const int32 InSampleRate);

		// This must be a completely loaded buffer. This buffer reader doesn't OWN the buffer memory.
		SIGNALPROCESSING_API void SetBuffer(const int16* InBufferPtr, const int32 InNumBufferSamples, const int32 InNumChannels, const int32 InBufferSampleRate);

		// Seeks the buffer the given time in seconds. Returns true if succeeded.
		SIGNALPROCESSING_API void SeekTime(const float InTimeSec, const ESeekType::Type InSeekType = ESeekType::FromBeginning, const bool bWrap = true);

		// Sets the pitch of the buffer reader. Can be negative. Will linearly interpolate over the given time value.
		SIGNALPROCESSING_API void SetPitch(const float InPitch, const float InterpolationTimeSec = 0.0f);

		// Puts the wave reader into scrub mode
		SIGNALPROCESSING_API void SetScrubMode(const bool bInIsScrubMode);
		
		// Sets the scrub width. The sound will loop between the scrub width region and the current frame
		SIGNALPROCESSING_API void SetScrubTimeWidth(const float InScrubTimeWidthSec);

		// Returns the number of channels of this buffer.
		int32 GetNumChannels() const { return BufferNumChannels; }

		// Returns the number of frames of the buffer.
		int32 GetNumFrames() const { return BufferNumFrames; }

		// Returns the current playback position in seconds
		float GetPlaybackProgress() const { return PlaybackProgress; }

		// Generates the next block of audio. Returns true if it's no longer playing (reached end of the buffer and not set to wrap)
		SIGNALPROCESSING_API bool Generate(float* OutAudioBuffer, const int32 NumFrames, const int32 OutChannels, const bool bWrap = false);

		// Whether or not the buffer reader has a buffer
		bool HasBuffer() const { return BufferPtr != nullptr; }

		// Clears current buffer and resets state
		SIGNALPROCESSING_API void ClearBuffer();

	protected:

		SIGNALPROCESSING_API float GetSampleValueForChannel(const int32 Channel);
		SIGNALPROCESSING_API void UpdateScrubMinAndMax();
		SIGNALPROCESSING_API void UpdateSeekFrame();
		SIGNALPROCESSING_API float GetSampleValue(const int16* InBuffer, const int32 SampleIndex);

		const int16* BufferPtr;
		int32 BufferNumSamples;
		int32 BufferNumFrames;
		int32 BufferSampleRate;
		int32 BufferNumChannels;
		int32 FadeFrames;
		float FadeValue;
		float FadeIncrement;

		float DeviceSampleRate;

		// The current frame alpha
		float BasePitch;
		float PitchScale;
		Audio::FLinearEase Pitch;

		int32 CurrentFrameIndex;
		int32 NextFrameIndex;
		double AlphaLerp;

		double CurrentBufferFrameIndexInterpolated;

		float PlaybackProgress;

		double ScrubAnchorFrame;
		double ScrubMinFrame;
		double ScrubMaxFrame;

		double ScrubWidthFrames;
		float CurrentSeekTime;
		float CurrentScrubWidthSec;

		ESeekType::Type CurrentSeekType;
		bool bWrap;
		bool bIsScrubMode;
		bool bIsFinished;
	};

}

