// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	class ISampleRateConverter
	{
	public:
		virtual ~ISampleRateConverter() = default;

		virtual void Init(const float InSampleRateRatio, const int32 InNumChannels) = 0;

		// Sets the sample rate ratio and the number of frames to interpolate from the current sample rate ratio
		virtual void SetSampleRateRatio(const float InSampleRateRatio, const int32 NumInterpolationFrames = 0) = 0;

		/**
		*	Given an input buffer, the output buffer is fully sample-rate converted.
		*	@param	InBuffer		Input audio buffer
		*	@param	InNumSamples	Number of input samples.
		*	@param  OutBuffer		The output buffer of converted audio.
		*	@return Number of frames generated.
		*/
		virtual int32 ProcessFullbuffer(const int16* InBuffer, const int32 InNumSamples, TArray<float>& OutBuffer) = 0;
		virtual int32 ProcessFullbuffer(const float* InBuffer, const int32 InNumSamples, TArray<float>& OutBuffer) = 0;

		/**
		*	Process chunks of audio at a time. Saves state so can properly handle buffer boundaries. Use with Reset() to 
		*	reset internal state before starting realtime SRC.
		*	@param	BufferChunk		Input audio buffer chunk
		*	@param	NumInputSamples	Number of input samples.
		*	@param	RequestedFrames	Number of frames requested to process
		*	@param  OutBuffer		The output buffer of converted audio.
		*	@return Number of frames generated.
		*/
		virtual int32 ProcessChunk(const int16* BufferChunk, const int32 NumInputSamples, const int32 RequestedFrames, TArray<float>& OutBuffer) = 0;
		virtual int32 ProcessChunk(const float* BufferChunk, const int32 NumInputSamples, const int32 RequestedFrames, TArray<float>& OutBuffer) = 0;

		// Creates a new instance
		static SIGNALPROCESSING_API ISampleRateConverter* CreateSampleRateConverter();
	};

}

