// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	/** 
	* FOsc
	* Direct-form sinusoid oscillator. 
	* Created with a biquad filter (using only feedback coefficients) with poles directly on unit circle in z-plane.
	* Setting frequency uses current filter state to compute initial conditions to avoid pops when changing frequency.
	* Extremely cheap to run but expensive to set new frequencies. Good for test tones.
	*/
	class FSineOsc
	{
	public:
		/** Constructor */
		SIGNALPROCESSING_API FSineOsc();

		/** Non-default constructor */
		SIGNALPROCESSING_API FSineOsc(const int32 InSampleRate, const float InFrequencyHz, const float Scale = 1.0f, const float Add = 0.0f);

		/** Virtual destructor */
		SIGNALPROCESSING_API virtual ~FSineOsc();

		/** Initialize the oscillator with a sample rate and new frequency. Must be called before playing oscillator. */
		SIGNALPROCESSING_API void Init(const int32 InSampleRate, const float InFrequencyHz, const float Scale = 1.0f, const float Add = 0.0f);

		/** Sets the scale of the oscillator. */
		SIGNALPROCESSING_API void SetScale(const float InScale);

		/** Sets the scale of the oscillator. */
		SIGNALPROCESSING_API void SetAdd(const float InAdd);

		/** Sets the frequency of the oscillator in Hz (based on sample rate). Performs initial condition calculation to avoid pops. */
		SIGNALPROCESSING_API void SetFrequency(const float InFrequencyHz);

		/** Returns the current frequency. */
		SIGNALPROCESSING_API float GetFrequency() const;

		/** Generates the next sample of the oscillator. */
		SIGNALPROCESSING_API float ProcessAudio();

	protected:
		int32 SampleRate;
		float FrequencyHz;
		float B1;	// Biquad feedback coefficients
		float B2;
		float Yn_1;	// y(n - 1)
		float Yn_2;	// y(n - 2)
		float Scale;
		float Add;
	};


}
