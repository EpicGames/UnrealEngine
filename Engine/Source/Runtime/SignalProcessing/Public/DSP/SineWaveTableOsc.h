// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	class FWaveTableOsc;


	// A sinusoidal wave table oscillator class
	class FSineWaveTableOsc
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FSineWaveTableOsc();

		// Virtual Destructor
		SIGNALPROCESSING_API virtual ~FSineWaveTableOsc();

		// Initialize the wave table oscillator
		SIGNALPROCESSING_API void Init(const float InSampleRate, const float InFrequencyHz, const float InPhase);

		// Sets the sample rate of the oscillator.
		SIGNALPROCESSING_API void SetSampleRate(const float InSampleRate);

		// Resets the wave table read indices.
		SIGNALPROCESSING_API void Reset();

		// Sets the frequency of the wave table oscillator.
		SIGNALPROCESSING_API void SetFrequencyHz(const float InFrequencyHz);

		// Returns the frequency of the wave table oscillator.
		float GetFrequencyHz() const { return FrequencyHz; }
		
		// Sets the phase of the wave table oscillator.
		SIGNALPROCESSING_API void SetPhase(const float InPhase);

		// Processes the wave table and fills a buffer
		SIGNALPROCESSING_API void Generate(float* OutBuffer, const int32 NumSamples);

		// The static sinusoidal wave table 
		static SIGNALPROCESSING_API const TArray<float>& GetWaveTable();

	protected:
		SIGNALPROCESSING_API void UpdatePhaseIncrement();

		// The wave table buffer
		const TArray<float>& WaveTableBuffer = GetWaveTable();
		
		// The frequency of the output (given the sample rate)
		float FrequencyHz = 440.0f;

		// The sample rate of the oscillator
		float SampleRate = 48000.0f;

		// Read index
		float ReadIndex = 0.0f;

		// The phase increment (based on frequency)
		float PhaseIncrement = 0.0f;

		// The initial/cached phase, will be clamped to range 0.0 - 1.0f
		float InitialPhase = 0.0f;

		// The instantaneous phase, clamped to range 0.0 - 1.0f
		float InstantaneousPhase = 0.0f;
	};
}
