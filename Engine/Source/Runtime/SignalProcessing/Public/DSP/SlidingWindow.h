// Copyright Epic Games, Inc. All Rights Reserved.

/** Sliding Window implementation which enables ranged for loop iteration over 
 * sequential input buffers of varying length.
 */
#pragma once

#include "CoreMinimal.h"


namespace Audio
{
	// Forward delcaration
	template <typename InSampleType>
	class TSlidingWindow;


	/** TSlidingBuffer
	 *
	 * TSlidingBuffer defines the window size and hop size of the sliding window, and
	 * it stores any samples needed to produce additional windows.
	 *
	 * TSlidingBuffer should be used in conjunction with the TSlidingWindow, TScopedSlidingWindow 
	 * or TAutoSlidingWindow classes.
	 */
	template <typename InSampleType>
	class TSlidingBuffer
	{
		public:

			/**
			 * Constructs a TSlidingBuffer with a constant window and hop size
			 */
			TSlidingBuffer(const int32 InNumWindowSamples, const int32 InNumHopSamples)
			: NumWindowSamples(InNumWindowSamples)
			, NumHopSamples(InNumHopSamples)
			, NumUnderflowSamples(0)
			{
				check(NumWindowSamples >= 1);
				check(NumHopSamples >= 1);
			}

			TSlidingBuffer()
			: TSlidingBuffer(1, 1)
			{
			}

			TSlidingBuffer(const TSlidingBuffer&) = default;
			TSlidingBuffer(TSlidingBuffer&&) = default;

			TSlidingBuffer& operator=(const TSlidingBuffer& InOther) = default;
			TSlidingBuffer& operator=(TSlidingBuffer&& InOther) = default;

			/** Returns the number of samples in a window. */
			int32 GetNumWindowSamples() const
			{
				return NumWindowSamples;
			}

			/** Returns the number of samples between windows. */
			int32 GetNumHopSamples() const
			{
				return NumHopSamples;
			}

			/** Returns the number of underflow samples (when hopsize > windowsize) */
			int32 GetNumUnderflowSamples() const
			{
				return NumUnderflowSamples;
			}

			TArrayView<const InSampleType> GetStorage() const
			{
				return StorageBuffer;
			}

			/**
			 * StoreForFutureWindows stores the necessary samples from InBuffer which will
			 * be needed for future windows. It ignores all values in InBuffer which can
			 * already be composed as a complete window.
			 */
			void StoreForFutureWindows(TArrayView<const InSampleType> InBuffer)
			{
				if (NumUnderflowSamples > 0)
				{
					// Consume some underflow samples from the storage buffer.
					if (StorageBuffer.Num() > NumUnderflowSamples)
					{
						StorageBuffer.RemoveAt(0, NumUnderflowSamples);
						NumUnderflowSamples = 0;
					}
					else
					{
						NumUnderflowSamples -= StorageBuffer.Num();
						StorageBuffer.Reset();
					}
				}

				// Total number of samples starting at beginning of first window generated from this buffer
				int32 NumSamples = InBuffer.Num() + StorageBuffer.Num() - NumUnderflowSamples;

				if (NumSamples < 0)
				{
					// No windows generated, but some underflow samples accounted for
					NumUnderflowSamples -= InBuffer.Num();
				}
				else if (NumSamples < NumWindowSamples)
				{
					// No windows generated, but we should store data for future windows.
					int32 NumToCopy = NumSamples - StorageBuffer.Num();
					int32 InBufferIndex = InBuffer.Num() - NumToCopy;
					StorageBuffer.Append(&InBuffer.GetData()[InBufferIndex], NumToCopy);

					// All underflow samples accounted for.
					NumUnderflowSamples = 0;
				}
				else
				{
					// Calculate number of windows generated from samples in StorageBuffer and InBuffer.
					int32 NumWindowsGenerated = (NumSamples - NumWindowSamples) / NumHopSamples + 1;

					// Calculate number of samples to keep for future windows
					int32 NumRemainingSamples = NumSamples - (NumWindowsGenerated * NumHopSamples);

					if (NumRemainingSamples > InBuffer.Num())
					{
						// Need to keep some samples from storage buffer and from InBuffer
						int32 NumToKeep = NumRemainingSamples - InBuffer.Num();
						int32 NumToRemove = StorageBuffer.Num() - NumToKeep;

						if (NumToRemove > 0)
						{
							// May need to remove soem from the storage buffer
							StorageBuffer.RemoveAt(0, NumToRemove);
						}

						StorageBuffer.Append(InBuffer.GetData(), InBuffer.Num());
					}
					else if (NumRemainingSamples > 0)
					{
						// Only need to keep samples from InBuffer. Can discard samples in StorageBuffer
						StorageBuffer.Reset(NumRemainingSamples);
						StorageBuffer.AddUninitialized(NumRemainingSamples);

						int32 NewBufferCopyIndex = InBuffer.Num() - NumRemainingSamples;
						FMemory::Memcpy(StorageBuffer.GetData(), &InBuffer.GetData()[NewBufferCopyIndex], NumRemainingSamples * sizeof(InSampleType));
					}
					else
					{
						// This occurs when HopSize > WindowSize. We have hopped to the next window, but don't have enough samples
						// to account for the hop. We track the number of underflow samples to make sure they are consumed before
						// the next window starts.
						NumUnderflowSamples = -NumRemainingSamples;
						StorageBuffer.Reset(0);
					}
				}
			}

			/**
			 * Resets the internal storage.
			 */
			void Reset()
			{
				StorageBuffer.Reset();
				NumUnderflowSamples = 0;
			}

		private:
			// NumWindowSamples describes the number of samples in a window.
			int32 NumWindowSamples;

			// NumHopSamples describes the number of samples between adjacent windows.
			int32 NumHopSamples;

			// Stores samples from previous calls which are still needed for future buffers
			TArray<InSampleType> StorageBuffer;

			// When HopSize is greater than WindowSize a situation can occur where we need 
			// to account for hop samples that we have not yet ingested.  
			int32 NumUnderflowSamples;
	};

	/** TSlidingWindow
	 *
	 * TSlidingWindow allows windows of samples to be iterated over with STL like iterators. 
	 *
	 */
	template <typename InSampleType>
	class TSlidingWindow
	{
		friend class TSlidingWindowIterator;

		protected:
			// Accessed from friendship with TSlidingBuffer
			TArrayView<const InSampleType> StorageBuffer;

			// New buffer passed in.
			TArrayView<const InSampleType> NewBuffer;

			// Copied from TSlidingBuffer
			const int32 NumWindowSamples;

			// Copied from TSlidingBuffer
			const int32 NumHopSamples;

		private:

			int32 MaxReadIndex;
			int32 NumUnderflowSamples;

		public:

			/** TSlidingWindowIterator
			 *
			 * An forward iterator which slides a window over the given buffers.
			 */
			template<typename InAllocator = FDefaultAllocator>
			class TSlidingWindowIterator
			{
					const TSlidingWindow SlidingWindow;

					// Samples in window will be copied into this array.
					TArray<InSampleType, InAllocator>& WindowBuffer;

					// Index into array for reading out data.
					int32 ReadIndex;

				public:

					// Sentinel value marking that the last possible window has been generated.
					static const int32 ReadIndexEnd = INDEX_NONE;

					/**
					 * Construct an iterator over a sliding window.
					 */
					TSlidingWindowIterator(const TSlidingWindow& InSlidingWindow, TArray<InSampleType, InAllocator>& OutWindowBuffer, int32 InReadIndex)
					:	SlidingWindow(InSlidingWindow)
					,	WindowBuffer(OutWindowBuffer)
					,	ReadIndex(InReadIndex)
					{
						if (ReadIndex > SlidingWindow.MaxReadIndex)
						{
							ReadIndex = ReadIndexEnd;
						}
					}

					/**
					 * Increment sliding window iterator forward.
					 */
					TSlidingWindowIterator operator++()
					{
						if (ReadIndex != ReadIndexEnd)
						{
							ReadIndex += SlidingWindow.NumHopSamples;
							if (ReadIndex > SlidingWindow.MaxReadIndex)
							{
								ReadIndex = ReadIndexEnd;
							}
						}

						return *this;
					}

					/**
					 * Check whether iterators are equal. TSlidingWindowIterators derived from different
					 * TSlidingWindows should not be compared.
					 */
					bool operator!=(const TSlidingWindowIterator& Other) const
					{
						return ReadIndex !=	Other.ReadIndex;
					}

					/**
					 * Access array of windowed data currently pointed to by iterator.
					 */
					TArray<InSampleType, InAllocator>& operator*()
					{
						if (ReadIndex != ReadIndexEnd)
						{
							// Resize output window
							WindowBuffer.Reset(SlidingWindow.NumWindowSamples);
							WindowBuffer.AddUninitialized(SlidingWindow.NumWindowSamples);

							int32 SamplesFilled = 0;

							if (ReadIndex < SlidingWindow.StorageBuffer.Num())
							{
								// The output window overlaps the storage buffer. Copy appropriate samples from the storage buffer.
								int32 SamplesToCopy = SlidingWindow.StorageBuffer.Num() - ReadIndex;
								FMemory::Memcpy(WindowBuffer.GetData(), &SlidingWindow.StorageBuffer.GetData()[ReadIndex], SamplesToCopy * sizeof(InSampleType));
								SamplesFilled += SamplesToCopy;
							}

							if (SamplesFilled < SlidingWindow.NumWindowSamples)
							{
								// The output window overlaps the new buffer. Copy appropriate samples from the new buffer.
								int32 NewBufferIndex = ReadIndex - SlidingWindow.StorageBuffer.Num() + SamplesFilled;
								int32 NewBufferRemaining = SlidingWindow.NewBuffer.Num() - NewBufferIndex;
								int32 SamplesToCopy = FMath::Min(SlidingWindow.NumWindowSamples - SamplesFilled, NewBufferRemaining);
								
								if (SamplesToCopy > 0)
								{
									FMemory::Memcpy(&WindowBuffer.GetData()[SamplesFilled], &SlidingWindow.NewBuffer.GetData()[NewBufferIndex], SamplesToCopy * sizeof(InSampleType));

									SamplesFilled += SamplesToCopy;
								}
							}

							if (SamplesFilled < SlidingWindow.NumWindowSamples)
							{
								// The output window still needs more samples (due to zeropadding & flushing), so set zeros.
								int32 SamplesToZeropad = SlidingWindow.NumWindowSamples - SamplesFilled;
								FMemory::Memset(&WindowBuffer.GetData()[SamplesFilled], 0, sizeof(InSampleType) * SamplesToZeropad);
							}
						}
						else
						{
							// Empty window if past end sliding window. ReadIndex == ReadIndexEnd
							WindowBuffer.Reset();
						}

						return WindowBuffer;
					}
			};

			/**
			 * TSlidingWindow constructor
			 *
			 * InSlidingBuffer Holds the previous samples which were not completely used in previous sliding windows.  It also defines the window and hop size.
			 * InNewBuffer Holds new samples which have not yet been ingested by the InSlidingBuffer.
			 * bDoFlush Controls whether zeros to the final output windows until all possible windows with data from InNewBuffer have been covered.
			 */
			TSlidingWindow(const TSlidingBuffer<InSampleType>& InSlidingBuffer, TArrayView<const InSampleType> InNewBuffer, bool bDoFlush)
			:	StorageBuffer(InSlidingBuffer.GetStorage())
			,	NewBuffer(InNewBuffer)
			,	NumWindowSamples(InSlidingBuffer.GetNumWindowSamples())
			,	NumHopSamples(InSlidingBuffer.GetNumHopSamples())
			,	MaxReadIndex(0)
			,	NumUnderflowSamples(InSlidingBuffer.GetNumUnderflowSamples())
			{
				// Total samples to be slid over.
				int32 NumSamples = NewBuffer.Num() + StorageBuffer.Num();

				if (bDoFlush && (NumSamples > 0))
				{
					// If flushing, calculate the number of samples to zeropad
					int32 NumZeroPad = 0;

					if (NumSamples < NumWindowSamples)
					{
						NumZeroPad = NumWindowSamples - NumSamples;
					}
					else
					{
						// Determine number of windows 
						int32 NumWindowsGenerated = (NumSamples - NumWindowSamples - NumUnderflowSamples) / NumHopSamples + 1;
						int32 NumRemaining = NumSamples - (NumWindowsGenerated * NumHopSamples);
						if (NumRemaining > 0)
						{
							NumZeroPad = NumWindowSamples - NumRemaining;
						}
					}

					NumSamples += NumZeroPad;
				}

				MaxReadIndex = NumSamples - NumWindowSamples;

				if (MaxReadIndex < 0)
				{
					MaxReadIndex = TSlidingWindowIterator<>::ReadIndexEnd;
				}
			}

			virtual ~TSlidingWindow() = default;

			/**
			 * Creates STL like iterator which slides over samples.
			 *
			 * OutWindowBuffer Used to construct the TSlidingWindowIterator. The iterator will populate the window with samples when the * operator is called.
			 */
			template<typename InAllocator = FDefaultAllocator>
			TSlidingWindowIterator<InAllocator> begin(TArray<InSampleType, InAllocator>& OutWindowBuffer) const
			{
				if (MaxReadIndex == TSlidingWindowIterator<>::ReadIndexEnd)
				{
					return end<InAllocator>(OutWindowBuffer);
				}
				// Set the starting read index to NumUnderflowSamples to account for samples that still need to be consumed.
				return TSlidingWindowIterator<InAllocator>(*this, OutWindowBuffer, NumUnderflowSamples);
			}

			/**
			 * Creates STL like iterator denotes the end of the sliding window.
			 *
			 * OutWindowBuffer Used to construct the TSlidingWindowIterator. The iterator will populate the window with samples when the * operator is called.
			 */
			template<typename InAllocator = FDefaultAllocator>
			TSlidingWindowIterator<InAllocator> end(TArray<InSampleType, InAllocator>& OutWindowBuffer) const
			{
				return TSlidingWindowIterator<InAllocator>(*this, OutWindowBuffer, TSlidingWindowIterator<InAllocator>::ReadIndexEnd);
			}
	};

	/** TScopedSlidingWindow
	 * 
	 * TScopedSlidingWindow provides a sliding window iterator interface over arrays. When TScopedSlidingWindow is destructed,
	 * it calls StoreForFutureWindow(...) on the TSlidingBuffer passed into the constructor.
	 *
	 */
	template <class InSampleType>
	class TScopedSlidingWindow : public TSlidingWindow<InSampleType>
	{
			// Do not allow copying or moving since that may cause the destructor to be called inadvertently.
			TScopedSlidingWindow(TScopedSlidingWindow const &) = delete;
			void operator=(TScopedSlidingWindow const &) = delete;
			TScopedSlidingWindow(TScopedSlidingWindow&&) = delete;
			TScopedSlidingWindow& operator=(TScopedSlidingWindow&&) = delete;

			TSlidingBuffer<InSampleType>& SlidingBuffer;
		public:

			/**
			 * TScopedSlidingWindow constructor
			 *
			 * InSlidingBuffer Holds the previous samples which were not completely used in previous sliding windows.  It also defines the window and hop size.
			 * InNewBuffer Holds new samples which have not yet been ingested by the InSlidingBuffer.
			 * bDoFlush Controls whether zeros to the final output windows until all possible windows with data from InNewBuffer have been covered.
			 */
			TScopedSlidingWindow(TSlidingBuffer<InSampleType>& InSlidingBuffer, TArrayView<const InSampleType> InNewBuffer, bool bDoFlush = false)
			:	TSlidingWindow<InSampleType>(InSlidingBuffer, InNewBuffer, bDoFlush)
			,	SlidingBuffer(InSlidingBuffer)
			{}

			/**
			 * Calls InSlidingBuffer.StoreForFutureWindows(InNewBuffer).
			 */
			virtual ~TScopedSlidingWindow()
			{
				SlidingBuffer.StoreForFutureWindows(TSlidingWindow<InSampleType>::NewBuffer);
			}
	};

	/** TAutoSlidingWindow
	 * 
	 * TAutoSlidingWindow enables use of a sliding window within a range-based for loop.
	 *
	 * Example:
	 *
	 * void ProcessAudio(TSlidingBuffer<float>& SlidingBuffer, const TArray<float>& NewSamples)
	 * {
	 * 		TArray<float> WindowData;
	 * 		TAutoSlidingWindow<float> SlidingWindow(SlidingBuffer, NewSamples, WindowData);
	 *
	 * 		for (TArray<float>& Window : SlidingWindow)
	 * 		{
	 * 			... audio processing on single window here
	 * 		}
	 * }
	 *
	 * int main()
	 * {
	 * 		int32 NumWindowSamples = 4;
	 * 		int32 NumHopSamples = 2;
	 * 		TSlidingBuffer<float> SlidingBuffer(NumWindowSamples, NumHopSamples);
	 *
	 * 		TArray<float> Buffer1({1, 2, 3, 4, 5, 6, 7});
	 *
	 * 		ProcessAudio(SlidingBuffer, Buffer1);
	 *
	 * 		TArray<float> Buffer2({8, 9, 10, 11});
	 *
	 * 		ProcessAudio(SlidingBuffer, Buffer2);
	 * }
	 */
	template <typename InSampleType, typename InAllocator = FDefaultAllocator>
	class TAutoSlidingWindow : public TScopedSlidingWindow<InSampleType>
	{
		TArray<InSampleType, InAllocator>& WindowBuffer;

		typedef typename TSlidingWindow<InSampleType>::template TSlidingWindowIterator<InAllocator> TAutoSlidingWindowIterator;

		public:
			/**
			 * TAutoSlidingWindow constructor
			 *
			 * InSlidingBuffer Holds the previous samples which were not completely used in previous sliding windows.  It also defines the window and hop size.
			 * InNewBuffer Holds new samples which have not yet been ingested by the InSlidingBuffer.
			 * OutWindow is shared by all iterators created by calling begin() or end().
			 * bDoFlush Controls whether zeros to the final output windows until all possible windows with data from InNewBuffer have been covered.
			 */
			TAutoSlidingWindow(TSlidingBuffer<InSampleType>& InBuffer, TArrayView<const InSampleType> InNewBuffer, TArray<InSampleType, InAllocator>& OutWindow, bool bDoFlush = false)
			:	TScopedSlidingWindow<InSampleType>(InBuffer, InNewBuffer, bDoFlush)
			,	WindowBuffer(OutWindow)
			{}

			/**
			 * Creates STL like iterator which slides over samples.
			 *
			 * This iterator maintains a reference to the OutWindow passed into the constructor. That array will be manipulated when the iterator's * operator is called. 
			 */
			TAutoSlidingWindowIterator begin() 
			{
				return TSlidingWindow<InSampleType>::template begin<InAllocator>(WindowBuffer);
			}

			/**
			 * Creates STL like iterator denotes the end of the sliding window.
			 *
			 * This iterator maintains a reference to the OutWindow passed into the constructor. That array will be manipulated when the iterator's * operator is called. 
			 */
			TAutoSlidingWindowIterator end()
			{
				return TSlidingWindow<InSampleType>::template end<InAllocator>(WindowBuffer);
			}
	};
}
