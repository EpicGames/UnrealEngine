// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "DSP/AudioFFT.h"
#include "DSP/BufferVectorOperations.h"
#include "SampleBuffer.h"
#include "Async/AsyncWork.h"

namespace Audio
{
	class IFFTAlgorithm;
	class FSpectrumAnalyzer;

	struct FSpectrumAnalyzerSettings
	{
		// Actual FFT size used. For FSpectrumAnalyzer, we never zero pad the input buffer.
		enum class EFFTSize : uint16
		{
			Default = 512,
			TestingMin_8 = 8,
			Min_64 = 64,
			Small_256 = 256,
			Medium_512 = 512,
			Large_1024 = 1024,
			VeryLarge_2048 = 2048,
			TestLarge_4096 = 4096
		};

		EWindowType WindowType;
		EFFTSize FFTSize;

		/**
			* Hop size as a percentage of FFTSize.
			* 1.0 indicates a full hop.
			* Keeping this as 0.0 will use whatever hop size
			* can be used for WindowType to maintain COLA.
			*/
		float HopSize;

		FSpectrumAnalyzerSettings()
			: WindowType(EWindowType::Hann)
			, FFTSize(EFFTSize::Default)
			, HopSize(0.0f)
		{}
	};


	/** Settings for band extractor. */
	struct FSpectrumBandExtractorSettings
	{
		/** Metric for output band values. */
		enum class EMetric : uint8
		{
			/** Return the magnitude spectrum value. */
			Magnitude,

			/** Return the power spectrum value. */
			Power,

			/** Return the decibel spectrum value. Decibels are calculated
			 * with 0dB equal to 1.f magnitude.  */
			Decibel
		};

		/** Metric used to calculate return value. */
		EMetric Metric = EMetric::Decibel;


		/** If the metric is Decibel, this is the minimum decibel value allowed. */
		float DecibelNoiseFloor = -40.f; 

		/** 
		 * If true, all values are scaled and clamped between 0.0 and 1.f. In the 
		 * case of Decibels, 0.0 corresponds to the decibel noise floor and 1.f to 0dB.
		 * If bDoAutoRange is true, then values are relatively to recent maximum and minimums
		 * regardless of the metric used.
		 */
		bool bDoNormalize = true;

		/** 
		 * If true and bDoNormalize is true, then values will be scaled between 0 and 1
		 * based upon relatively recent minimum and maximum values. 
		 */
		bool bDoAutoRange = true;

		/** Time in seconds for autorange to reach 99% of a smaller range. */
		float AutoRangeReleaseTimeInSeconds = 30.f;

		/** Time in seconds for autorange to reach 99% of a larger range. */
		float AutoRangeAttackTimeInSeconds = 1.f;
	};

	/** Settings describing the spectrum used for in the band extractor. */
	struct FSpectrumBandExtractorSpectrumSettings
	{
		/** Sample rate of audio */
		float SampleRate; 

		/** Size of fft used in spectrum analyzer */
		int32 FFTSize; 

		/** Forward scaling of FFT used in spectrum analyzer */
		EFFTScaling FFTScaling; 

		/** Window used when perform FFT */
		EWindowType WindowType;

		FSpectrumBandExtractorSpectrumSettings()
		:	SampleRate(48000.f)
		,	FFTSize(1024)
		,	FFTScaling(EFFTScaling::None)
		,	WindowType(EWindowType::None)
		{}

		/** Compare whether two settings structures are equal. */
		bool operator==(const FSpectrumBandExtractorSpectrumSettings& Other) const
		{
			bool bIsEqual = ((SampleRate == Other.SampleRate)
					&& (FFTSize == Other.FFTSize)
					&& (FFTScaling == Other.FFTScaling)
					&& (WindowType == Other.WindowType));
			return bIsEqual;
		}

		/** Compare whether two settings structures are not equal. */
		bool operator!=(const FSpectrumBandExtractorSpectrumSettings& Other) const
		{
			return !(*this == Other);
		}
	};

	/** Interface for spectrum band extractors.
	 *
	 *  The SpectrumBandExtractor allows for band information
	 *  to be maintained across multiple calls to retrieve bands values.
	 *  By maintaining band information across multiple calls, some intermediate 
	 *  values can be cached to speed up the operation.
	 */
	class ISpectrumBandExtractor
	{
		public:

			enum class EBandType : uint8
			{
				/** Look up band value via nearest FFT band. */
				NearestNeighbor,

				/** Band value is calculated by lerping between FFT bands adjacent to center frequency. */
				Lerp,

				/** Band value is calculated by fitting quadratic to 3 adjancent FFT bands and solving for center frequency. */
				Quadratic,

				/** Band value is calculated by a weighted sum of a window of fft bands around center frequency. The window size is determined by the QFactor of the band. */
				ConstantQ
			};

			/** Settings for a single band */
			struct FBandSettings
			{
			 	/** Type of band to extract. */
				EBandType Type = EBandType::ConstantQ;

			 	/**  Frequency of interest in hz. */
				float CenterFrequency = 0.f;

			 	/** 
				 * QFactor is only applicable for the ConstantQ band type. 
				 * QFactor = CenterFreq / BandWidth. Eg. A small QFactor results in a wide band.
				 */
				float QFactor = 10.0f;
			};

			virtual ~ISpectrumBandExtractor() = default;

			/** Sets and updates the settings for the band extractor */
			virtual void SetSettings(const FSpectrumBandExtractorSettings& InSettings) = 0;

			/** Set the settings and update cached internal values if needed */
			virtual void SetSpectrumSettings(const FSpectrumBandExtractorSpectrumSettings& InSettings) = 0;
			
			/** Removes all added bands. */
			virtual void RemoveAllBands() = 0;

			/** Returns the total number of bands. */
			virtual int32 GetNumBands() const = 0;

			/** Adds a band to extract based on the given settings. */
			virtual void AddBand(const FBandSettings& InSettings) = 0;

			/** Extract the bands from a complex frequency buffer.
			 *
			 * @param InComplexBuffer - Buffer of complex frequency data from a FFT.
			 * @param InTimestamp - A timestamp associated with the input complex buffer.
			 * @param OutValues - Array to store output bands.
			 */
			virtual void ExtractBands(const FAlignedFloatBuffer& InComplexBuffer, double InTimestamp, TArray<float>& OutValues) = 0;

			/** Creates a ISpectrumBandExtractor. */
			static SIGNALPROCESSING_API TUniquePtr<ISpectrumBandExtractor> CreateSpectrumBandExtractor(const FSpectrumBandExtractorSettings& InSettings);
	};

	/**
	 * This class locks an input buffer (for writing) and an output buffer (for reading).
	 * Uses triple buffering semantics.
	 */
	class FSpectrumAnalyzerBuffer
	{
	public:
		FSpectrumAnalyzerBuffer();
		FSpectrumAnalyzerBuffer(int32 InNum);

		void Reset(int32 InNum);

		// Input. Used on analysis thread to lock a buffer to write to.
		FAlignedFloatBuffer& StartWorkOnBuffer();

		// When calling stop work on buffer, also set timestmap associated with buffer.
		void StopWorkOnBuffer(double InTimestamp);
		
		// Output. Used to lock the most recent buffer we analyzed.
		const FAlignedFloatBuffer& LockMostRecentBuffer() const;

		// Output. Used to lock the most recent buffer we analyzed.
		// OutTimestamp is populated with the timestamp associated with the buffer wehn StopWorkOnBuffer is called.
		const FAlignedFloatBuffer& LockMostRecentBuffer(double& OutTimestamp) const;
		void UnlockBuffer();

	private:
		TArray<FAlignedFloatBuffer> ComplexBuffers;
		TArray<double> Timestamps;

		// Private functions. Either increments or decrements the respective counter,
		// based on which index is currently in use. Mutually locked.
		void IncrementInputIndex();
		void IncrementOutputIndex();

		int32 OutputIndex;
		int32 InputIndex;

		// This mutex is locked when we increment either the input or output index.
		mutable FCriticalSection BufferIndicesCriticalSection;
	};

	class FSpectrumAnalysisAsyncWorker 
	{

	public:
		FSpectrumAnalysisAsyncWorker(TWeakPtr<FSpectrumAnalyzer, ESPMode::ThreadSafe> InAnalyzer, bool bInUseLatestAudio)
			: AnalyzerWeakPtr(InAnalyzer)
			, bUseLatestAudio(bInUseLatestAudio)
			, bIsAbandoned(false)
		{}

		inline TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FSpectrumAnalysisAsyncWorker, STATGROUP_ThreadPoolAsyncTasks);
		}

		void DoWork();

		bool CanAbandon()
		{
			return true;
		}

		void Abandon();

	private:
		FSpectrumAnalysisAsyncWorker() = delete;

		TWeakPtr<FSpectrumAnalyzer, ESPMode::ThreadSafe> AnalyzerWeakPtr;
		bool bUseLatestAudio;
		bool bIsAbandoned;

		FCriticalSection NonAbandonableSection;
	};

	typedef FAsyncTask<FSpectrumAnalysisAsyncWorker> FSpectrumAnalyzerTask;

	/**
	 * Class built to be a rolling spectrum analyzer for arbitrary, monaural audio data.
	 * Class is meant to scale accuracy with CPU and memory budgets.
	 * Typical usage is to either call PushAudio() and then PerformAnalysisIfPossible immediately afterwards,
	 * or have a separate thread call PerformAnalysisIfPossible().
	 */
	class FSpectrumAnalyzer 
	{

	public:
		// Peak interpolation method. If the EFFTSize is small but will be densely sampled,
		// it's worth using a linear or quadratic interpolation method.
		enum class EPeakInterpolationMethod : uint8
		{
			NearestNeighbor,
			Linear,
			Quadratic
		};

		// If an instance is created using the default constructor, Init() must be called before it is used.
		SIGNALPROCESSING_API FSpectrumAnalyzer();

		// If an instance is created using either of these constructors, Init() is not neccessary.
		SIGNALPROCESSING_API FSpectrumAnalyzer(float InSampleRate);
		SIGNALPROCESSING_API FSpectrumAnalyzer(const FSpectrumAnalyzerSettings& InSettings, float InSampleRate);

		virtual ~FSpectrumAnalyzer() = default;

		// Initialize sample rate of analyzer if not known at time of construction
		SIGNALPROCESSING_API void Init(float InSampleRate);
		SIGNALPROCESSING_API void Init(const FSpectrumAnalyzerSettings& InSettings, float InSampleRate);

		// Update the settings used by this Spectrum Analyzer. Safe to call on any thread, but should not be called every tick.
		SIGNALPROCESSING_API void SetSettings(const FSpectrumAnalyzerSettings& InSettings);

		// Get the current settings used by this Spectrum Analyzer.
		SIGNALPROCESSING_API void GetSettings(FSpectrumAnalyzerSettings& OutSettings);

		// Samples magnitude (linearly) for a given frequency, in Hz.
		SIGNALPROCESSING_API float GetMagnitudeForFrequency(float InFrequency, EPeakInterpolationMethod InMethod = EPeakInterpolationMethod::Linear);
		SIGNALPROCESSING_API float GetNormalizedMagnitudeForFrequency(float InFrequency, EPeakInterpolationMethod InMethod = EPeakInterpolationMethod::Linear);

		// Samples phase for a given frequency, in Hz.
		SIGNALPROCESSING_API float GetPhaseForFrequency(float InFrequency, EPeakInterpolationMethod InMethod = EPeakInterpolationMethod::Linear);

		// Return array of bands using spectrum band extractor.
		SIGNALPROCESSING_API void GetBands(ISpectrumBandExtractor& InExtractor, TArray<float>& OutValues);

		// You can call this function to ensure that you're sampling the same window of frequency data,
		// Then call UnlockOutputBuffer when you're done.
		// Otherwise, GetMagnitudeForFrequency and GetPhaseForFrequency will always use the latest window
		// of frequency data.
		SIGNALPROCESSING_API void LockOutputBuffer();
		SIGNALPROCESSING_API void UnlockOutputBuffer();
		
		// Push audio to queue. Returns false if the queue is already full.
		SIGNALPROCESSING_API bool PushAudio(const TSampleBuffer<float>& InBuffer);
		SIGNALPROCESSING_API bool PushAudio(const float* InBuffer, int32 NumSamples);

		// Thread safe call to perform actual FFT. Returns true if it performed the FFT, false otherwise.
		// If bUseLatestAudio is set to true, this function will flush the entire input buffer, potentially losing data.
		// Otherwise it will only consume enough samples necessary to perform a single FFT.
		SIGNALPROCESSING_API bool PerformAnalysisIfPossible(bool bUseLatestAudio = false);

		// Returns false if this instance of FSpectrumAnalyzer was constructed with the default constructor 
		// and Init() has not been called yet.
		SIGNALPROCESSING_API bool IsInitialized();
	
	private:


		// Called on analysis thread.
		SIGNALPROCESSING_API void ResetSettings();

		// Called in GetMagnitudeForFrequency and GetPhaseForFrequency.
		SIGNALPROCESSING_API void PerformInterpolation(const FAlignedFloatBuffer& InComplexBuffer, EPeakInterpolationMethod InMethod, const float InFreq, float& OutReal, float& OutImag);

		// Cached current settings. Only actually used in ResetSettings().
		FSpectrumAnalyzerSettings CurrentSettings;
		volatile bool bSettingsWereUpdated;

		volatile bool bIsInitialized;

		float SampleRate;

		// Cached window that is applied prior to running the FFT.
		FWindow Window;
		int32 FFTSize;
		int32 HopInSamples;
		EFFTScaling FFTScaling;

		FAlignedFloatBuffer AnalysisTimeDomainBuffer;
		FThreadSafeCounter SampleCounter; 
		TCircularAudioBuffer<float> InputQueue;
		FSpectrumAnalyzerBuffer FrequencyBuffer;

		// if non-null, owns pointer to locked frequency vector we're using.
		double LockedBufferTimestamp;
		const FAlignedFloatBuffer* LockedFrequencyVector;


		TUniquePtr<IFFTAlgorithm> FFT;
	};

	class FSpectrumAnalyzerScopeLock
	{
	public:
		FSpectrumAnalyzerScopeLock(FSpectrumAnalyzer* InAnalyzer)
			: Analyzer(InAnalyzer)
		{
			Analyzer->LockOutputBuffer();
		}

		~FSpectrumAnalyzerScopeLock()
		{
			Analyzer->UnlockOutputBuffer();
		}

	private:
		FSpectrumAnalyzer* Analyzer;
	};

	// SpectrumAnalyzer for computing spectrum in async task. 
	class FAsyncSpectrumAnalyzer 
	{

		FAsyncSpectrumAnalyzer(const FSpectrumAnalyzer&) = delete;
		FAsyncSpectrumAnalyzer(FSpectrumAnalyzer&&) = delete;

		FAsyncSpectrumAnalyzer& operator=(const FSpectrumAnalyzer&) = delete;
		FAsyncSpectrumAnalyzer& operator=(FSpectrumAnalyzer&&) = delete;


	public:
		SIGNALPROCESSING_API FAsyncSpectrumAnalyzer();
		// If an instance is created using either of these constructors, Init() is not neccessary.
		SIGNALPROCESSING_API FAsyncSpectrumAnalyzer(float InSampleRate);
		SIGNALPROCESSING_API FAsyncSpectrumAnalyzer(const FSpectrumAnalyzerSettings& InSettings, float InSampleRate);

		SIGNALPROCESSING_API virtual ~FAsyncSpectrumAnalyzer();

		// Initialize sample rate of analyzer if not known at time of construction
		SIGNALPROCESSING_API void Init(float InSampleRate);
		SIGNALPROCESSING_API void Init(const FSpectrumAnalyzerSettings& InSettings, float InSampleRate);

		// Returns false if this instance of FSpectrumAnalyzer was constructed with the default constructor 
		// and Init() has not been called yet.
		SIGNALPROCESSING_API bool IsInitialized();

		// Update the settings used by this Spectrum Analyzer. Safe to call on any thread, but should not be called every tick.
		SIGNALPROCESSING_API void SetSettings(const FSpectrumAnalyzerSettings& InSettings);

		// Get the current settings used by this Spectrum Analyzer.
		SIGNALPROCESSING_API void GetSettings(FSpectrumAnalyzerSettings& OutSettings);

		// Samples magnitude (linearly) for a given frequency, in Hz.
		SIGNALPROCESSING_API float GetMagnitudeForFrequency(float InFrequency, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod = FSpectrumAnalyzer::EPeakInterpolationMethod::Linear);
		SIGNALPROCESSING_API float GetNormalizedMagnitudeForFrequency(float InFrequency, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod = FSpectrumAnalyzer::EPeakInterpolationMethod::Linear);

		// Samples phase for a given frequency, in Hz.
		SIGNALPROCESSING_API float GetPhaseForFrequency(float InFrequency, FSpectrumAnalyzer::EPeakInterpolationMethod InMethod = FSpectrumAnalyzer::EPeakInterpolationMethod::Linear);

		// Return array of bands using spectrum band extractor.
		SIGNALPROCESSING_API void GetBands(ISpectrumBandExtractor& InExtractor, TArray<float>& OutValues);

		// You can call this function to ensure that you're sampling the same window of frequency data,
		// Then call UnlockOutputBuffer when you're done.
		// Otherwise, GetMagnitudeForFrequency and GetPhaseForFrequency will always use the latest window
		// of frequency data.
		SIGNALPROCESSING_API void LockOutputBuffer();
		SIGNALPROCESSING_API void UnlockOutputBuffer();
		
		// Push audio to queue. Returns false if the queue is already full.
		SIGNALPROCESSING_API bool PushAudio(const TSampleBuffer<float>& InBuffer);
		SIGNALPROCESSING_API bool PushAudio(const float* InBuffer, int32 NumSamples);

		// Thread safe call to perform actual FFT. Returns true if it performed the FFT, false otherwise.
		// If bUseLatestAudio is set to true, this function will flush the entire input buffer, potentially losing data.
		// Otherwise it will only consume enough samples necessary to perform a single FFT.
		SIGNALPROCESSING_API bool PerformAnalysisIfPossible(bool bUseLatestAudio = false);


		// Thread safe call to perform actual FFT. Returns true if it performed the FFT, false otherwise.
		// If bUseLatestAudio is set to true, this function will flush the entire input buffer, potentially losing data.
		// Otherwise it will only consume enough samples necessary to perform a single FFT.
		SIGNALPROCESSING_API bool PerformAsyncAnalysisIfPossible(bool bUseLatestAudio = false);
	

	private:

		TSharedRef<FSpectrumAnalyzer, ESPMode::ThreadSafe> Analyzer;

		// This is used if PerformAsyncAnalysisIfPossible is called
		TUniquePtr<FSpectrumAnalyzerTask> AsyncAnalysisTask;
	};

	class FAsyncSpectrumAnalyzerScopeLock
	{
	public:
		FAsyncSpectrumAnalyzerScopeLock(FAsyncSpectrumAnalyzer* InAnalyzer)
			: Analyzer(InAnalyzer)
		{
			Analyzer->LockOutputBuffer();
		}

		~FAsyncSpectrumAnalyzerScopeLock()
		{
			Analyzer->UnlockOutputBuffer();
		}

	private:
		FAsyncSpectrumAnalyzer* Analyzer;
	};
}
