// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/UnrealMemory.h"
#include "Math/Matrix.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"


/*
	Azimuth angle is measured CCW from front.
	Elevation is 0 horizontal plane, + is above horizontal plane
*/

class FSphericalHarmonicCalculator
{
public:
	enum UE_DEPRECATED(5.7, "Not necessary, please use a raw index for ACN values") AmbiChanNumber
	{
		/* 0th-Order */	ACN_0 = 0,
		/* 1st-Order */	ACN_1, ACN_2, ACN_3,
		/* 2nd-Order */	ACN_4, ACN_5, ACN_6, ACN_7, ACN_8,
		/* 3rd-Order */	ACN_9, ACN_10, ACN_11, ACN_12, ACN_13, ACN_14, ACN_15,
		/* 4th-Order */	ACN_16, ACN_17, ACN_18, ACN_19, ACN_20, ACN_21, ACN_22, ACN_23, ACN_24,
		/* 5th-Order */	ACN_25, ACN_26, ACN_27, ACN_28, ACN_29, ACN_30, ACN_31, ACN_32, ACN_33, ACN_34, ACN_35,
	};

	static SIGNALPROCESSING_API void ComputeSoundfieldChannelGains(const int32 Order, const float Azimuth, const float Elevation, float* OutGains);
	static SIGNALPROCESSING_API void ComputeSoundfieldChannelGains(const int32 Order, const float Azimuth, const float Elevation, TArrayView<float> OutGains);

	static SIGNALPROCESSING_API void GenerateFirstOrderRotationMatrixGivenRadians(const float RotXRadians, const float RotYRadians, const float RotZRadians, FMatrix& OutMatrix);
	static SIGNALPROCESSING_API void GenerateFirstOrderRotationMatrixGivenDegrees(const float RotXDegrees, const float RotYDegrees, const float RotZDegrees, FMatrix& OutMatrix);

	template<typename T>
	static void AdjustUESphericalCoordinatesForAmbisonics(UE::Math::TVector2<T>& InOutVector)
	{
		InOutVector.X = -(InOutVector.X - HALF_PI);
		Swap(InOutVector.X,InOutVector.Y);
	}
	
	static int64 OrderToNumChannels(const int32 InOrder)
	{
		check(InOrder >= 0);
		return (InOrder+1) * (InOrder+1);
	}

	static SIGNALPROCESSING_API bool IndexToOrderAndDegree(const int32 InChannel, int32& OutOrder, int32& OutDegree);
	

	static SIGNALPROCESSING_API void NormalizeGains(TArrayView<float> Gains);
};
