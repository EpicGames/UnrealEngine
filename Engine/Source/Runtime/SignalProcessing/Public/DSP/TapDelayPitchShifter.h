// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "DSP/Delay.h"

namespace Audio
{
	// Class which reads doppler-shifted tap from a delay buffer 
	class FTapDelayPitchShifter
	{
	public:
		static constexpr float MinDelayLength = 10.0f;
		static constexpr float MaxDelayLength = 100.0f;
		static constexpr float MaxAbsPitchShiftInOctaves = 6.0f;
		
		// Constructor
		SIGNALPROCESSING_API FTapDelayPitchShifter();

		// Virtual Destructor
		SIGNALPROCESSING_API virtual ~FTapDelayPitchShifter();

		// Initialization of the internal delay with given sample rate, pitch shift in semitones, and delay length in milliseconds
		SIGNALPROCESSING_API void Init(const float InSampleRate, const float InPitchShift, const float InDelayLength);

		// Sets the internal delay line length.
		SIGNALPROCESSING_API void SetDelayLength(const float InDelayLength);

		// Sets the pitch shift in semitones
		SIGNALPROCESSING_API void SetPitchShift(const float InPitchScaleSemitones);

		// Sets the pitch shift in terms of a sample rate ratio
		SIGNALPROCESSING_API void SetPitchShiftRatio(const float InPitchShiftRatio);
		
		// Reads the next sample tap from the input delay buffer
		SIGNALPROCESSING_API float ReadDopplerShiftedTapFromDelay(const Audio::FDelay& InDelayBuffer, const float ReadOffsetMilliseconds = 0.0f);

		// Process whole block of audio reading to and writing from delay line from input buffer
		SIGNALPROCESSING_API void ProcessAudio(Audio::FDelay& InDelayBuffer, const float* InAudioBuffer, const int32 InNumFrames, float* OutAudioBuffer);
		
	private:
		SIGNALPROCESSING_API void UpdatePhasorPhaseIncrement();
		
		float SampleRate = 0.0f;
		Audio::FExponentialEase CurrentDelayLength;
		float CurrentTargetDelayLength = 0.0f;
		float CurrentPitchShift = 0.0f;
		float CurrentPitchShiftRatio = 1.0f;
		float PhasorPhase = 0.0f;
		float PhasorPhaseIncrement = 0.0f;
	};

}
