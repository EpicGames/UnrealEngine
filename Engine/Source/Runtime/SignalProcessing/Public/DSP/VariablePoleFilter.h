// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Filter.h"

namespace Audio
{
	enum class EFilterOrder : uint32
	{
		TwoPole = 1,
		FourPole,
		SixPole,
		EightPole,
	};

	class FVariablePoleFilter
	{
	public:
		int32 NumFilters = 0;
		int32 NumChannels = 0;


		EBiquadFilter::Type Type = EBiquadFilter::Lowpass;
		FVariablePoleFilter() {};

		SIGNALPROCESSING_API void Init(EFilterOrder InOrder
				  , const float InSampleRate
				  , const int32 InNumChannels
				  , const float InFrequency
				  , const EBiquadFilter::Type InType
				  , const float InBandwidth = 2.f
				  , const float InGain = 0.f);

		SIGNALPROCESSING_API void SetParams(const EBiquadFilter::Type InFilterType, const float InCutoffFrequency, const float InBandwidth = 2.f, const float InGain = 0.f);

		SIGNALPROCESSING_API void ProcessAudioFrame(const float* InFrame, float* OutFrame);
		SIGNALPROCESSING_API void ProcessAudioBuffer(const float* InFrame, float* OutFrame, const int32 NumSamples);

		SIGNALPROCESSING_API void Reset();

		const TArray<FBiquadFilter>& GetFilters() const { return Filters; }

	private:
		TArray<FBiquadFilter> Filters;
	};
}
