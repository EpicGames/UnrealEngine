// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Math/Vector.h"

namespace Audio
{
	// Generic panner interface.
	struct IDiscretePanner
	{
		virtual ~IDiscretePanner() = default;
		struct FSpeaker
		{
												// Listener relative in degrees
			float AzimuthDegrees = 0.0f;		// -180 - 180.  (0 forward).
			float ElevationDegrees = 0.0f;		// -180 - 180.  (0 ground).
			FName ChannelID = {};				// Unique id of speaker.
			int32 ChannelIndex = INDEX_NONE;	// Index in the format. 
		};
		struct FInputParams
		{
												// Listener relative in degrees
			float AzimuthDegrees=0.0f;			// -180 - 180.  (0 forward).
			float ElevationDegrees=0.0f;		// -180 - 180.  (0 ground).
			bool bAllowAzimuthMirroring = false;// Optionally mirror the azimuth into the northern hemisphere.
												// in the case where there's no speaker data defined in the south.
		};
		struct FPanResult
		{
			FName ChannelID={};					// Unique ID of the Speaker Name.
			int32 ChannelIndex=INDEX_NONE;		// Index in the speaker in format. 
			float Gain=0.0f;
		};
		struct FOutputParams
		{
			static constexpr int32 MaxResults = 3;
			TArray<FPanResult, TInlineAllocator<MaxResults>> Results;
			void Reset() { Results.Empty(); }
		};
		virtual bool ComputeGains(const FInputParams& InParams, FOutputParams& OutResults) const = 0;

		static FVector3f SphericalToUnitCartesian(const float InAzimuth, const float InElevation);
	};
	
	class FVBap2D final : public IDiscretePanner
	{
	public:
		SIGNALPROCESSING_API explicit FVBap2D(const TArray<FSpeaker>& InSpeaker);
		virtual ~FVBap2D() override = default;
		virtual bool ComputeGains(const FInputParams& InParams, FOutputParams& OutResults) const override;
	private:

		struct FSpan 
		{
			FSpeaker A;		// Start.
			FSpeaker B;		// End
			FVector2f DirA;
			FVector2f DirB;
		};
		TArray<FSpan> Spans;
		float MaxAzimuth = TNumericLimits<float>::Lowest();
		float MinAzimuth = TNumericLimits<float>::Max();		
		
		void BuildSpans(const TArray<FSpeaker>& InSpeakers);
		int32 FindBestSpan(const float InAzimuthDegrees) const;
	};
}
