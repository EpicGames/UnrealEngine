// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"

namespace Audio
{
	/**
		FVectorLinearResampler

		Performs a basic linear resampling using SIMD for optimization.

		Advancing is done in fixed point - 65536 is a 1.0f sample rate.

		Stereo requires a deinterleaved format.

		The design is based on a "pull" model, assuming you need to fill a buffer from a source,
		rather than computing how much output you would need to resample a given input. If you need
		to do that, just run the resampler in chunks until you drain the input, appending to the 
		output as you go.

		Usage:
			FVectorLinearResample Resampler = {};

			uint32 FixedSampleRate = (uint32)(1.0f * 65536);

			// the example source buffer is assumed to be stereo deinterleaved with the
			// right channel directly after the left.
			float* SourceFrames = 0;//<data from elsewhere>;
			uint32 SourceBufferFrameCount = 0;//<# of frames - i.e.len(SourceFrames) / (sizeof(float) * 2) since stereo>;
			uint32 SourceBufferFloatsToRightChannel = SourceBufferFrameCount; // right channel directly after the left

			const uint32 OutputChunkFrames = 512;
			while (SourceBufferFrameCount)
			{
				// stereo output chunk. Or pointer to wherever you're mixing to.
				float OutputFrames[OutputChunkFrames * 2];

				// find out how much input we need for the resampler to be able to generate
				// our chunk.
				uint32 SourceFramesNeeded = Resampler.SourceFramesNeeded(OutputChunkFrames, FixedSampleRate);

				// make sure we have that available.
				if (SourceFramesNeeded <= SourceBufferFrameCount)
				{
					// direct resample
					uint32 SourceFramesConsumed = Resampler.ResampleStereo(OutputChunkFrames, FixedSampleRate,
						SourceFrames, SourceBufferFloatsToRightChannel,
						OutputFrames, OutputChunkFrames);

					// do something with output_frames

					// advance the input
					SourceFrames += SourceFramesConsumed;
					SourceBufferFrameCount -= SourceFramesConsumed;
					continue;
				}

				// here we need to append zeroes as we don't have enough input to fill.
				// usually need a temp buffer unless you can make dramatic assumptions about
				// your source buffer.
				float* TempSource = FMemory::Malloc(sizeof(float) * 2 * SourceFramesNeeded);
				float* TempSourceLeft = TempSource;
				float* TempSourceRight = TempSource + SourceFramesNeeded;

				uint32 ZeroedFramesNeeded = SourceFramesNeeded - SourceBufferFrameCount;

				// left				
				FMemory::Memcpy(TempSourceLeft, SourceFrames, SourceBufferFrameCount * sizeof(float));

				// left tail
				FMemory::Memset(TempSourceLeft + SourceBufferFrameCount, 0, ZeroedFramesNeeded * sizeof(float));

				// right				
				FMemory::Memcpy(TempSourceRight, SourceFrames + SourceBufferFloatsToRightChannel, SourceBufferFrameCount * sizeof(float));

				// right tail
				FMemory::Memset(TempSourceRight + SourceBufferFrameCount, 0, ZeroedFramesNeeded * sizeof(float));

				Resampler.ResampleStereo(OutputChunkFrames, FixedSampleRate,
					TempSource, SourceFramesNeeded,
					OutputFrames, OutputChunkFrames);

				FMemory::Free(TempSource);

				SourceBufferFrameCount = 0;
			}

	*/
	struct FVectorLinearResampler
	{
		uint32 CurrentFrameFraction;

		// Returns the number of source frames necessary to generate the requested output
		// count at the given fixed point sample rate (1.0 rate is 65536)
		uint32 SourceFramesNeeded(uint32 OutputFramesNeeded, uint32 FixedPointSampleRate)
		{
			// the last run of the resampler will index [farthest_position >> 16] + 1, so we need
			// that +1 to make it a count from an index.
			uint32 FarthestPosition = CurrentFrameFraction + (OutputFramesNeeded - 1) * FixedPointSampleRate;
			return (FarthestPosition >> 16) + 2;
		}

		// Generate OutputFramesNeeded resampled output at the given fixed point sample rate (1.0 = 65536).
		// Returns the number of source frames to consume. The next run of the resample expects to get
		// SourceFrames + previous runs return value.
		//
		// stereo is deinterleaved, with the right channels being specified by Frames+StrideFloats.
		//
		SIGNALPROCESSING_API uint32 ResampleMono(uint32 OutputFramesNeeded, uint32 FixedPointSampleRate, float const* SourceFrames, float* OutputFrames);
		SIGNALPROCESSING_API uint32 ResampleStereo(uint32 OutputFramesNeeded, uint32 FixedPointSampleRate, float const* SourceFrames, uint32 SourceFramesStrideFloats, float* OutputFrames, uint32 OutputFramesStrideFloats);
	};
}
