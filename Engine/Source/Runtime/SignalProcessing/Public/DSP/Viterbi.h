// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

namespace Audio
{
	/** Interface class for viterbi observations.
	 *
	 * An IViterbiObservations serves as the source of information for the
	 * 	- Number of possible states.
	 * 	- Number of possible timesteps.
	 *  - Emission log probability for a given state and timestep.
	 *
	 *  Note that probabilities are returned as the logarithm of a probability to 
	 *  avoid numerical under/over flows. 
	 */
	class IViterbiObservations
	{
		public:
			virtual ~IViterbiObservations() = default;
			
			/** Returns the number of states at a given states. */
			virtual int32 GetNumStates() const = 0;

			/** Returns the number of given time steps. */
			virtual int32 GetNumTimeSteps() const = 0;

			/** Returns the log probability for a given state at a specific time step.
			 *
			 * @param InTimeStep - The timestep of interest in the range [0 , GetNumTimeSteps() - 1]. 
			 * @param InState - The state of interest in the range [0, GetNumStates() - 1].
			 *
			 * @return The log prability of the emission at the specified timestep for the specified state. 
			 */
			virtual float GetEmissionLogProbability(int32 InTimeStep, int32 InState) const = 0;
	};

	/** Interface class for viterbi initial log probabilities. */
	class IViterbiInitialProbability
	{
		public:
			virtual ~IViterbiInitialProbability() = default;

			/** Returns the log probability of an initial state. 
			 *
			 * @param InState - The state of interest. 
			 *
			 * @return The initial log probability of the state.
			 */
			virtual float GetInitialLogProbability(int32 InState) const = 0;
	};

	/** Interface class for viterbi transition log probabilities. */
	class IViterbiTransitionProbability
	{
		public:
			virtual ~IViterbiTransitionProbability() = default;

			/** Return the log probability of going from one state to another state at a specific time step.
			 *
			 * @param InTimeStep - The current time step.
			 * @param InPreviousState - The state at the previous time step.
			 * @param InCurrentState - The state at the current time step.
			 *
			 * @return The log probability of going from InPreviousState to InCurrentState at timestep InTimeStep.
			 */
			virtual float GetTransitionLogProbability(int32 InTimeStep, int32 InPreviousState, int32 InCurrentState) const = 0;
	};

	struct FViterbi
	{
		/** Decode a sequence using the viterbi algorithm.
		 *
		 * @param InObservations - The emission log probabilities for all observations.
		 * @param InInitialProbability - The initial log probabilities for all states. 
		 * @param InTransitionProbability - The transition log probability for all transitions.
		 *
		 * @return An array of the most likely state sequence determined by decoding using the viterbi algorithm.
		 */
		SIGNALPROCESSING_API static TArray<int32> Decode(const IViterbiObservations& InObservations, const IViterbiInitialProbability& InInitialProbability, const IViterbiTransitionProbability& InTransitionProbability);
	};
}
