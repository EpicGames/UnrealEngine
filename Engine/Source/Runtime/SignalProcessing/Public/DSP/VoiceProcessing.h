// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Delay.h"
#include "DSP/AudioFFT.h"
#include "DSP/MultithreadedPatching.h"

namespace Audio
{
	/**
	 * This class buffers audio while maintaining a running average of the underlying buffer.
	 * This is useful for cases where we can't use a peak detector with asymptotic tracking.
	 * For example: lookahead limiters, silence detection, etc.
	 */
	class FMovingAverager
	{
	public:
		// Delay length in samples.
		SIGNALPROCESSING_API FMovingAverager(uint32 NumSamples);

		// Returns average amplitude across the internal buffer, and fills Output with the delay line output.
		SIGNALPROCESSING_API float ProcessInput(const float& Input, float& Output);

		SIGNALPROCESSING_API void SetNumSamples(uint32 NumSamples);

	private:
		FMovingAverager();

		TArray<float> AudioBuffer;
		int32 BufferCursor;

		float AccumulatedSum;

		// Contended by ProcessInput and SetNumSamples.
		FCriticalSection ProcessCriticalSection;
	};

	/**
	 * Vectorized version of FMovingAverager.
	 */
	class FMovingVectorAverager
	{
	public:
		// Delay length in samples. NumSamples must be divisible by four.
		SIGNALPROCESSING_API FMovingVectorAverager(uint32 NumSamples);

		// Returns average amplitude across the internal buffer, and fills Output with the delay line output.
		SIGNALPROCESSING_API float ProcessAudio(const VectorRegister4Float& Input, VectorRegister4Float& Output);

	private:
		FMovingVectorAverager();

		TArray<VectorRegister4Float> AudioBuffer;
		int32 BufferCursor;

		VectorRegister4Float AccumulatedSum;

		// Contended by ProcessInput and SetNumSamples.
		FCriticalSection ProcessCriticalSection;
	};

	/**
	 * This object will return buffered audio while the input signal is louder than the specified threshold,
	 * and buffer audio when the input signal otherwise.
	 */
	class FSilenceDetection
	{
	public:
		// InOnsetThreshold is the minimum amplitude of a signal before we begin outputting audio, in linear gain.
		// InReleaseThreshold is the amplitude of the signal before we stop outputting audio, in linear gain.
		// AttackDurationInSamples is the amount of samples we average over when calculating our amplitude when the in audio is below the threshold.
		// ReleaseDurationInSamples is the amount of samples we average over when calculating our amplitude when the input audio is above the threshold.
		SIGNALPROCESSING_API FSilenceDetection(float InOnsetThreshold, float InReleaseThreshold, int32 AttackDurationInSamples, int32 ReleaseDurationInSamples);

		// Buffers InAudio and renders any non-silent audio to OutAudio. Returns the number of samples written to OutAudio.
		// The number of samples returned will only be less than NumSamples if the signal becomes audible mid-buffer.
		// We do not return partial buffers when returning from an audible state to a silent state.
		// This should also work in place, i.e. if InAudio == OutAudio.
		SIGNALPROCESSING_API int32 ProcessBuffer(const float* InAudio, float* OutAudio, int32 NumSamples);

		// Set the threshold of audibility, in terms of linear gain.
		SIGNALPROCESSING_API void SetThreshold(float InThreshold);

		// Returns the current estimate of the current amplitude of the input signal, in linear gain.
		SIGNALPROCESSING_API float GetCurrentAmplitude();
	private:
		FSilenceDetection();

		FMovingVectorAverager Averager;
		float ReleaseTau;
		float OnsetThreshold;
		float ReleaseThreshold;
		float CurrentAmplitude;
		bool bOnsetWasInLastBuffer;
	};

	/**
	 * This object accepts an input buffer and current amplitude estimate of that input buffer,
	 * Then applies a computed gain target. Works like a standard feed forward limiter, with a threshold of 0.
	 */
	class FSlowAdaptiveGainControl
	{
	public:
		// InGainTarget is our target running linear gain.
		// InAdaptiveRate is the time it will take to respond to changes in amplitude, in numbers of buffer callbacks.
		// InGainMin is the most we will attenuate the input signal.
		// InGainMax is the most we will amplify the input signal.
		SIGNALPROCESSING_API FSlowAdaptiveGainControl(float InGainTarget, int32 InAdaptiveRate, float InGainMin = 0.01f, float InGainMax = 2.0f);
		
		// Takes an amplitude estimate and an input buffer, and attenuates InAudio based on it.
		// Returns the linear gain applied to InAudio.
		SIGNALPROCESSING_API float ProcessAudio(float* InAudio, int32 NumSamples, float InAmplitude);

		// Sets the responsiveness of the adaptive gain control, in number of buffer callbacks.
		SIGNALPROCESSING_API void SetAdaptiveRate(int32 InAdaptiveRate);

	private:
		FSlowAdaptiveGainControl();

		float GetTargetGain(float InAmplitude);

		FMovingAverager PeakDetector;
		float GainTarget;
		float PreviousGain;
		float GainMin;
		float GainMax;
	};
}
