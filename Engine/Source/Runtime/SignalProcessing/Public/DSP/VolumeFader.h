// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"


namespace Audio
{
	/** Type of fade to use when adjusting a fader's volume over time. */
	enum class EFaderCurve : uint8
	{
		// Linear Fade
		Linear,

		// Logarithmic Fade
		Logarithmic,

		// S-Curve, Sinusoidal Fade
		SCurve,

		// Equal Power, Sinusoidal Fade
		Sin,

		Count
	};


	/** Control-rate fader for managing volume fades of various standard shapes. */
	class FVolumeFader
	{
	public:
		SIGNALPROCESSING_API FVolumeFader();

		/**
		 * Activates the fader if currently deactivated. Sets duration to indefinite (-1.0f)
		 */
		SIGNALPROCESSING_API void Activate();

		/**
		 * Deactivates the fader, causing it to remain
		 * at the current value and disregard update.
		 */
		SIGNALPROCESSING_API void Deactivate();

		/**
		 * Gets time fader is to remain active (-1.0f if always active).
		 */
		SIGNALPROCESSING_API float GetActiveDuration() const;

		/**
		 * Returns current volume of fader
		 */
		SIGNALPROCESSING_API float GetVolume() const;

		/**
		 * Returns the volume given the delta from the current time
		 * into the future (Effectively like running to retrieve volume
		 * but without actually updating internal state).
		 */
		SIGNALPROCESSING_API float GetVolumeAfterTime(float InDeltaTime) const;

		/**
		 * Returns the duration of the fade.
		 */
		SIGNALPROCESSING_API float GetFadeDuration() const;

		/**
		 * Returns the curve type of the fader
		 */
		SIGNALPROCESSING_API EFaderCurve GetCurve() const;

		/**
		 * Returns the target volume of the fader
		 */
		SIGNALPROCESSING_API float GetTargetVolume() const;

		/**
		 * Whether or not the fader is active.
		 */
		SIGNALPROCESSING_API bool IsActive() const;

		/**
		 * Returns whether or not the fader is currently
		 * fading over time.
		 */
		SIGNALPROCESSING_API bool IsFading() const;

		/**
		 * Returns whether or not the fader is currently
		 * fading over time and value is increasing.
		 */
		SIGNALPROCESSING_API bool IsFadingIn() const;

		/**
		 * Returns whether or not the fader is currently
		 * fading over time and value is decreasing.
		 */
		SIGNALPROCESSING_API bool IsFadingOut() const;

		/**
		 * Sets the duration the fader is to be active in the future,
		 * after which point the fader is disabled.  When disabled,
		 * fader will hold the current fade state until activated,
		 * at which point it will continue applying the fade over the
		 * remainder of the fade duration.
		 */
		SIGNALPROCESSING_API void SetActiveDuration(float InDuration);

		/**
		 * Sets the volume immediately, interrupting any currently active fade.
		 */
		SIGNALPROCESSING_API void SetVolume(float InVolume);

		/**
		 * Applies a volume fade over time with the provided parameters.
		 */
		SIGNALPROCESSING_API void StartFade(float InVolume, float InDuration, EFaderCurve InCurve);

		/**
		 * Stops fade, maintaining the current value as the target.
		 */
		SIGNALPROCESSING_API void StopFade();

		/**
		 * Updates the fader's state with the given delta in time since last update.
		 */
		SIGNALPROCESSING_API void Update(float InDeltaTime);

	private:
		/** Converts value to final resulting volume */
		static float AlphaToVolume(float InAlpha, EFaderCurve InCurve, float InStartingValue, float InTarget);

		/** Current value used to linear interpolate over update delta
		  * (Normalized value for non-log, -80dB to 0dB for log)
		  */
		float Alpha;

		/** Target value used to linear interpolate over update delta
		  * (Normalized value for non-log, -80dB to 0dB for log)
		  */
		float Target;

		/** Duration fader is to be active */
		float ActiveDuration;

		/** Duration fader is to perform fade */
		float FadeDuration;

		/** Time elapsed since fade has been initiated */
		float Elapsed;

		/** Audio fader curve to use */
		EFaderCurve FadeCurve;

		/** Initial value from the start of the fade */
		float StartingValue;
	};
} // namespace Audio
