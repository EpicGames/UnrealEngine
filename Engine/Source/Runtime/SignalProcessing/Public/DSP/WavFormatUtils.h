// Copyright Epic Games, Inc. All Rights Reserved.

// Move to WaveUtils or something.
#pragma once

#include "HAL/Platform.h"
#include "Containers/Array.h"
#include "AudioMixerChannel.h"

// Forward declare
enum EAudioSpeakers : int32;

#define UE_API SIGNALPROCESSING_API

namespace WavFormatUtils
{
	/**
	 * Convert Array of channel names into AudioMixer enum entries. If they match.
	 * @param InChannelIds Array of Channel Names. 
	 * @param OutChannels  Array of AudioMixerChannel enum entries.
	 * @return true if all channels correctly mapped to an enum entry. false if it was a partial/fail. 
	 */
	UE_API bool ChannelIdToMixerChannels(const TArray<FName>& InChannelIds, TArray<EAudioMixerChannel::Type>& OutChannels);

	/**
	 * Convert Array of AudioMixerChannel enum entries to an array of channel Ids (names).
	 * @param InMixerChannels Array of AudioMixerChannel entries.
	 * @param OutChannelIds Array of channel Ids (names).
	 */
	UE_API void MixerChannelToChannelIds(const TArray<EAudioMixerChannel::Type>& InMixerChannels, TArray<FName>& OutChannelIds);

	/**
	 * Convert a single speaker to a string that describes the mask. 
	 * @param InSingleSpeakerMask Bitfield containing a single bit that describes which channel you want a description for. 
	 * @return Pointer the text description or "Unknown WavBitMask".
	 */
	UE_API const TCHAR* LexToStringWavChannelMask(const uint32 InSingleSpeakerMask);

	/**
	 * Returns a string for the "common" name of this channel mask. Examples being. "Stereo (2.0)" etc. 
	 * @param InChannelMask The channel mask to query.
	 * @return Pointer to the text description. Or "Unknown WavFormat" if it fails to find one.
	 */
	UE_API const TCHAR* LexToStringCommonFormat(const uint32 InChannelMask);

	/**
	 * Get a short version of the AudioMixerChannel. i.e. FrontLeft -> FL.
	 * @param InChannel AudioMixerEntry to query.
	 * @return Pointer to the Description. "Unknown" otherwise.
	 */
	UE_API const TCHAR* LexToShortString(const EAudioMixerChannel::Type InChannel);

	/**
	 * Returns a string for the "common" name of this channel mask. Examples being. "Stereo (2.0)" etc.  
	 * @param InChannelIds Array of Channel ID (Names).
	 * @return Pointer to the text description. Or "Unknown WavFormat" if it fails to find one. 
	 */
	UE_API const TCHAR* LexToStringCommonFormat(const TArray<FName>& InChannelIds);

	/**
	 * Returns a string for the "common" name of this channel mask. Examples being. "Stereo (2.0)" etc.  
	 * @param InChannels Array of AudioMixerChannels. (enums).
	 * @return Pointer to the text description. Or "Unknown WavFormat" if it fails to find one.  
	 */
	UE_API const TCHAR* LexToStringCommonFormat(const TArray<EAudioMixerChannel::Type>& InChannels);

	/**
	 * Make a pretty string for logging of the Channel Mask. i.e. "FrontLeft, FrontRight"
	 * @param InChannelMask ChannelMask to use
	 * @return A Generated String.
	 */
	UE_API FString MakePrettyString(const uint32 InChannelMask);

	/**
	 * Make pretty string for logging of the AudioMixerChannel array. i.e. "FrontLeft, FrontRight".
	 * @param InArray of AudioMixerChannel entries.
	 * @return A Generated String.
	 */
	UE_API FString MakePrettyString(const TArray<EAudioMixerChannel::Type>& InArray);

	/**
	 * Make pretty string for logging of the AudioMixerChannel array. i.e. "FrontLeft, FrontRight". 
	 * @param InChannelIDs Array of ChannelIds. (names). 
	 * @return A Generated String. 
	 */
	UE_API FString MakePrettyString(const TArray<FName>& InChannelIDs );

	/**
	 * Makes a pretty "short" string, i.e. abbreviated form. "FL, FR"
	 * @return A Generated String.  
	 */
	UE_API FString MakePrettyShortString(const uint32 InChannelMask);

	/**
	 * Makes a pretty "short" string, i.e. abbreviated form. "FL, FR" 
	 * @param InChannels Array of AudioMixerChannel entries.
	 * @return A Generated String.
	 */
	UE_API FString MakePrettyShortString(const TArray<EAudioMixerChannel::Type>& InChannels);

	/**
	 * Makes a pretty "short" string, i.e. abbreviated form. "FL, FR"
	 * NOTE: If this fails to map the AudioMixerChannels, it will just render the long form instead.
	 * @param InChannelIds Array of Channel IDs (names). 
	 * @return  A Generated String.
	 */
	UE_API FString MakePrettyShortString(const TArray<FName>& InChannelIds);

	/**
	 * Converts a Channel Mask into an Array of AudioMixerChannels 
	 * @param InChannelMask The ChannelMask
	 * @return Array of AudioMixerChannels.
	 */
	UE_API TArray<EAudioMixerChannel::Type> ChannelMaskToMixerChannels(const uint32 InChannelMask);

	/**
	 * Convert a single bit of a channel mask to AudioMixerChannel. NOTE. > 1 bit will assert.
	 * @param InChannelMask The Single Channel mask.
	 * @return AudioMixerChannel enum.
	 */
	UE_API EAudioMixerChannel::Type ChannelMaskToMixerChannel(const uint32 InChannelMask);

	/**
	 * Convert an Array of AudioMixerChannels to a ChannelMask
	 * @param InChannels Array of AudioMixerChannel 
	 * @return the ChannelMask.
	 */
	UE_API uint32 MixerChannelsToChannelMask(const TArray<EAudioMixerChannel::Type>& InChannels);

	/**
	 * NumChannels -> Array of AudioMixerChannels (ref version).
	 * NOTE: This is an optimized form of NumChannelsToMixerChannels which caches statically the results. 
	 *  As a result this is limited to <= 8 channels. > 8 will assert.
	 * @param InNumChannels Num of Channels to use for lookup.
	 * @return Array of AudioMixerChannels (const reference)
	 */
	UE_API const TArray<EAudioMixerChannel::Type>& NumChannelsToMixerChannelsRef(const int32 InNumChannels);

	/**
	 * NumChannels -> Array of AudioMixerChannels. 
	 * Attempts to map NumChannels to a particular AudioMixerChannel order. This is mainly for legacy reasons,
	 * and a Bitmask/Array of channels is preferred for ambiguity reasons.
	 * @param InNumChannels Num of Channels to use for lookup.
	 * @return Array of AudioMixerChannels.
	 */
	UE_API TOptional<TArray<EAudioMixerChannel::Type>> NumChannelsToMixerChannels(const int32 InNumChannels);

	/**
	 * NumChannels -> Bitmask
	 * Attempts to map Num Of Channels to a bitmask. This is mainly for legacy reasons, and a bitmask/array 
	 * is preferred for ambiguity reasons.
	 * @param InNumChannels Num of Chanels to use for lookup.
	 * @return Optional bitmask. Optional will not be set if a good mapping can't be found. 
	 */
	UE_API TOptional<uint32> NumChannelsToCommonChannelMask(const int32 InNumChannels);
	
	// Back compat with old EAudioSpeakers enum.
	// These will likely not be necessary, as we will be deprecating EAudioSpeakers enum shortly.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_API const TCHAR* LexToString(const EAudioSpeakers Speaker);
	UE_API const TCHAR* LexToShortString(const EAudioSpeakers InSpeaker);
	UE_API FString MakePrettyString(const TArray<EAudioSpeakers>& InSpeakers);
	UE_API FString MakePrettyShortString(const TArray<EAudioSpeakers>& InSpeakers);
	UE_API TArray<EAudioSpeakers> ChannelMaskToSpeakers(const uint32 InChannelMask);
	UE_API uint32 SpeakersToChannelMask(const TArray<EAudioSpeakers>& InSpeakers);
	UE_API TArray<EAudioMixerChannel::Type> SpeakersToMixerChannels(const TArray<EAudioSpeakers>& InSpeakers);
	UE_API bool MixerChannelsToSpeakers(const TArray<EAudioMixerChannel::Type>& InChannelIds, TArray<EAudioSpeakers>& OutSpeakers);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
}

#undef UE_API