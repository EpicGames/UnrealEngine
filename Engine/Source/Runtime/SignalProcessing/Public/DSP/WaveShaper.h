// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	enum class EWaveShaperType : int32
	{
		Sin,
		ATan,
		Tanh,
		Cubic,
		HardClip
	};

	// A digital wave shaping effect to cause audio distortion
	// https://en.wikipedia.org/wiki/Waveshaper
	class FWaveShaper
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FWaveShaper();

		// Destructor
		SIGNALPROCESSING_API ~FWaveShaper();

		// Initialize the equalizer
		SIGNALPROCESSING_API void Init(const float InSampleRate);

		// Sets the amount of wave shapping. 0.0 is no effect.
		SIGNALPROCESSING_API void SetAmount(const float InAmount);

		// Set DC offset before processing
		SIGNALPROCESSING_API void SetBias(const float InBias);

		// Sets the output gain of the waveshaper
		SIGNALPROCESSING_API void SetOutputGainDb(const float InGainDb);

		// Sets the output gain of the waveshaper
		SIGNALPROCESSING_API void SetOutputGainLinear(const float InGainLinear);

		SIGNALPROCESSING_API void SetType(const EWaveShaperType InType);

		// Processes one Sample, using ATan
		// Deprecating in favor of ProcessAudioBuffer
		SIGNALPROCESSING_API void ProcessAudio(const float InSample, float& OutSample);

		// Process an entire buffer of audio
		SIGNALPROCESSING_API void ProcessAudioBuffer(const float* InBuffer, float* OutBuffer, int32 NumFrames);

	private:

		float Amount;
		float OutputGain;
		float Bias;
		EWaveShaperType Type;

		float OneOverAtanAmount;
		float OneOverTanhAmount;

		void ProcessHardClip(const float* InBuffer, float* OutBuffer, int32 NumSamples);
		void ProcessTanh(const float* InBuffer, float* OutBuffer, int32 NumSamples);
		void ProcessATan(const float* InBuffer, float* OutBuffer, int32 NumSamples);
		void ProcessCubic(const float* InBuffer, float* OutBuffer, int32 NumSamples);
		void ProcessSin(const float* InBuffer, float* OutBuffer, int32 NumSamples);
	};

}
