// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	// Wavetable oscillator types
	namespace EWaveTable
	{
		enum Type
		{
			None,
			SineWaveTable,
			SawWaveTable,
			TriangleWaveTable,
			SquareWaveTable,
			BandLimitedSawWaveTable,
			BandLimitedTriangleWaveTable,
			BandLimitedSquareWaveTable,
			Custom
		};
	}

	class FWaveTableOsc;


	// A wave table oscillator class
	class FWaveTableOsc
	{
	public:
		// Constructor
		SIGNALPROCESSING_API FWaveTableOsc();

		// Virtual Destructor
		SIGNALPROCESSING_API virtual ~FWaveTableOsc();

		// Initialize the wave table oscillator
		SIGNALPROCESSING_API void Init(const float InSampleRate, const float InFrequencyHz);

		// Sets the sample rate of the oscillator.
		SIGNALPROCESSING_API void SetSampleRate(const float InSampleRate);

		// Resets the wave table read indices.
		SIGNALPROCESSING_API void Reset();

		// Sets the amount to scale and add to the output of the wave table
		SIGNALPROCESSING_API void SetScaleAdd(const float InScale, const float InAdd);

		// Returns the type of the wave table oscillator.
		EWaveTable::Type GetType() const { return WaveTableType; }

		// Sets the frequency of the wave table oscillator.
		SIGNALPROCESSING_API void SetFrequencyHz(const float InFrequencyHz);

		// Returns the frequency of the wave table oscillator.
		float GetFrequencyHz() const { return FrequencyHz; }

		// Returns the internal table used in the wave table.
		SIGNALPROCESSING_API TArray<float>& GetTable();
		SIGNALPROCESSING_API const TArray<float>& GetTable() const;

		// Processes the wave table, outputs the normal and quad phase (optional) values 
		SIGNALPROCESSING_API void Generate(float* OutputNormalPhase, float* OutputQuadPhase = nullptr);

		// Creates a wave table using internal factories for standard wave tables or uses custom wave table factor if it exists.
		static SIGNALPROCESSING_API TSharedPtr<FWaveTableOsc> CreateWaveTable(const EWaveTable::Type WaveTableType, const int32 WaveTableSize = 1024);

	protected:
		SIGNALPROCESSING_API void UpdateFrequency();

		// The wave table buffer
		TArray<float> WaveTableBuffer;

		// The frequency of the output (given the sample rate)
		float FrequencyHz;

		// The sample rate of the oscillator
		float SampleRate;

		// Normal phase read index
		float NormalPhaseReadIndex;

		// The quad-phase read index
		float QuadPhaseReadIndex;

		// The phase increment (based on frequency)
		float PhaseIncrement;

		// Amount to scale the output by
		float OutputScale;

		// Amount to add to the output
		float OutputAdd;

		// The wave table oscillator type
		EWaveTable::Type WaveTableType;
	};

}
