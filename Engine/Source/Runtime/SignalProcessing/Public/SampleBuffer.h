// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	typedef int16 DefaultUSoundWaveSampleType;
	
	/************************************************************************/
	/* TSampleBuffer<class SampleType>                                      */
	/* This class owns an audio buffer.                                     */
	/* To convert between fixed Q15 buffers and float buffers,              */
	/* Use the assignment operator. Example:                                */
	/*                                                                      */
	/* TSampleBuffer<float> AFloatBuffer;                                   */
	/* TSampleBuffer<int16> AnIntBuffer = AFloatBuffer;                     */
	/************************************************************************/
	template <class SampleType = DefaultUSoundWaveSampleType>
	class TSampleBuffer
	{
	private:
		// raw PCM data buffer
		TArray<SampleType> RawPCMData;
		// The number of samples in the buffer
		int32 NumSamples;
		// The number of frames in the buffer
		int32 NumFrames;
		// The number of channels in the buffer
		int32 NumChannels;
		// The sample rate of the buffer	
		int32 SampleRate;
		// The duration of the buffer in seconds
		float SampleDuration;

	public:
		// Ensure that we can trivially copy construct private members between templated TSampleBuffers:
		template <class> friend class TSampleBuffer;

		inline TSampleBuffer()
			: NumSamples(0)
			, NumFrames(0)
			, NumChannels(0)
			, SampleRate(0)
			, SampleDuration(0.0f)
		{}

		inline TSampleBuffer(const TSampleBuffer& Other)
		{
			NumSamples = Other.NumSamples;
			NumFrames = Other.NumFrames;
			NumChannels = Other.NumChannels;
			SampleRate = Other.SampleRate;
			SampleDuration = Other.SampleDuration;

			RawPCMData.Reset(NumSamples);
			RawPCMData.AddUninitialized(NumSamples);
			FMemory::Memcpy(RawPCMData.GetData(), Other.RawPCMData.GetData(), NumSamples * sizeof(SampleType));
		}

		inline TSampleBuffer(const FAlignedFloatBuffer& InData, int32 InNumChannels, int32 InSampleRate)
		{
			*this =  TSampleBuffer(InData.GetData(), InData.Num(), InNumChannels, InSampleRate);
		}

		inline TSampleBuffer(const float* InBufferPtr, int32 InNumSamples, int32 InNumChannels, int32 InSampleRate)
		{
			NumSamples = InNumSamples;
			NumFrames = NumSamples / InNumChannels;
			NumChannels = InNumChannels;
			SampleRate = InSampleRate;
			SampleDuration = ((float)NumFrames) / SampleRate;

			RawPCMData.Reset(NumSamples);
			RawPCMData.AddUninitialized(NumSamples);

			if constexpr(std::is_same_v<SampleType, float>)
			{
				FMemory::Memcpy(RawPCMData.GetData(), InBufferPtr, NumSamples * sizeof(float));
			}
			else if constexpr(std::is_same_v<SampleType, int16>)
			{
				// Convert from float to int:
				for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
				{
					RawPCMData[SampleIndex] = (int16)(FMath::Clamp(InBufferPtr[SampleIndex], -1.0f, 1.0f) * 32767.0f);
				}
			}
			else
			{
				// for any other types, we don't know how to explicitly convert, so we fall back to casts:
				for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
				{
					RawPCMData[SampleIndex] = (SampleType)(InBufferPtr[SampleIndex]);
				}
			}
		}

		inline TSampleBuffer(const int16* InBufferPtr, int32 InNumSamples, int32 InNumChannels, int32 InSampleRate)
		{
			NumSamples = InNumSamples;
			NumFrames = NumSamples / InNumChannels;
			NumChannels = InNumChannels;
			SampleRate = InSampleRate;
			SampleDuration = ((float)NumFrames) / SampleRate;

			RawPCMData.Reset(NumSamples);
			RawPCMData.AddUninitialized(NumSamples);

			if constexpr(std::is_same_v<SampleType, int16>)
			{
				FMemory::Memcpy(RawPCMData.GetData(), InBufferPtr, NumSamples * sizeof(int16));
			}
			else if constexpr(std::is_same_v<SampleType, float>)
			{
				// Convert from int to float:
				Audio::ArrayPcm16ToFloat(MakeArrayView(InBufferPtr, NumSamples), RawPCMData);
			}
			else
			{
				// for any other types, we don't know how to explicitly convert, so we fall back to casts:
				for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
				{
					RawPCMData[SampleIndex] = (SampleType)(InBufferPtr[SampleIndex]);
				}
			}
		}

		// Vanilla assignment operator:
		TSampleBuffer& operator=(const TSampleBuffer& Other)
		{
			NumSamples = Other.NumSamples;
			NumFrames = Other.NumFrames;
			NumChannels = Other.NumChannels;
			SampleRate = Other.SampleRate;
			SampleDuration = Other.SampleDuration;

			RawPCMData.Reset(NumSamples);
			RawPCMData.AddUninitialized(NumSamples);

			FMemory::Memcpy(RawPCMData.GetData(), Other.RawPCMData.GetData(), NumSamples * sizeof(SampleType));

			return *this;
		}

		//SampleType converting assignment operator:
		template<class OtherSampleType>
		TSampleBuffer& operator=(const TSampleBuffer<OtherSampleType>& Other)
		{
			NumSamples = Other.NumSamples;
			NumFrames = Other.NumFrames;
			NumChannels = Other.NumChannels;
			SampleRate = Other.SampleRate;
			SampleDuration = Other.SampleDuration;

			RawPCMData.Reset(NumSamples);
			RawPCMData.AddUninitialized(NumSamples);

			if constexpr(std::is_same_v<SampleType, OtherSampleType>)
			{
				// If buffers are of the same type, copy over:
				FMemory::Memcpy(RawPCMData.GetData(), Other.RawPCMData.GetData(), NumSamples * sizeof(SampleType));
			}
			else if constexpr(std::is_same_v<SampleType, int16> && std::is_same_v<OtherSampleType, float>)
			{
				// Convert from float to int:
				Audio::ArrayFloatToPcm16(MakeArrayView(Other.RawPCMData), MakeArrayView(RawPCMData));
			}
			else if constexpr(std::is_same_v<SampleType, float> && std::is_same_v<OtherSampleType, int16>)
			{
				// Convert from int to float:
				Audio::ArrayPcm16ToFloat(MakeArrayView(Other.RawPCMData), MakeArrayView(RawPCMData));
			}
			else
			{
				// for any other types, we don't know how to explicitly convert, so we fall back to casts:
				for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
				{
					RawPCMData[SampleIndex] = (SampleType)Other.RawPCMData[SampleIndex];
				}
			}

			return *this;
		}

		// copy from a container of the same element type
		void CopyFrom(const TArray<SampleType>& InArray, int32 InNumChannels, int32 InSampleRate)
		{
			NumSamples = InArray.Num();
			NumFrames = NumSamples / InNumChannels;
			NumChannels = InNumChannels;
			SampleRate = InSampleRate;
			SampleDuration = ((float)NumFrames) / SampleRate;

			RawPCMData.Reset(NumSamples);
			RawPCMData.AddZeroed(NumSamples);

			FMemory::Memcpy(RawPCMData.GetData(), InArray.GetData(), NumSamples * sizeof(SampleType));
		}

		// Append audio data to internal buffer of different sample type of this sample buffer
		template<class OtherSampleType>
		void Append(const OtherSampleType* InputBuffer, int32 InNumSamples)
		{
			int32 StartIndex = RawPCMData.AddUninitialized(InNumSamples);

			if constexpr(std::is_same_v<SampleType, OtherSampleType>)
			{
				FMemory::Memcpy(&RawPCMData[StartIndex], InputBuffer, InNumSamples * sizeof(SampleType));
			}
			else
			{
				if constexpr(std::is_same_v<SampleType, int16> && std::is_same_v<OtherSampleType, float>)
				{
					// Convert from float to int:
					Audio::ArrayFloatToPcm16(MakeArrayView(InputBuffer, InNumSamples), MakeArrayView(&RawPCMData[StartIndex], InNumSamples));
				}
				else if constexpr(std::is_same_v<SampleType, float> && std::is_same_v<OtherSampleType, int16>)
				{
					// Convert from int to float:
					Audio::ArrayPcm16ToFloat(MakeArrayView(InputBuffer, InNumSamples), MakeArrayView(&RawPCMData[StartIndex], InNumSamples));
				}
				else
				{
					// for any other types, we don't know how to explicitly convert, so we fall back to casts:
					for (int32 SampleIndex = 0; SampleIndex < InNumSamples; SampleIndex++)
					{
						RawPCMData[StartIndex + SampleIndex] = InputBuffer[SampleIndex];
					}
				}
			}

			// Update meta-data
			NumSamples += InNumSamples;
			NumFrames = NumSamples / NumChannels;
			SampleDuration = (float)NumFrames / SampleRate;
		}

		// Overload of Append that also sets the number of channels and sample rate.
		template<class OtherSampleType>
		void Append(const OtherSampleType* InputBuffer, int32 InNumSamples, int32 InNumChannels, int32 InSampleRate)
		{
			NumChannels = InNumChannels;
			SampleRate = InSampleRate;

			Append(InputBuffer, InNumSamples);
		}

		~TSampleBuffer() = default;

		void Reset()
		{
			RawPCMData.Reset();
			NumSamples = 0;
			NumFrames = 0;
			NumChannels = 0;
			SampleRate = 0.0f;
			SampleDuration = 0.0f;
		}

		// Gets the raw PCM data of the sound wave
		inline const SampleType* GetData() const
		{
			return RawPCMData.GetData();
		}

		inline TArrayView<SampleType> GetArrayView()
		{
			return MakeArrayView(RawPCMData);
		}

		inline TArrayView<const SampleType> GetArrayView() const
		{
			return MakeArrayView(RawPCMData);
		}

		// Gets the number of samples of the sound wave
		inline int32 GetNumSamples() const
		{
			return NumSamples;
		}

		// Gets the number of frames of the sound wave
		inline int32 GetNumFrames() const
		{
			return NumFrames;
		}

		// Gets the number of channels of the sound wave
		inline int32 GetNumChannels() const
		{
			return NumChannels;
		}

		// Gets the sample rate of the sound wave
		inline int32 GetSampleRate() const
		{
			return SampleRate;
		}

		inline float GetSampleDuration() const
		{
			return SampleDuration;
		}

		void MixBufferToChannels(int32 InNumChannels)
		{
			if (!RawPCMData.Num() || InNumChannels <= 0)
			{
				return;
			}

			TUniquePtr<SampleType[]> TempBuffer;
			TempBuffer.Reset(new SampleType[InNumChannels * NumFrames]);
			FMemory::Memset(TempBuffer.Get(), 0, InNumChannels * NumFrames * sizeof(SampleType));

			const SampleType* SrcBuffer = GetData();

			// Downmixing using the channel modulo assumption:
			// TODO: Use channel matrix for channel conversions.
			for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
			{
				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
				{
					const int32 DstSampleIndex = FrameIndex * InNumChannels + (ChannelIndex % InNumChannels);
					const int32 SrcSampleIndex = FrameIndex * NumChannels + ChannelIndex;

					TempBuffer[DstSampleIndex] += SrcBuffer[SrcSampleIndex];
				}
			}

			NumChannels = InNumChannels;
			NumSamples = NumFrames * NumChannels;
			
			// Resize our buffer and copy the result in:
			RawPCMData.Reset(NumSamples);
			RawPCMData.AddUninitialized(NumSamples);

			FMemory::Memcpy(RawPCMData.GetData(), TempBuffer.Get(), NumSamples * sizeof(SampleType));
		}

		void Clamp(float Ceiling = 1.0f)
		{
			if constexpr(std::is_same_v<SampleType, float>)
			{
				// Float case:
				float ClampMin = Ceiling * -1.0f;

				for (int32 SampleIndex = 0; SampleIndex < RawPCMData.Num(); SampleIndex++)
				{
					RawPCMData[SampleIndex] = static_cast<SampleType>(FMath::Clamp<float>(RawPCMData[SampleIndex], ClampMin, Ceiling));
				}
			}
			else if constexpr(std::is_same_v<SampleType, int16>)
			{
				// int16 case:
				Ceiling = FMath::Clamp(Ceiling, 0.0f, 1.0f);

				int16 ClampMax = static_cast<int16>(Ceiling * 32767.0f);
				int16 ClampMin = static_cast<int16>(Ceiling * -32767.0f);

				for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
				{
					RawPCMData[SampleIndex] = FMath::Clamp<int16>(RawPCMData[SampleIndex], ClampMin, ClampMax);
				}
			}
			else
			{
				// Unknown type case:
				float ClampMin = Ceiling * -1.0f;

				for (int32 SampleIndex = 0; SampleIndex < RawPCMData.Num(); SampleIndex++)
				{
					RawPCMData[SampleIndex] = static_cast<SampleType>(FMath::Clamp<SampleType>(RawPCMData[SampleIndex], ClampMin, Ceiling));
				}
			}
		}

		/**
		 * Appends zeroes to the end of this buffer.
		 * If called with no arguments or NumFramesToAppend = 0, this will ZeroPad
		 */
		void ZeroPad(int32 NumFramesToAppend = 0)
		{
			if (!NumFramesToAppend)
			{
				NumFramesToAppend = FMath::RoundUpToPowerOfTwo(NumFrames) - NumFrames;
			}

			RawPCMData.AddZeroed(NumFramesToAppend * NumChannels);
			NumFrames += NumFramesToAppend;
			NumSamples = NumFrames * NumChannels;
		}

		void SetNumFrames(int32 InNumFrames)
		{
			RawPCMData.SetNum(InNumFrames * NumChannels);
			NumFrames = RawPCMData.Num() / NumChannels;
			NumSamples = RawPCMData.Num();
		}

		// InIndex [0.0f, NumSamples - 1.0f]
		// OutFrame is the multichannel output for one index value
		// Returns InIndex wrapped between 0.0 and NumFrames
		float GetAudioFrameAtFractionalIndex(float InIndex, TArray<SampleType>& OutFrame) const
		{
			InIndex = FMath::Fmod(InIndex, static_cast<float>(NumFrames));

			GetAudioFrameAtFractionalIndexInternal(InIndex, OutFrame);

			return InIndex;
		}

		// InPhase [0, 1], wrapped, through duration of file (ignores sample rate)
		// OutFrame is the multichannel output for one phase value
		// Returns InPhase wrapped between 0.0 and 1.0
		float GetAudioFrameAtPhase(float InPhase, TArray<SampleType>& OutFrame) const
		{
			InPhase = FMath::Fmod(InPhase, 1.0f);

			GetAudioFrameAtFractionalIndexInternal(InPhase * NumFrames, OutFrame);

			return InPhase;
		}


		// InTimeSec, get the value of the buffer at the given time (uses sample rate)
		// OutFrame is the multichannel output for one time value
		// Returns InTimeSec wrapped between 0.0 and (NumSamples / SampleRate)
		float GetAudioFrameAtTime(float InTimeSec, TArray<SampleType>& OutFrame) const
		{
			if (InTimeSec >= SampleDuration)
			{
				InTimeSec -= SampleDuration;
			}

			check(InTimeSec >= 0.0f && InTimeSec <= SampleDuration);

			GetAudioFrameAtFractionalIndexInternal(NumSamples * (InTimeSec / SampleDuration), OutFrame);

			return InTimeSec;
		}

	private:
		// Internal implementation. Called by all public GetAudioFrameAt_ _ _ _() functions
		// public functions do range checking/wrapping and then call this function
		void GetAudioFrameAtFractionalIndexInternal(float InIndex, TArray<SampleType>& OutFrame) const 
		{
			const float Alpha = FMath::Fmod(InIndex, 1.0f);
			const int32 WholeThisIndex = FMath::FloorToInt(InIndex);
			int32 WholeNextIndex = WholeThisIndex + 1;

			// check for interpolation between last and first frames
			if (WholeNextIndex == NumFrames)
			{
				WholeNextIndex = 0;
			}

			// TODO: if(NumChannels < 4)... do the current (non vectorized) way
			OutFrame.SetNumUninitialized(NumChannels);

			for (int32 i = 0; i < NumChannels; ++i)
			{
				float SampleA, SampleB;

				if constexpr(std::is_same_v<SampleType, float>)
				{
					SampleA = RawPCMData[(WholeThisIndex * NumChannels) + i];
					SampleB = RawPCMData[(WholeNextIndex * NumChannels) + i];
					OutFrame[i] = FMath::Lerp(SampleA, SampleB, Alpha);
				}
				else
				{
					SampleA = static_cast<float>(RawPCMData[(WholeThisIndex * NumChannels) + i]);
					SampleB = static_cast<float>(RawPCMData[(WholeNextIndex * NumChannels) + i]);
					OutFrame[i] = static_cast<SampleType>(FMath::Lerp(SampleA, SampleB, Alpha));
				}
			}

			// TODO: else { do vectorized version }
			// make new function in BufferVectorOperations.cpp
			// (use FMath::Lerp() overload for VectorRegisters)
		}
	};

	// FSampleBuffer is a strictly defined TSampleBuffer that uses the same sample format we use for USoundWaves.
	typedef TSampleBuffer<> FSampleBuffer;
}
