// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Public dependencies
*****************************************************************************/

#include "CoreMinimal.h"

SIGNALPROCESSING_API DECLARE_LOG_CATEGORY_EXTERN(LogSignalProcessing, Log, All);