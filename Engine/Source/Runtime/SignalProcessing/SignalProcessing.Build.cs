// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SignalProcessing : ModuleRules
	{
		public SignalProcessing(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"MathCore",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] 
				{
					"IntelISPC"
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target,
					"UEOgg",
					"Vorbis",
					"VorbisFile",
					"libOpus",
					"UELibSampleRate"
					);

			// This is used to get FSoundQualityInfo struct for IAudioEncoder.
            PrivateIncludePathModuleNames.Add("TargetPlatform");

			// This is used to reference the EAudioMixerChannel enumeration.
            PrivateIncludePathModuleNames.Add("AudioMixerCore");

			// This is used to reference the EAudioSpeakers enumeration.
			PrivateIncludePathModuleNames.Add("Engine");
		}
	}
}
