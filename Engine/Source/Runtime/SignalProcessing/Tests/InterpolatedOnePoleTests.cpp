// Copyright Epic Games, Inc. All Rights Reserved.

#include "DSP/InterpolatedOnePole.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Audio::Tests
{
	static constexpr float TestSampleRate = 48000.0f;
	static constexpr float TestCutoffHz   = 1000.0f;

	// ------------------------------------------------------------------ helpers

	// Run a filter for NumSamples frames of mono constant-value input.
	// Returns the last output sample.
	static float RunLPFConstantMono(FInterpolatedLPF& Filter, float InputValue, int32 NumSamples)
	{
		float Output = 0.0f;
		for (int32 i = 0; i < NumSamples; ++i)
		{
			Filter.ProcessAudioFrame(&InputValue, &Output);
		}
		return Output;
	}

	static float RunHPFConstantMono(FInterpolatedHPF& Filter, float InputValue, int32 NumSamples)
	{
		float Output = 0.0f;
		for (int32 i = 0; i < NumSamples; ++i)
		{
			Filter.ProcessAudioFrame(&InputValue, &Output);
		}
		return Output;
	}

	// ================================================================== LPF ==

	// LPF: DC passthrough — constant input should converge to itself.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedLPFDCPassthrough,
		"System.SignalProcessing.InterpolatedLPF.DCPassthrough",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedLPFDCPassthrough::RunTest(const FString&)
	{
		FInterpolatedLPF Filter;
		Filter.Init(TestSampleRate, 1);
		Filter.StartFrequencyInterpolation(TestCutoffHz);

		// 500 samples is >> 5x the time constant at 1 kHz / 48 kHz
		const float Last = RunLPFConstantMono(Filter, 1.0f, 500);
		UTEST_TRUE("LPF output converges to 1.0 for DC input", FMath::IsNearlyEqual(Last, 1.0f, 1e-3f));
		return true;
	}

	// LPF: Reset() must clear the delay memory so zero input immediately gives zero output.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedLPFResetClearsMemory,
		"System.SignalProcessing.InterpolatedLPF.ResetClearsMemory",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedLPFResetClearsMemory::RunTest(const FString&)
	{
		FInterpolatedLPF Filter;
		Filter.Init(TestSampleRate, 1);
		Filter.StartFrequencyInterpolation(TestCutoffHz);
		RunLPFConstantMono(Filter, 1.0f, 100);

		Filter.Reset();

		float ZeroIn = 0.0f;
		float Output = 0.0f;
		Filter.ProcessAudioFrame(&ZeroIn, &Output);
		UTEST_EQUAL("LPF output is 0 after Reset() + zero input", Output, 0.0f);
		return true;
	}

	// LPF: ProcessAudioBuffer (single call) must produce the same samples as
	//      looping ProcessAudioFrame one-by-one.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedLPFBufferMatchesFrameLoop,
		"System.SignalProcessing.InterpolatedLPF.ProcessBufferMatchesFrameLoop",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedLPFBufferMatchesFrameLoop::RunTest(const FString&)
	{
		constexpr int32 NumSamples = 128;

		// Build a deterministic input with some variation (ramp + sign flip)
		TArray<float> Input;
		Input.SetNumUninitialized(NumSamples);
		for (int32 i = 0; i < NumSamples; ++i)
		{
			Input[i] = ((i % 7) - 3) * 0.1f;
		}

		// Filter A: ProcessAudioBuffer
		FInterpolatedLPF FilterA;
		FilterA.Init(TestSampleRate, 1);
		FilterA.StartFrequencyInterpolation(TestCutoffHz);
		TArray<float> OutputA;
		OutputA.SetNumZeroed(NumSamples);
		FilterA.ProcessAudioBuffer(Input.GetData(), OutputA.GetData(), NumSamples);

		// Filter B: ProcessAudioFrame loop
		FInterpolatedLPF FilterB;
		FilterB.Init(TestSampleRate, 1);
		FilterB.StartFrequencyInterpolation(TestCutoffHz);
		TArray<float> OutputB;
		OutputB.SetNumZeroed(NumSamples);
		for (int32 i = 0; i < NumSamples; ++i)
		{
			FilterB.ProcessAudioFrame(&Input[i], &OutputB[i]);
		}

		for (int32 i = 0; i < NumSamples; ++i)
		{
			if (!FMath::IsNearlyEqual(OutputA[i], OutputB[i], 1e-5f))
			{
				AddError(FString::Printf(TEXT("LPF output mismatch at sample %d: buffer=%.8f frame=%.8f"), i, OutputA[i], OutputB[i]));
				return false;
			}
		}
		return true;
	}

	// LPF: Two-channel filter must process channels independently.
	//      Channel 0 = 1.0, Channel 1 = 0.0 → after settling, Ch0 ≈ 1, Ch1 ≈ 0.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedLPFTwoChannelIndependence,
		"System.SignalProcessing.InterpolatedLPF.TwoChannelIndependence",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedLPFTwoChannelIndependence::RunTest(const FString&)
	{
		FInterpolatedLPF Filter;
		Filter.Init(TestSampleRate, 2);
		Filter.StartFrequencyInterpolation(TestCutoffHz);

		const float Frame[2] = { 1.0f, 0.0f };
		float Out[2] = { 0.0f, 0.0f };
		for (int32 i = 0; i < 500; ++i)
		{
			Filter.ProcessAudioFrame(Frame, Out);
		}

		UTEST_TRUE("LPF 2-ch: Ch0 converges to 1.0", FMath::IsNearlyEqual(Out[0], 1.0f, 1e-3f));
		UTEST_TRUE("LPF 2-ch: Ch1 stays near 0.0",   FMath::IsNearlyEqual(Out[1], 0.0f, 1e-3f));
		return true;
	}

	// LPF: first call to StartFrequencyInterpolation is applied instantly (no ramp).
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedLPFFirstFrequencyChangeIsInstant,
		"System.SignalProcessing.InterpolatedLPF.FirstFrequencyChangeIsInstant",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedLPFFirstFrequencyChangeIsInstant::RunTest(const FString&)
	{
		// Two filters: one using InterpLength=1 on first change (should snap),
		//              one using InterpLength=100 on first change (should also snap).
		FInterpolatedLPF FilterSnap, FilterLong;
		FilterSnap.Init(TestSampleRate, 1);
		FilterLong.Init(TestSampleRate, 1);

		FilterSnap.StartFrequencyInterpolation(TestCutoffHz, 1);
		FilterLong.StartFrequencyInterpolation(TestCutoffHz, 100);

		// After the first call both should have the same cutoff frequency
		UTEST_EQUAL("LPF: first freq change is instant regardless of InterpLength",
			FilterSnap.GetCutoffFrequency(), FilterLong.GetCutoffFrequency());

		// And they should produce identical output (coefficient already at target)
		float Input = 0.5f;
		float OutSnap = 0.0f, OutLong = 0.0f;
		FilterSnap.ProcessAudioFrame(&Input, &OutSnap);
		FilterLong.ProcessAudioFrame(&Input, &OutLong);
		UTEST_EQUAL("LPF: first-frame output identical regardless of initial InterpLength", OutSnap, OutLong);
		return true;
	}

	// ================================================================== HPF ==

	// HPF: DC rejection — constant input must converge to zero output.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedHPFDCRejection,
		"System.SignalProcessing.InterpolatedHPF.DCRejection",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedHPFDCRejection::RunTest(const FString&)
	{
		FInterpolatedHPF Filter;
		Filter.Init(TestSampleRate, 1);
		Filter.StartFrequencyInterpolation(TestCutoffHz);

		const float Last = RunHPFConstantMono(Filter, 1.0f, 500);
		UTEST_TRUE("HPF output converges to 0 for DC input", FMath::Abs(Last) < 1e-3f);
		return true;
	}

	// HPF: Reset() must clear delay memory so zero input immediately gives zero output.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedHPFResetClearsMemory,
		"System.SignalProcessing.InterpolatedHPF.ResetClearsMemory",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedHPFResetClearsMemory::RunTest(const FString&)
	{
		FInterpolatedHPF Filter;
		Filter.Init(TestSampleRate, 1);
		Filter.StartFrequencyInterpolation(TestCutoffHz);
		RunHPFConstantMono(Filter, 1.0f, 100);

		Filter.Reset();

		float ZeroIn = 0.0f;
		float Output = 0.0f;
		Filter.ProcessAudioFrame(&ZeroIn, &Output);
		UTEST_EQUAL("HPF output is 0 after Reset() + zero input", Output, 0.0f);
		return true;
	}

	// HPF: ProcessAudioBuffer must produce the same samples as looping ProcessAudioFrame.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedHPFBufferMatchesFrameLoop,
		"System.SignalProcessing.InterpolatedHPF.ProcessBufferMatchesFrameLoop",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedHPFBufferMatchesFrameLoop::RunTest(const FString&)
	{
		constexpr int32 NumSamples = 128;

		TArray<float> Input;
		Input.SetNumUninitialized(NumSamples);
		for (int32 i = 0; i < NumSamples; ++i)
		{
			Input[i] = ((i % 7) - 3) * 0.1f;
		}

		FInterpolatedHPF FilterA;
		FilterA.Init(TestSampleRate, 1);
		FilterA.StartFrequencyInterpolation(TestCutoffHz);
		TArray<float> OutputA;
		OutputA.SetNumZeroed(NumSamples);
		FilterA.ProcessAudioBuffer(Input.GetData(), OutputA.GetData(), NumSamples);

		FInterpolatedHPF FilterB;
		FilterB.Init(TestSampleRate, 1);
		FilterB.StartFrequencyInterpolation(TestCutoffHz);
		TArray<float> OutputB;
		OutputB.SetNumZeroed(NumSamples);
		for (int32 i = 0; i < NumSamples; ++i)
		{
			FilterB.ProcessAudioFrame(&Input[i], &OutputB[i]);
		}

		for (int32 i = 0; i < NumSamples; ++i)
		{
			if (!FMath::IsNearlyEqual(OutputA[i], OutputB[i], 1e-5f))
			{
				AddError(FString::Printf(TEXT("HPF output mismatch at sample %d: buffer=%.8f frame=%.8f"), i, OutputA[i], OutputB[i]));
				return false;
			}
		}
		return true;
	}

	// HPF: Two-channel filter must process channels independently.
	//      Both channels fed DC 1.0, both should converge to 0.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedHPFTwoChannelIndependence,
		"System.SignalProcessing.InterpolatedHPF.TwoChannelIndependence",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedHPFTwoChannelIndependence::RunTest(const FString&)
	{
		FInterpolatedHPF Filter;
		Filter.Init(TestSampleRate, 2);
		Filter.StartFrequencyInterpolation(TestCutoffHz);

		// Ch0 = DC 1.0, Ch1 = DC -0.5 — both should reject DC
		const float Frame[2] = { 1.0f, -0.5f };
		float Out[2] = { 0.0f, 0.0f };
		for (int32 i = 0; i < 500; ++i)
		{
			Filter.ProcessAudioFrame(Frame, Out);
		}

		UTEST_TRUE("HPF 2-ch: Ch0 converges to 0",  FMath::Abs(Out[0]) < 1e-3f);
		UTEST_TRUE("HPF 2-ch: Ch1 converges to 0",  FMath::Abs(Out[1]) < 1e-3f);
		return true;
	}

	// HPF + LPF: processing the same signal through both filters should confirm
	//            that the HPF output is always (Input - internal_LPF_component),
	//            i.e. HPF output + LPF-component = Input.
	//            We verify this by checking HPF + separate FInterpolatedLPF ≠ Input
	//            (they use different designs) while confirming HPF alone converges correctly.
	//            The practical invariant we test: HPF(DC) → 0 and LPF(DC) → 1 independently.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FInterpolatedLPFAndHPFComplementary,
		"System.SignalProcessing.InterpolatedOnePole.LPFAndHPFComplementary",
		EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	bool FInterpolatedLPFAndHPFComplementary::RunTest(const FString&)
	{
		FInterpolatedLPF LPF;
		FInterpolatedHPF HPF;
		LPF.Init(TestSampleRate, 1);
		HPF.Init(TestSampleRate, 1);
		LPF.StartFrequencyInterpolation(TestCutoffHz);
		HPF.StartFrequencyInterpolation(TestCutoffHz);

		float InputSample = 1.0f;
		float LPFOut = 0.0f, HPFOut = 0.0f;
		for (int32 i = 0; i < 500; ++i)
		{
			LPF.ProcessAudioFrame(&InputSample, &LPFOut);
			HPF.ProcessAudioFrame(&InputSample, &HPFOut);
		}

		// After settling, LPF should pass DC and HPF should block it
		UTEST_TRUE("LPF converges to 1.0 for DC", FMath::IsNearlyEqual(LPFOut, 1.0f, 1e-3f));
		UTEST_TRUE("HPF converges to 0.0 for DC",  FMath::Abs(HPFOut) < 1e-3f);
		return true;
	}

} // namespace Audio::Tests

#endif // WITH_DEV_AUTOMATION_TESTS
